[CoreRedirects]
+PackageRedirects=(OldName="/Script/GLTFExporterRuntime",NewName="/Script/GLTFExporter")
