{
	"FileVersion": 3,
	"Version": 131,
	"VersionName": "1.3.1",
	"FriendlyName": "glTF Exporter",
	"Description": "An exporter for Khronos glTF 2.0.",
	"Category": "Exporters",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"Modules": [
		{
			"Name": "GLTFExporter",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [ "Win64", "Mac", "Linux" ]
		}
	],
	"Plugins": [
		{
			"Name": "VariantManagerContent",
			"Enabled": true
		},
		{
			"Name": "Interchange",
			"Enabled": true,
			"Optional": true,
			"SupportedTargetPlatforms": [
				"Win64",
				"Linux",
				"Mac"
			]
		},
		{
			"Name": "InterchangeAssets",
			"Enabled": true
		}
	]
}
