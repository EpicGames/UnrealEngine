// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Private/Common.ush"

Texture2D Texture;
SamplerState TextureSampler;

void Main(
	in float2 TextureCoordinate : TEXCOORD0,
	out float4 OutColor : SV_Target0
	)
{
	const float4 TextureSample = Texture2DSample(Texture, TextureSampler, TextureCoordinate);
	OutColor = RETURN_COLOR(TextureSample);
}
