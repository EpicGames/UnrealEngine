// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GLTFExporter : ModuleRules
{
	public GLTFExporter(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
			}
		);

		if (Target.Platform == UnrealTargetPlatform.Win64 || Target.Platform == UnrealTargetPlatform.Linux || Target.Platform == UnrealTargetPlatform.Mac)
		{
			PublicDefinitions.Add("GLTF_EXPORT_ENABLE=1");

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"InterchangeImport",
				}
			);
		}
		else
		{
			PublicDefinitions.Add("GLTF_EXPORT_ENABLE=0");
		}

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Json",
				"RenderCore",
				"RHI",
				"ImageWrapper",
				"LevelSequence",
				"MovieScene",
				"MovieSceneTracks",
				"VariantManagerContent",
				"Projects",
				"EngineSettings",
				"Analytics",
				"Landscape",
				"InterchangeEngine",
				"InterchangeCore",
				"InterchangeFactoryNodes",
				"SkeletalMeshDescription"
			}
		);

		if (Target.bBuildEditor)
		{
			// TODO: remove this when we no longer need to include MaterialBakingStructures.h in GLTFMeshData.h
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"MaterialBaking",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"UnrealEd",
					"MessageLog",
					"Slate",
					"SlateCore",
					"MainFrame",
					"InputCore",
					"EditorStyle",
					"PropertyEditor",
					"ToolMenus",
					"ToolWidgets",
					"ContentBrowser",
					"MaterialUtilities",
					"MeshMergeUtilities",
					"MeshDescription",
					"StaticMeshDescription",
					"MeshMergeUtilities",
				}
			);
		}

		bDisableAutoRTFMInstrumentation = true;
	}
}
