// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "GLTFExporterModule.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyle.h"

class FGLTFEditorStyle : public FSlateStyleSet
{
public:

	 FGLTFEditorStyle()
		 : FSlateStyleSet("GLTFEditorStyle")
	 {
	 	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(GLTFEXPORTER_MODULE_NAME);
	 	FSlateStyleSet::SetContentRoot(Plugin->GetBaseDir() / TEXT("Resources"));

	 	Set("Icon16", new FSlateImageBrush(FSlateStyleSet::RootToContentDir(TEXT("Icon16.png")), FVector2D(16.0f, 16.0f)));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	 }

	 virtual ~FGLTFEditorStyle() override
	 {
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	 }

	static FGLTFEditorStyle& Get()
	{
		static FGLTFEditorStyle Singleton;
		return Singleton;
	}
};

#endif
