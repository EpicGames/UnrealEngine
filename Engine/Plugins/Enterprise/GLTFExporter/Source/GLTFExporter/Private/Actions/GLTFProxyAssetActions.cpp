// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Actions/GLTFProxyAssetActions.h"
#include "Actions/GLTFEditorStyle.h"
#include "AssetToolsModule.h"
#include "Materials/MaterialInterface.h"
#include "Options/GLTFProxyOptions.h"
#include "UI/GLTFProxyOptionsWindow.h"
#include "Converters/GLTFMaterialProxyFactory.h"
#include "UserData/GLTFMaterialUserData.h"
#include "ContentBrowserMenuContexts.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "GLTFProxyAssetActions"

FName FGLTFProxyAssetActions::MenuName = TEXT("ContentBrowser.AssetContextMenu.MaterialInterface");
FName FGLTFProxyAssetActions::SectionName = TEXT("GetAssetActions");
FName FGLTFProxyAssetActions::EntryName = TEXT("MenuEntry_CreateProxy");

void FGLTFProxyAssetActions::AddMenuEntry()
{
	UToolMenus* ToolMenus = UToolMenus::TryGet();
	if (ToolMenus == nullptr)
	{
		return;
	}
	
	// Do not add any entry if UGLTFMaterialExportOptions is not supported
	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	TSharedPtr<FPathPermissionList> AssetClassPermissionList = AssetTools.GetAssetClassPathPermissionList(EAssetClassAction::CreateAsset);
	if (AssetClassPermissionList && AssetClassPermissionList->HasFiltering())
	{
		if (!AssetClassPermissionList->PassesFilter(UGLTFMaterialExportOptions::StaticClass()->GetPathName()))
		{
			return;
		}
	}

	UToolMenu* Menu = ToolMenus->ExtendMenu(MenuName);
	FToolMenuSection& Section = Menu->FindOrAddSection(SectionName);
	Section.AddDynamicEntry(EntryName, FNewToolMenuSectionDelegate::CreateStatic(&FGLTFProxyAssetActions::OnConstructMenu));
}

void FGLTFProxyAssetActions::RemoveMenuEntry()
{
	UToolMenus* ToolMenus = UToolMenus::TryGet();
	if (ToolMenus == nullptr)
	{
		return;
	}

	ToolMenus->RemoveEntry(MenuName, SectionName, EntryName);
}

void FGLTFProxyAssetActions::OnConstructMenu(FToolMenuSection& MenuSection)
{
	const TAttribute<FText> Label = LOCTEXT("MenuEntry_CreateProxy", "Create glTF Proxy Material");
	const TAttribute<FText> ToolTip = LOCTEXT("MenuEntry_CreateProxyTooltip", "Creates a proxy version of this material for glTF export.");
	const FSlateIcon Icon = FSlateIcon(FGLTFEditorStyle::Get().GetStyleSetName(), TEXT("Icon16"));
	const FToolMenuExecuteAction Action = FToolMenuExecuteAction::CreateStatic(&FGLTFProxyAssetActions::OnExecuteAction);
	MenuSection.AddMenuEntry("MenuEntry_CreateProxy", Label, ToolTip, Icon, Action);
}

void FGLTFProxyAssetActions::OnExecuteAction(const FToolMenuContext& MenuContext)
{
	const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext);
	if (Context == nullptr)
	{
		return;
	}

	UGLTFProxyOptions* Options = NewObject<UGLTFProxyOptions>();
	if (!SGLTFProxyOptionsWindow::ShowDialog(Options))
	{
		return;
	}

	FGLTFMaterialProxyFactory ProxyFactory(Options);
	for (UMaterialInterface* Material : Context->LoadSelectedObjects<UMaterialInterface>())
	{
		ProxyFactory.RootPath = FPaths::GetPath(Material->GetPathName()) / TEXT("GLTF");
		ProxyFactory.Create(Material);
	}

	ProxyFactory.OpenLog();
}

#undef LOCTEXT_NAMESPACE

#endif
