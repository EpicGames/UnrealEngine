// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_EDITOR

struct FToolMenuSection;
struct FToolMenuContext;

class FGLTFProxyAssetActions
{
public:

	static FName MenuName;
	static FName SectionName;
	static FName EntryName;

	static void AddMenuEntry();
	static void RemoveMenuEntry();

private:

	static void OnConstructMenu(FToolMenuSection& MenuSection);
	static void OnExecuteAction(const FToolMenuContext& MenuContext);
};

#endif
