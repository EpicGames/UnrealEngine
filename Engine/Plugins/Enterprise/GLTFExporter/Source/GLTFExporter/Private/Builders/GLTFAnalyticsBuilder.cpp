// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builders/GLTFAnalyticsBuilder.h"
#include "EngineAnalytics.h"
#include "StaticMeshResources.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Converters/GLTFMeshUtilities.h"
#include "Converters/GLTFMaterialUtilities.h"
#include "Utilities/GLTFCoreUtilities.h"
#include "MaterialShaderType.h"

FGLTFAnalyticsBuilder::FGLTFAnalyticsBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions)
	: FGLTFBufferBuilder(FileName, ExportOptions)
{
}

TArray<FAnalyticsEventAttribute> FGLTFAnalyticsBuilder::GenerateAnalytics() const
{
	int32 NumberOfActors = ActorsRecorded.Num();
	int32 NumberOfStaticMeshes = StaticMeshesRecorded.Num();
	int32 NumberOfSkeletalMeshes = SkeletalMeshesRecorded.Num();
	int32 NumberOfSplineStaticMeshes = SplineStaticMeshesRecorded.Num();
	int32 NumberOfLandscapes = LandscapeComponentsRecorded.Num();
	int32 NumberOfLevelSequences = LevelSequencesRecorded.Num();
	int32 NumberOfAnimSequences = AnimSequencesRecorded.Num();
	int32 NumberOfMaterials = MaterialsRecorded.Num();
	int32 NumberOfTextures = TexturesRecorded.Num();
	int32 NumberOfCameras = CamerasRecorded.Num();
	int32 NumberOfLights = LightsRecorded.Num();
	
	TArray<FAnalyticsEventAttribute> EventAttributes;
	auto Add = [&EventAttributes](const FString& Name, int32 Value)
	{
		EventAttributes.Emplace(Name, LexToString(Value));
	};

	Add(TEXT("NumberOfActors"), NumberOfActors);
	Add(TEXT("NumberOfStaticMeshes"), NumberOfStaticMeshes);
	Add(TEXT("NumberOfSkeletalMeshes"), NumberOfSkeletalMeshes);
	Add(TEXT("NumberOfSplineStaticMeshes"), NumberOfSplineStaticMeshes);
	Add(TEXT("NumberOfLandscapes"), NumberOfLandscapes);
	Add(TEXT("NumberOfLevelSequences"), NumberOfLevelSequences);
	Add(TEXT("NumberOfAnimSequences"), NumberOfAnimSequences);
	Add(TEXT("NumberOfMaterials"), NumberOfMaterials);
	Add(TEXT("NumberOfTextures"), NumberOfTextures);
	Add(TEXT("NumberOfCameras"), NumberOfCameras);
	Add(TEXT("NumberOfLights"), NumberOfLights);

	uint64 NumberOfVertices = 0;
	{
		//Statics:
		for (const UStaticMesh* StaticMesh : StaticMeshesRecorded)
		{
			const FStaticMeshLODResources& RenderData = FGLTFMeshUtilities::GetRenderData(StaticMesh, 0);
			NumberOfVertices += RenderData.GetNumVertices();
		}

		//Skeletals:
		for (const USkeletalMesh* SkeletalMesh : SkeletalMeshesRecorded)
		{
			const FSkeletalMeshLODRenderData& RenderData = FGLTFMeshUtilities::GetRenderData(SkeletalMesh, 0);
			NumberOfVertices += RenderData.GetNumVertices();
		}
	}
	EventAttributes.Emplace(TEXT("NumberOfVertices"), LexToString(NumberOfVertices));

	//Material related Analytics:
	{
		TMap<EMaterialShadingModel, int32> NumberOfShadingModels;
		for (const UMaterialInterface* MaterialInterface : MaterialsRecorded)
		{
			FString WarningMessage;
			EMaterialShadingModel ShadingModel = FGLTFMaterialUtilities::GetShadingModel(MaterialInterface, WarningMessage);

			int32& NumberOfShadingModel = NumberOfShadingModels.FindOrAdd(ShadingModel);
			NumberOfShadingModel++;
		}
		TMap<FString, int32> NumberOfShadingModelsMap;
		for (const TPair<EMaterialShadingModel, int32> NumberOfShadingModel : NumberOfShadingModels)
		{
			FString ShadingModelString = GetShadingModelString(NumberOfShadingModel.Key);
			NumberOfShadingModelsMap.Add(ShadingModelString, NumberOfShadingModel.Value);
		}
		EventAttributes.Emplace(TEXT("NumberOfShadingModels"), NumberOfShadingModelsMap);

		TArray<FString> ExporterSupportedShadingModels;
		constexpr int32 NumMaterials = static_cast<int32>(EGLTFJsonShadingModel::NumShadingModels);
		for (size_t MaterialIndex = 1; MaterialIndex < NumMaterials; MaterialIndex++)//skipping None
		{
			ExporterSupportedShadingModels.Add(FGLTFCoreUtilities::GetShadingModelString(EGLTFJsonShadingModel(MaterialIndex)));
		}
		EventAttributes.Emplace(TEXT("ExporterSupportedShadingModels"), ExporterSupportedShadingModels);
	}

	return EventAttributes;
}

void FGLTFAnalyticsBuilder::RecordActor(const AActor* Object)
{
	ActorsRecorded.Add(Object);
}

void FGLTFAnalyticsBuilder::RecordStaticMesh(const UStaticMesh* Object)
{
	StaticMeshesRecorded.Add(Object);
}

void FGLTFAnalyticsBuilder::RecordSkeletalMesh(const USkeletalMesh* Object)
{
	SkeletalMeshesRecorded.Add(Object);
}

void FGLTFAnalyticsBuilder::RecordSplineStaticMesh(const UStaticMesh* Object)
{
	SplineStaticMeshesRecorded.Add(Object);
}
void FGLTFAnalyticsBuilder::RecordLandscapeComponent(const ULandscapeComponent* Object)
{
	LandscapeComponentsRecorded.Add(Object);
}

void FGLTFAnalyticsBuilder::RecordLevelSequence(const ULevelSequence* Object)
{
	LevelSequencesRecorded.Add(Object);
}

void FGLTFAnalyticsBuilder::RecordAnimSequence(const UAnimSequence* Object)
{
	AnimSequencesRecorded.Add(Object);
}

void FGLTFAnalyticsBuilder::RecordMaterial(const UMaterialInterface* Object)
{
	MaterialsRecorded.Add(Object);
}

void FGLTFAnalyticsBuilder::RecordTexture(const UTexture* Object)
{
	TexturesRecorded.Add(Object);
}

void FGLTFAnalyticsBuilder::RecordCamera(const UCameraComponent* Object)
{
	CamerasRecorded.Add(Object);
}

void FGLTFAnalyticsBuilder::RecordLight(const ULightComponent* Object)
{
	LightsRecorded.Add(Object);
}