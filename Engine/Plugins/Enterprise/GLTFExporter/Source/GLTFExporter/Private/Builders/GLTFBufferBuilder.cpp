// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builders/GLTFBufferBuilder.h"
#include "Builders/GLTFMemoryArchive.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

FGLTFBufferBuilder::FGLTFBufferBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions)
	: FGLTFJsonBuilder(FileName, ExportOptions)
{
}

FGLTFBufferBuilder::~FGLTFBufferBuilder()
{
	if (BufferArchive != nullptr)
	{
		BufferArchive->Close();
	}
}

void FGLTFBufferBuilder::InitializeBuffer()
{
	if (BufferArchive != nullptr)
	{
		return;
	}

	JsonBuffer = AddBuffer();
	BufferArchive = MakeShared<FGLTFMemoryArchive>();

	if (!bIsGLB)
	{
		const FString URI = FPaths::ChangeExtension(FileName, TEXT(".bin"));
		JsonBuffer->URI = AddExternalFile(URI, BufferArchive);
	}
}

const FGLTFMemoryArchive* FGLTFBufferBuilder::GetBufferData() const
{
	return BufferArchive.Get();
}

FGLTFJsonBufferView* FGLTFBufferBuilder::AddBufferView(const void* RawData, uint64 ByteLength, EGLTFJsonBufferTarget BufferTarget, uint8 DataAlignment)
{
	InitializeBuffer();

	uint64 ByteOffset = BufferArchive->Num();

	// Data offset must be a multiple of the size of the glTF component type (given by ByteAlignment).
	const int64 Padding = (DataAlignment - (ByteOffset % DataAlignment)) % DataAlignment;
	if (Padding > 0)
	{
		ByteOffset += Padding;
		BufferArchive->Seek(ByteOffset);
	}

	BufferArchive->Serialize(const_cast<void*>(RawData), ByteLength);
	JsonBuffer->ByteLength = BufferArchive->Tell();

	FGLTFJsonBufferView* JsonBufferView = FGLTFJsonBuilder::AddBufferView();
	JsonBufferView->Buffer = JsonBuffer;
	JsonBufferView->ByteOffset = ByteOffset;
	JsonBufferView->ByteLength = ByteLength;
	JsonBufferView->Target = BufferTarget;

	return JsonBufferView;
}
