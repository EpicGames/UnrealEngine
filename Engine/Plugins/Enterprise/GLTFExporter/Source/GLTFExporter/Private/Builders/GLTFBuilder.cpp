// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builders/GLTFBuilder.h"
#include "Converters/GLTFMeshUtilities.h"
#include "Misc/Paths.h"
#include "UserData/GLTFMaterialUserData.h"

FGLTFBuilder::FGLTFBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions)
	: FileName(FPaths::GetCleanFilename(FileName))
	, bIsGLB(this->FileName.EndsWith(TEXT(".glb"), ESearchCase::IgnoreCase))
	, ExportOptions(SanitizeExportOptions(ExportOptions))
	, ExportOptionsGuard(this->ExportOptions)
{
}

const UMaterialInterface* FGLTFBuilder::ResolveProxy(const UMaterialInterface* Material) const
{
	return ExportOptions->bExportProxyMaterials ? UGLTFMaterialExportOptions::ResolveProxy(Material) : Material;
}

void FGLTFBuilder::ResolveProxies(TArray<const UMaterialInterface*>& Materials) const
{
	if (ExportOptions->bExportProxyMaterials)
	{
		for (const UMaterialInterface*& Material : Materials)
		{
			Material = UGLTFMaterialExportOptions::ResolveProxy(Material);
		}
	}
}

FGLTFMaterialBakeSize FGLTFBuilder::GetBakeSizeForMaterialProperty(const UMaterialInterface* Material, EGLTFMaterialPropertyGroup PropertyGroup) const
{
	FGLTFMaterialBakeSize DefaultValue = ExportOptions->DefaultMaterialBakeSize;
	if (const FGLTFOverrideMaterialBakeSettings* BakeSettings = ExportOptions->DefaultInputBakeSettings.Find(PropertyGroup))
	{
		if (BakeSettings->bOverrideSize)
		{
			DefaultValue = BakeSettings->Size;
		}
	}

	// TODO: add option to ignore override
	return UGLTFMaterialExportOptions::GetBakeSizeForPropertyGroup(Material, PropertyGroup, DefaultValue);
}

TextureFilter FGLTFBuilder::GetBakeFilterForMaterialProperty(const UMaterialInterface* Material, EGLTFMaterialPropertyGroup PropertyGroup) const
{
	TextureFilter DefaultValue = ExportOptions->DefaultMaterialBakeFilter;
	if (const FGLTFOverrideMaterialBakeSettings* BakeSettings = ExportOptions->DefaultInputBakeSettings.Find(PropertyGroup))
	{
		if (BakeSettings->bOverrideFilter)
		{
			DefaultValue = BakeSettings->Filter;
		}
	}

	// TODO: add option to ignore override
	return UGLTFMaterialExportOptions::GetBakeFilterForPropertyGroup(Material, PropertyGroup, DefaultValue);
}

TextureAddress FGLTFBuilder::GetBakeTilingForMaterialProperty(const UMaterialInterface* Material, EGLTFMaterialPropertyGroup PropertyGroup) const
{
	TextureAddress DefaultValue = ExportOptions->DefaultMaterialBakeTiling;
	if (const FGLTFOverrideMaterialBakeSettings* BakeSettings = ExportOptions->DefaultInputBakeSettings.Find(PropertyGroup))
	{
		if (BakeSettings->bOverrideTiling)
		{
			DefaultValue = BakeSettings->Tiling;
		}
	}

	// TODO: add option to ignore override
	return UGLTFMaterialExportOptions::GetBakeTilingForPropertyGroup(Material, PropertyGroup, DefaultValue);
}

int32 FGLTFBuilder::SanitizeLOD(const UStaticMesh* StaticMesh, const UStaticMeshComponent* StaticMeshComponent, int32 LODIndex) const
{
	return LODIndex > 0
		? FMath::Min(LODIndex, FGLTFMeshUtilities::GetMaximumLOD(StaticMesh))
		: FGLTFMeshUtilities::GetLOD(StaticMesh, StaticMeshComponent, ExportOptions->DefaultLevelOfDetail);
}

int32 FGLTFBuilder::SanitizeLOD(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent, int32 LODIndex) const
{
	return LODIndex > 0
		? FMath::Min(LODIndex, FGLTFMeshUtilities::GetMaximumLOD(SkeletalMesh))
		: FGLTFMeshUtilities::GetLOD(SkeletalMesh, SkeletalMeshComponent, ExportOptions->DefaultLevelOfDetail);
}

int32 FGLTFBuilder::SanitizeLOD(const UStaticMesh* StaticMesh, const USplineMeshComponent* SplineMeshComponent, int32 LODIndex) const
{
	return LODIndex > 0
		? FMath::Min(LODIndex, FGLTFMeshUtilities::GetMaximumLOD(StaticMesh))
		: FGLTFMeshUtilities::GetLOD(StaticMesh, SplineMeshComponent, ExportOptions->DefaultLevelOfDetail);
}

const UGLTFExportOptions* FGLTFBuilder::SanitizeExportOptions(const UGLTFExportOptions* Options)
{
	if (Options == nullptr)
	{
		UGLTFExportOptions* NewOptions = NewObject<UGLTFExportOptions>();
		NewOptions->ResetToDefault();
		Options = NewOptions;
	}

	if (!FApp::CanEverRender())
	{
		if (Options->BakeMaterialInputs != EGLTFMaterialBakeMode::Disabled || Options->TextureImageFormat != EGLTFTextureImageFormat::None)
		{
			// TODO: warn the following options requires rendering support and will be overriden
			UGLTFExportOptions* OverridenOptions = DuplicateObject(Options, nullptr);
			OverridenOptions->BakeMaterialInputs = EGLTFMaterialBakeMode::Disabled;
			OverridenOptions->TextureImageFormat = EGLTFTextureImageFormat::None;
			Options = OverridenOptions;
		}
	}

	return Options;
}
