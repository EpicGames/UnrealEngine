// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builders/GLTFContainerBuilder.h"
#include "Builders/GLTFMemoryArchive.h"
#include "Misc/Paths.h"

FGLTFContainerBuilder::FGLTFContainerBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions, const TSet<AActor*>& SelectedActors)
	: FGLTFConvertBuilder(FileName, ExportOptions, SelectedActors)
{
}

bool FGLTFContainerBuilder::WriteInternalArchive(FArchive& Archive)
{
	ProcessSlowTasks();

	if (bIsGLB)
	{
		return WriteGlbArchive(Archive);
	}
	
	return WriteJsonArchive(Archive);
}

bool FGLTFContainerBuilder::WriteAllFiles(const FString& DirPath, uint32 WriteFlags)
{
	FGLTFMemoryArchive Archive;
	if (!WriteInternalArchive(Archive))
	{
		return false;
	}

	const FString FilePath = FPaths::Combine(DirPath, FileName);
	if (!SaveToFile(FilePath, Archive, WriteFlags))
	{
		return false;
	}

	return WriteExternalFiles(DirPath, WriteFlags);
}

bool FGLTFContainerBuilder::WriteAllFiles(const FString& DirPath, TArray<FString>& OutFilePaths, uint32 WriteFlags)
{
	if (!WriteAllFiles(DirPath, WriteFlags))
	{
		return false;
	}

	GetAllFiles(OutFilePaths, DirPath);
	return true;
}

void FGLTFContainerBuilder::GetAllFiles(TArray<FString>& OutFilePaths, const FString& DirPath) const
{
	OutFilePaths.Add(FPaths::Combine(DirPath, FileName));
	GetExternalFiles(OutFilePaths, DirPath);
}

bool FGLTFContainerBuilder::WriteGlbArchive(FArchive& Archive)
{
	FGLTFMemoryArchive JsonData;
	if (!WriteJsonArchive(JsonData))
	{
		return false;
	}

	const TArray64<uint8>* BufferData = GetBufferData();
	return WriteGlb(Archive, JsonData, BufferData);
}

bool FGLTFContainerBuilder::WriteGlb(FArchive& Archive, const TArray64<uint8>& JsonData, const TArray64<uint8>* BinaryData)
{
	constexpr uint32 JsonChunkType = 0x4E4F534A; // "JSON" in ASCII
	constexpr uint32 BinaryChunkType = 0x004E4942; // "BIN" in ASCII

	uint64 FileLength =
		3 * sizeof(uint32) + // header
		2 * sizeof(uint32) + JsonData.Num() + GetChunkPaddingLength(JsonData.Num()); // JSON chuck

	if (BinaryData != nullptr)
	{
		FileLength += 2 * sizeof(uint32) + BinaryData->Num() + GetChunkPaddingLength(BinaryData->Num()); // BIN chuck
	}

	if (FileLength > UINT32_MAX)
	{
		LogError(FString::Printf(TEXT("Final file size (%lld) exceedes maximum supported size (%d) in glTF binary container format (.glb)"), FileLength, UINT32_MAX));
		return false;
	}

	WriteHeader(Archive, static_cast<uint32>(FileLength));
	WriteChunk(Archive, JsonChunkType, JsonData, 0x20);

	if (BinaryData != nullptr)
	{
		WriteChunk(Archive, BinaryChunkType, *BinaryData, 0x0);
	}

	return true;
}

void FGLTFContainerBuilder::WriteHeader(FArchive& Archive, uint32 FileLength)
{
	constexpr uint32 FileSignature = 0x46546C67; // "glTF" in ASCII
	constexpr uint32 FileVersion = 2;

	WriteInt(Archive, FileSignature);
	WriteInt(Archive, FileVersion);
	WriteInt(Archive, FileLength);
}

void FGLTFContainerBuilder::WriteChunk(FArchive& Archive, uint32 ChunkType, const TArray64<uint8>& ChunkData, uint8 PaddingValue)
{
	const uint32 PaddingLength = GetChunkPaddingLength(ChunkData.Num());
	const uint32 ChunkLength = static_cast<uint32>(ChunkData.Num() + PaddingLength);

	WriteInt(Archive, ChunkLength);
	WriteInt(Archive, ChunkType);
	WriteData(Archive, ChunkData);
	WriteFill(Archive, PaddingLength, PaddingValue);
}

void FGLTFContainerBuilder::WriteInt(FArchive& Archive, uint32 Value)
{
	Archive.SerializeInt(Value, MAX_uint32);
}

void FGLTFContainerBuilder::WriteData(FArchive& Archive, const TArray64<uint8>& Data)
{
	Archive.Serialize(const_cast<uint8*>(Data.GetData()), Data.Num());
}

void FGLTFContainerBuilder::WriteFill(FArchive& Archive, uint32 Size, uint8 Value)
{
	while (Size != 0)
	{
		Size--;
		Archive.Serialize(&Value, sizeof(Value));
	}
}

uint32 FGLTFContainerBuilder::GetChunkPaddingLength(uint64 Size)
{
	return (4 - static_cast<uint32>(Size & 3)) & 3;
}
