// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builders/GLTFConvertBuilder.h"
#include "Converters/GLTFMeshUtilities.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "LandscapeComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Converters/GLTFMeshAttributesArray.h"

FGLTFConvertBuilder::FGLTFConvertBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions, const TSet<AActor*>& SelectedActors)
	: FGLTFAnalyticsBuilder(FileName, ExportOptions)
	, SelectedActors(SelectedActors)
{
}

bool FGLTFConvertBuilder::IsSelectedActor(const AActor* Actor) const
{
	return SelectedActors.Num() == 0 || SelectedActors.Contains(Actor);
}

bool FGLTFConvertBuilder::IsRootActor(const AActor* Actor) const
{
	const AActor* ParentActor = Actor->GetAttachParentActor();
	return ParentActor == nullptr || !IsSelectedActor(ParentActor);
}

FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniquePositionAccessor(const FGLTFMeshSection* MeshSection, const FPositionVertexBuffer* VertexBuffer)
{
	if (VertexBuffer == nullptr)
	{
		return nullptr;
	}

	return PositionBufferConverter->GetOrAdd(MeshSection, VertexBuffer);
}

FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniquePositionDeltaAccessor(const FGLTFMeshSection* MeshSection, const TMap<uint32, FVector3f>* TargetPositionDeltas)
{
	if (TargetPositionDeltas == nullptr)
	{
		return nullptr;
	}

	return PositionDeltaBufferConverter->GetOrAdd(MeshSection, TargetPositionDeltas);
}

FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniquePositionAccessor(const FGLTFPositionArray& VertexBuffer)
{
	return PositionBufferConverterRaw->GetOrAdd(VertexBuffer);
}

FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniqueColorAccessor(const FGLTFMeshSection* MeshSection, const FColorVertexBuffer* VertexBuffer)
{
	if (VertexBuffer == nullptr)
	{
		return nullptr;
	}

	return ColorBufferConverter->GetOrAdd(MeshSection, VertexBuffer);
}

FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniqueColorAccessor(const FGLTFColorArray& VertexColorBuffer)
{
	return ColorBufferConverterRaw->GetOrAdd(VertexColorBuffer);
}

FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniqueNormalAccessor(const FGLTFMeshSection* MeshSection, const FStaticMeshVertexBuffer* VertexBuffer)
{
	if (VertexBuffer == nullptr)
	{
		return nullptr;
	}

	return NormalBufferConverter->GetOrAdd(MeshSection, VertexBuffer);
}

FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniqueNormalDeltaAccessor(const FGLTFMeshSection* MeshSection, const TMap<uint32, FVector3f>* TargetNormalDeltas, const bool& bHighPrecision)
{
	if (TargetNormalDeltas == nullptr)
	{
		return nullptr;
	}

	return NormalDeltaBufferConverter->GetOrAdd(MeshSection, TargetNormalDeltas, bHighPrecision);
}

FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniqueNormalAccessor(const FGLTFNormalArray& Normals, const bool& bNormalize)
{
	return NormalBufferConverterRaw->GetOrAdd(Normals, bNormalize);
}

FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniqueTangentAccessor(const FGLTFMeshSection* MeshSection, const FStaticMeshVertexBuffer* VertexBuffer)
{
	if (VertexBuffer == nullptr)
	{
		return nullptr;
	}

	return TangentBufferConverter->GetOrAdd(MeshSection, VertexBuffer);
}

FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniqueTangentAccessor(const FGLTFTangentArray& Tangents)
{
	return TangentBufferConverterRaw->GetOrAdd(Tangents);
}

FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniqueUVAccessor(const FGLTFMeshSection* MeshSection, const FStaticMeshVertexBuffer* VertexBuffer, int32 UVIndex)
{
	if (VertexBuffer == nullptr)
	{
		return nullptr;
	}

	return UVBufferConverter->GetOrAdd(MeshSection, VertexBuffer, UVIndex);
}

FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniqueUVAccessor(const FGLTFUVArray& UVs)
{
	return UVBufferConverterRaw->GetOrAdd(UVs);
}

FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniqueJointAccessor(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, int32 InfluenceOffset)
{
	if (VertexBuffer == nullptr)
	{
		return nullptr;
	}

	return BoneIndexBufferConverter->GetOrAdd(MeshSection, VertexBuffer, InfluenceOffset);
}

FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniqueWeightAccessor(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, int32 InfluenceOffset)
{
	if (VertexBuffer == nullptr)
	{
		return nullptr;
	}

	return BoneWeightBufferConverter->GetOrAdd(MeshSection, VertexBuffer, InfluenceOffset);
}

FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniqueJointAccessor(const FGLTFJointInfluenceArray& BoneIndices)
{
	return BoneIndexBufferConverterRaw->GetOrAdd(BoneIndices);
}

FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniqueWeightAccessor(const FGLTFJointWeightArray& Weights)
{
	return BoneWeightBufferConverterRaw->GetOrAdd(Weights);
}


FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniqueIndexAccessor(const FGLTFMeshSection* MeshSection)
{
	if (MeshSection == nullptr)
	{
		return nullptr;
	}

	return IndexBufferConverter->GetOrAdd(MeshSection);
}

FGLTFJsonAccessor* FGLTFConvertBuilder::AddUniqueIndexAccessor(const FGLTFIndexArray& IndexBuffer, const FString& MeshName)
{
	return IndexBufferConverterRaw->GetOrAdd(IndexBuffer, MeshName);
}

FGLTFJsonMesh* FGLTFConvertBuilder::AddUniqueMesh(const UStaticMesh* StaticMesh, const FGLTFMaterialArray& Materials, int32 LODIndex)
{
	if (StaticMesh == nullptr)
	{
		return nullptr;
	}

	RecordStaticMesh(StaticMesh);

	return StaticMeshConverter->GetOrAdd(StaticMesh, nullptr, Materials, LODIndex);
}

FGLTFJsonMesh* FGLTFConvertBuilder::AddUniqueMesh(const USkeletalMesh* SkeletalMesh, const FGLTFMaterialArray& Materials, int32 LODIndex)
{
	if (SkeletalMesh == nullptr)
	{
		return nullptr;
	}

	RecordSkeletalMesh(SkeletalMesh);

	return SkeletalMeshConverter->GetOrAdd(SkeletalMesh, nullptr, Materials, LODIndex);
}

FGLTFJsonMesh* FGLTFConvertBuilder::AddUniqueMesh(const UMeshComponent* MeshComponent, const FGLTFMaterialArray& Materials, int32 LODIndex)
{
	if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent))
	{
		return AddUniqueMesh(StaticMeshComponent, Materials, LODIndex);
	}

	if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(MeshComponent))
	{
		return AddUniqueMesh(SkeletalMeshComponent, Materials, LODIndex);
	}

	return nullptr;
}

FGLTFJsonMesh* FGLTFConvertBuilder::AddUniqueMesh(const UStaticMeshComponent* StaticMeshComponent, const FGLTFMaterialArray& Materials, int32 LODIndex)
{
	if (StaticMeshComponent == nullptr)
	{
		return nullptr;
	}

	const UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
	if (StaticMesh == nullptr)
	{
		return nullptr;
	}

	RecordStaticMesh(StaticMesh);

	return StaticMeshConverter->GetOrAdd(StaticMesh, StaticMeshComponent, Materials, LODIndex);
}

FGLTFJsonMesh* FGLTFConvertBuilder::AddUniqueMesh(const USkeletalMeshComponent* SkeletalMeshComponent, const FGLTFMaterialArray& Materials, int32 LODIndex)
{
	if (SkeletalMeshComponent == nullptr)
	{
		return nullptr;
	}

	const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
	if (SkeletalMesh == nullptr)
	{
		return nullptr;
	}

	RecordSkeletalMesh(SkeletalMesh);

	return SkeletalMeshConverter->GetOrAdd(SkeletalMesh, SkeletalMeshComponent, Materials, LODIndex);
}

FGLTFJsonMesh* FGLTFConvertBuilder::AddUniqueMesh(const ULandscapeComponent* LandscapeComponent, const UMaterialInterface* LandscapeMaterial)
{
	if (LandscapeComponent == nullptr)
	{
		return nullptr;
	}

	RecordLandscapeComponent(LandscapeComponent);

	return LandscapeConverter->GetOrAdd(LandscapeComponent, LandscapeMaterial);
}

FGLTFJsonMesh* FGLTFConvertBuilder::AddUniqueMesh(const USplineMeshComponent* SplineMeshComponent, const FGLTFMaterialArray& Materials, int32 LODIndex)
{
	if (SplineMeshComponent == nullptr)
	{
		return nullptr;
	}

	const UStaticMesh* StaticMesh = SplineMeshComponent->GetStaticMesh();
	if (StaticMesh == nullptr)
	{
		return nullptr;
	}

	RecordSplineStaticMesh(StaticMesh);

	return SplineMeshConverter->GetOrAdd(StaticMesh, SplineMeshComponent, Materials, LODIndex);
}

const FGLTFMeshData* FGLTFConvertBuilder::AddUniqueMeshData(const UStaticMesh* StaticMesh, const UStaticMeshComponent* StaticMeshComponent, int32 LODIndex)
{
	RecordStaticMesh(StaticMesh);

	return StaticMeshDataConverter->GetOrAdd(StaticMesh, StaticMeshComponent, LODIndex);
}

const FGLTFMeshData* FGLTFConvertBuilder::AddUniqueMeshData(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent, int32 LODIndex)
{
	RecordSkeletalMesh(SkeletalMesh);

	return SkeletalMeshDataConverter->GetOrAdd(SkeletalMesh, SkeletalMeshComponent, LODIndex);
}

FGLTFJsonMaterial* FGLTFConvertBuilder::AddUniqueMaterial(const UMaterialInterface* Material, const UStaticMesh* StaticMesh, int32 LODIndex, int32 MaterialIndex)
{
	// TODO: optimize by skipping mesh data if material doesn't need it
	const FGLTFMeshData* MeshData = AddUniqueMeshData(StaticMesh, nullptr, LODIndex);
	const FGLTFIndexArray SectionIndices = FGLTFMeshUtilities::GetSectionIndices(StaticMesh, MeshData->LODIndex, MaterialIndex);
	return AddUniqueMaterial(Material, MeshData, SectionIndices);
}

FGLTFJsonMaterial* FGLTFConvertBuilder::AddUniqueMaterial(const UMaterialInterface* Material, const USkeletalMesh* SkeletalMesh, int32 LODIndex, int32 MaterialIndex)
{
	// TODO: optimize by skipping mesh data if material doesn't need it
	const FGLTFMeshData* MeshData = AddUniqueMeshData(SkeletalMesh, nullptr, LODIndex);
	const FGLTFIndexArray SectionIndices = FGLTFMeshUtilities::GetSectionIndices(SkeletalMesh, MeshData->LODIndex, MaterialIndex);
	return AddUniqueMaterial(Material, MeshData, SectionIndices);
}

FGLTFJsonMaterial* FGLTFConvertBuilder::AddUniqueMaterial(const UMaterialInterface* Material, const UMeshComponent* MeshComponent, int32 LODIndex, int32 MaterialIndex)
{
	if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent))
	{
		return AddUniqueMaterial(Material, StaticMeshComponent, LODIndex, MaterialIndex);
	}

	if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(MeshComponent))
	{
		return AddUniqueMaterial(Material, SkeletalMeshComponent, LODIndex, MaterialIndex);
	}

	return nullptr;
}

FGLTFJsonMaterial* FGLTFConvertBuilder::AddUniqueMaterial(const UMaterialInterface* Material, const UStaticMeshComponent* StaticMeshComponent, int32 LODIndex, int32 MaterialIndex)
{
	// TODO: optimize by skipping mesh data if material doesn't need it
	const UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
	const FGLTFMeshData* MeshData = AddUniqueMeshData(StaticMesh, StaticMeshComponent, LODIndex);
	const FGLTFIndexArray SectionIndices = FGLTFMeshUtilities::GetSectionIndices(StaticMesh, MeshData->LODIndex, MaterialIndex);
	return AddUniqueMaterial(Material, MeshData, SectionIndices);
}

FGLTFJsonMaterial* FGLTFConvertBuilder::AddUniqueMaterial(const UMaterialInterface* Material, const USkeletalMeshComponent* SkeletalMeshComponent, int32 LODIndex, int32 MaterialIndex)
{
	// TODO: optimize by skipping mesh data if material doesn't need it
	const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
	const FGLTFMeshData* MeshData = AddUniqueMeshData(SkeletalMesh, SkeletalMeshComponent, LODIndex);
	const FGLTFIndexArray SectionIndices = FGLTFMeshUtilities::GetSectionIndices(SkeletalMesh, MeshData->LODIndex, MaterialIndex);
	return AddUniqueMaterial(Material, MeshData, SectionIndices);
}

FGLTFJsonMaterial* FGLTFConvertBuilder::AddUniqueMaterial(const UMaterialInterface* Material, const FGLTFMeshData* MeshData, const FGLTFIndexArray& SectionIndices)
{
	if (Material == nullptr)
	{
		return nullptr;
	}

	RecordMaterial(Material);

	return MaterialConverter->GetOrAdd(Material, MeshData, SectionIndices);
}

FGLTFJsonSampler* FGLTFConvertBuilder::AddUniqueSampler(const UTexture* Texture)
{
	if (Texture == nullptr)
	{
		return nullptr;
	}

	return AddUniqueSampler(Texture->GetTextureAddressX(), Texture->GetTextureAddressY(), Texture->Filter, Texture->LODGroup);
}

FGLTFJsonSampler* FGLTFConvertBuilder::AddUniqueSampler(TextureAddress Address, TextureFilter Filter, TextureGroup LODGroup)
{
	return AddUniqueSampler(Address, Address, Filter, LODGroup);
}

FGLTFJsonSampler* FGLTFConvertBuilder::AddUniqueSampler(TextureAddress AddressX, TextureAddress AddressY, TextureFilter Filter, TextureGroup LODGroup)
{
	return SamplerConverter->GetOrAdd(AddressX, AddressY, Filter, LODGroup);
}

FGLTFJsonTexture* FGLTFConvertBuilder::AddUniqueTexture(const UTexture* Texture)
{
	return AddUniqueTexture(Texture, Texture->SRGB);
}

FGLTFJsonTexture* FGLTFConvertBuilder::AddUniqueTexture(const UTexture2D* Texture)
{
	return AddUniqueTexture(Texture, Texture->SRGB);
}

FGLTFJsonTexture* FGLTFConvertBuilder::AddUniqueTexture(const UTextureRenderTarget2D* Texture)
{
	return AddUniqueTexture(Texture, Texture->SRGB);
}

FGLTFJsonTexture* FGLTFConvertBuilder::AddUniqueTexture(const UTexture* Texture, bool bToSRGB, TextureAddress TextureAddressX, TextureAddress TextureAddressY)
{
	if (const UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
	{
		return AddUniqueTexture(Texture2D, bToSRGB, TextureAddressX, TextureAddressY);
	}

	if (const UTextureRenderTarget2D* RenderTarget2D = Cast<UTextureRenderTarget2D>(Texture))
	{
		return AddUniqueTexture(RenderTarget2D, bToSRGB, TextureAddressX, TextureAddressY);
	}

	return nullptr;
}

FGLTFJsonTexture* FGLTFConvertBuilder::AddUniqueTexture(const UTexture2D* Texture, bool bToSRGB, TextureAddress TextureAddressX, TextureAddress TextureAddressY)
{
	if (Texture == nullptr)
	{
		return nullptr;
	}

	RecordTexture(Texture);

	return Texture2DConverter->GetOrAdd(Texture, bToSRGB, TextureAddressX, TextureAddressY);
}

FGLTFJsonTexture* FGLTFConvertBuilder::AddUniqueTexture(const UTextureRenderTarget2D* Texture, bool bToSRGB)
{
	if (Texture == nullptr)
	{
		return nullptr;
	}

	RecordTexture(Texture);

	return TextureRenderTarget2DConverter->GetOrAdd(Texture, bToSRGB);
}

FGLTFJsonTexture* FGLTFConvertBuilder::AddUniqueTexture(const ULightMapTexture2D* Texture)
{
	if (Texture == nullptr)
	{
		return nullptr;
	}

	return TextureLightMapConverter->GetOrAdd(Texture);
}

FGLTFJsonImage* FGLTFConvertBuilder::AddUniqueImage(TGLTFSharedArray<FColor>& Pixels, FIntPoint Size, bool bIgnoreAlpha, const FString& Name)
{
	return ImageConverter->GetOrAdd(Name, bIgnoreAlpha, Size, Pixels);
}

FGLTFJsonSkin* FGLTFConvertBuilder::AddUniqueSkin(FGLTFJsonNode* RootNode, const USkeletalMesh* SkeletalMesh)
{
	if (RootNode == nullptr || SkeletalMesh == nullptr)
	{
		return nullptr;
	}

	return SkinConverter->GetOrAdd(RootNode, SkeletalMesh);
}

FGLTFJsonSkin* FGLTFConvertBuilder::AddUniqueSkin(FGLTFJsonNode* RootNode, const USkeletalMeshComponent* SkeletalMeshComponent)
{
	if (RootNode == nullptr || SkeletalMeshComponent == nullptr)
	{
		return nullptr;
	}

	const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();

	if (SkeletalMesh == nullptr)
	{
		return nullptr;
	}

	return AddUniqueSkin(RootNode, SkeletalMesh);
}

FGLTFJsonAnimation* FGLTFConvertBuilder::AddUniqueAnimation(FGLTFJsonNode* RootNode, const USkeletalMesh* SkeletalMesh, const UAnimSequence* AnimSequence)
{
	if (RootNode == nullptr || SkeletalMesh == nullptr || AnimSequence == nullptr)
	{
		return nullptr;
	}

	RecordAnimSequence(AnimSequence);

	return AnimationConverter->GetOrAdd(RootNode, SkeletalMesh, AnimSequence);
}

FGLTFJsonAnimation* FGLTFConvertBuilder::AddUniqueAnimation(FGLTFJsonNode* RootNode, const USkeletalMeshComponent* SkeletalMeshComponent)
{
	if (RootNode == nullptr || SkeletalMeshComponent == nullptr)
	{
		return nullptr;
	}

	return AnimationDataConverter->GetOrAdd(RootNode, SkeletalMeshComponent);
}

FGLTFJsonAnimation* FGLTFConvertBuilder::AddUniqueAnimation(const ULevel* Level, const ULevelSequence* LevelSequence)
{
	if (Level == nullptr || LevelSequence == nullptr)
	{
		return nullptr;
	}

	RecordLevelSequence(LevelSequence);

	return LevelSequenceConverter->GetOrAdd(Level, LevelSequence);
}

FGLTFJsonAnimation* FGLTFConvertBuilder::AddUniqueAnimation(const ALevelSequenceActor* LevelSequenceActor)
{
	if (LevelSequenceActor == nullptr)
	{
		return nullptr;
	}

	return LevelSequenceDataConverter->GetOrAdd(LevelSequenceActor);
}

FGLTFJsonNode* FGLTFConvertBuilder::AddUniqueNode(const AActor* Actor)
{
	if (Actor == nullptr)
	{
		return nullptr;
	}

	if (IsSelectedActor(Actor))
	{
		RecordActor(Actor);
	}

	return ActorConverter->GetOrAdd(Actor);
}

FGLTFJsonNode* FGLTFConvertBuilder::AddUniqueNode(const USceneComponent* SceneComponent)
{
	if (SceneComponent == nullptr)
	{
		return nullptr;
	}

	return ComponentConverter->GetOrAdd(SceneComponent);
}

FGLTFJsonNode* FGLTFConvertBuilder::AddUniqueNode(const USceneComponent* SceneComponent, FName SocketName)
{
	if (SceneComponent == nullptr)
	{
		return nullptr;
	}

	return ComponentSocketConverter->GetOrAdd(SceneComponent, SocketName);
}

FGLTFJsonNode* FGLTFConvertBuilder::AddUniqueNode(FGLTFJsonNode* RootNode, const UStaticMesh* StaticMesh, FName SocketName)
{
	if (RootNode == nullptr || StaticMesh == nullptr || SocketName == NAME_None)
	{
		return nullptr;
	}

	return StaticSocketConverter->GetOrAdd(RootNode, StaticMesh, SocketName);
}

FGLTFJsonNode* FGLTFConvertBuilder::AddUniqueNode(FGLTFJsonNode* RootNode, const USkeletalMesh* SkeletalMesh, FName SocketName)
{
	if (RootNode == nullptr || SkeletalMesh == nullptr || SocketName == NAME_None)
	{
		return nullptr;
	}

	return SkeletalSocketConverter->GetOrAdd(RootNode, SkeletalMesh, SocketName);
}

FGLTFJsonNode* FGLTFConvertBuilder::AddUniqueNode(FGLTFJsonNode* RootNode, const USkeletalMesh* SkeletalMesh, int32 BoneIndex)
{
	if (RootNode == nullptr || SkeletalMesh == nullptr || BoneIndex == INDEX_NONE)
	{
		return nullptr;
	}

	return SkeletalBoneConverter->GetOrAdd(RootNode, SkeletalMesh, BoneIndex);
}

FGLTFJsonScene* FGLTFConvertBuilder::AddUniqueScene(const UWorld* World)
{
	if (World == nullptr)
	{
		return nullptr;
	}

	return SceneConverter->GetOrAdd(World);
}

FGLTFJsonCamera* FGLTFConvertBuilder::AddUniqueCamera(const UCameraComponent* CameraComponent)
{
	if (CameraComponent == nullptr)
	{
		return nullptr;
	}

	RecordCamera(CameraComponent);

	return CameraConverter->GetOrAdd(CameraComponent);
}

FGLTFJsonLight* FGLTFConvertBuilder::AddUniqueLight(const ULightComponent* LightComponent)
{
	if (LightComponent == nullptr)
	{
		return nullptr;
	}

	RecordLight(LightComponent);

	return LightConverter->GetOrAdd(LightComponent);
}

FGLTFJsonLightIES* FGLTFConvertBuilder::AddUniqueLightIES(const ULightComponent* LightComponent)
{
	if (LightComponent == nullptr)
	{
		return nullptr;
	}

	//light are recorded on AddUniqueLight.

	return LightIESConverter->GetOrAdd(LightComponent);	
}

FGLTFJsonLightIESInstance* FGLTFConvertBuilder::AddUniqueLightIESInstance(const ULightComponent* LightComponent)
{
	if (LightComponent == nullptr)
	{
		return nullptr;
	}

	//light are recorded on AddUniqueLight.

	return LightIESInstanceConverter->GetOrAdd(LightComponent);
}

FGLTFJsonMaterialVariant* FGLTFConvertBuilder::AddUniqueMaterialVariant(const UVariant* Variant)
{
	if (Variant == nullptr)
	{
		return nullptr;
	}

	return MaterialVariantConverter->GetOrAdd(Variant);
}

FGLTFJsonLightMap* FGLTFConvertBuilder::AddUniqueLightMap(const UStaticMeshComponent* StaticMeshComponent)
{
	if (StaticMeshComponent == nullptr)
	{
		return nullptr;
	}

	return LightMapConverter->GetOrAdd(StaticMeshComponent);
}


void FGLTFConvertBuilder::RegisterObjectVariant(const UObject* Object, const UPropertyValue* Property)
{
	TArray<const UPropertyValue*>& Variants = ObjectVariants.FindOrAdd(Object);
	Variants.AddUnique(Property);
}

const TArray<const UPropertyValue*>* FGLTFConvertBuilder::GetObjectVariants(const UObject* Object) const
{
	return ObjectVariants.Find(Object);
}