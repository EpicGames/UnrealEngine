// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builders/GLTFFileBuilder.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

FGLTFFileBuilder::FGLTFFileBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions)
	: FGLTFTaskBuilder(FileName, ExportOptions)
{
}

FString FGLTFFileBuilder::AddExternalFile(const FString& DesiredURI, const TSharedPtr<FGLTFMemoryArchive>& Archive)
{
	const FString ActualFileName = GetUniqueFileName(SanitizeFileName(DesiredURI));
	ExternalArchives.Add(ActualFileName, Archive);
	return EncodeURI(ActualFileName);
}

void FGLTFFileBuilder::GetExternalFiles(TArray<FString>& OutFilePaths, const FString& DirPath) const
{
	for (const TPair<FString, TSharedPtr<FGLTFMemoryArchive>>& ExternalFile : ExternalArchives)
	{
		const FString FilePath = FPaths::Combine(DirPath, *ExternalFile.Key);
		OutFilePaths.Add(FilePath);
	}
}

const TMap<FString, TSharedPtr<FGLTFMemoryArchive>>& FGLTFFileBuilder::GetExternalArchives() const
{
	return ExternalArchives;
}

bool FGLTFFileBuilder::WriteExternalFiles(const FString& DirPath, uint32 WriteFlags)
{
	for (const TPair<FString, TSharedPtr<FGLTFMemoryArchive>>& ExternalFile : ExternalArchives)
	{
		const FString FilePath = FPaths::Combine(DirPath, *ExternalFile.Key);
		const TArray64<uint8>& FileData = *ExternalFile.Value;

		if (!SaveToFile(FilePath, FileData, WriteFlags))
		{
			return false;
		}
	}

	return true;
}

FString FGLTFFileBuilder::GetUniqueFileName(const FString& InFileName) const
{
	if (!ExternalArchives.Contains(InFileName))
	{
		return InFileName;
	}

	const FString BaseFileName = FPaths::GetBaseFilename(InFileName);
	const FString FileExtension = FPaths::GetExtension(InFileName, true);
	FString UniqueFileName;

	int32 Suffix = 1;
	do
	{
		UniqueFileName = BaseFileName + TEXT('_') + FString::FromInt(Suffix) + FileExtension;
		Suffix++;
	}
	while (ExternalArchives.Contains(UniqueFileName));

	return UniqueFileName;
}

FString FGLTFFileBuilder::SanitizeFileName(const FString& InFileName)
{
	static constexpr TCHAR IllegalChars[] = {
		TEXT('<'),
		TEXT('>'),
		TEXT(':'),
		TEXT('"'),
		TEXT('/'),
		TEXT('\\'),
		TEXT('|'),
		TEXT('?'),
		TEXT('*'),
	};

	FString SanitizedFileName = InFileName;

	for (const TCHAR& IllegalChar : IllegalChars)
	{
		SanitizedFileName.ReplaceCharInline(IllegalChar, TEXT('_'));
	}

	return SanitizedFileName;
}

FString FGLTFFileBuilder::EncodeURI(const FString& InFileName)
{
	// glTF only strictly requires the following reserved characters to be percent-encoded
	static const TCHAR* CharEncodingMap[][2] =
	{
		// Always replace "%" first to avoid double-encoding characters
		{ TEXT("%"), TEXT("%25") },
		{ TEXT(":"), TEXT("%3A") },
		{ TEXT("/"), TEXT("%2F") },
		{ TEXT("?"), TEXT("%3F") },
		{ TEXT("#"), TEXT("%23") },
		{ TEXT("["), TEXT("%5B") },
		{ TEXT("]"), TEXT("%5D") },
		{ TEXT("@"), TEXT("%40") },
		{ TEXT("!"), TEXT("%21") },
		{ TEXT("$"), TEXT("%24") },
		{ TEXT("&"), TEXT("%26") },
		{ TEXT("'"), TEXT("%27") },
		{ TEXT("("), TEXT("%28") },
		{ TEXT(")"), TEXT("%29") },
		{ TEXT("*"), TEXT("%2A") },
		{ TEXT("+"), TEXT("%2B") },
		{ TEXT(","), TEXT("%2C") },
		{ TEXT(";"), TEXT("%3B") },
		{ TEXT("="), TEXT("%3D") },
	};

	FString EncodedURI = InFileName;

	for (int32 Index = 0; Index < GetNum(CharEncodingMap); Index++ )
	{
		const TCHAR* OriginalChar = CharEncodingMap[Index][0];
		const TCHAR* EncodedChar = CharEncodingMap[Index][1];
		// Use ReplaceInline as that won't create a copy of the string if the character isn't found
		EncodedURI.ReplaceInline(OriginalChar, EncodedChar, ESearchCase::CaseSensitive);
	}

	return EncodedURI;
}

bool FGLTFFileBuilder::SaveToFile(const FString& FilePath, const TArray64<uint8>& FileData, uint32 WriteFlags)
{
	if (!FFileHelper::SaveArrayToFile(FileData, *FilePath, &IFileManager::Get(), WriteFlags))
	{
		LogError(FString::Printf(TEXT("Failed to save file: %s"), *FilePath));
		return false;
	}

	return true;
}
