// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builders/GLTFJsonBuilder.h"
#include "Interfaces/IPluginManager.h"
#include "Runtime/Launch/Resources/Version.h"
#include "GeneralProjectSettings.h"

FGLTFJsonBuilder::FGLTFJsonBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions)
	: FGLTFFileBuilder(FileName, ExportOptions)
	, DefaultScene(JsonRoot.DefaultScene)
{
	JsonRoot.Asset.Generator = GetGeneratorString();
	if (ExportOptions->bIncludeCopyrightNotice)
	{
		JsonRoot.Asset.Copyright = GetCopyrightString();
	}
}

bool FGLTFJsonBuilder::WriteJsonArchive(FArchive& Archive)
{
	ValidateAndFixGLTFJson();

	JsonRoot.WriteJson(Archive, !bIsGLB, ExportOptions->bSkipNearDefaultValues ? KINDA_SMALL_NUMBER : 0);
	return true;
}

void FGLTFJsonBuilder::AddExtension(EGLTFJsonExtension Extension, bool bIsRequired)
{
	JsonRoot.Extensions.Used.Add(Extension);
	if (bIsRequired)
	{
		JsonRoot.Extensions.Required.Add(Extension);
	}
}

FGLTFJsonAccessor* FGLTFJsonBuilder::AddAccessor()
{
	return JsonRoot.Accessors.Add();
}

FGLTFJsonAnimation* FGLTFJsonBuilder::AddAnimation()
{
	return JsonRoot.Animations.Add();
}

FGLTFJsonBuffer* FGLTFJsonBuilder::AddBuffer()
{
	return JsonRoot.Buffers.Add();
}

FGLTFJsonBufferView* FGLTFJsonBuilder::AddBufferView()
{
	return JsonRoot.BufferViews.Add();
}

FGLTFJsonCamera* FGLTFJsonBuilder::AddCamera()
{
	return JsonRoot.Cameras.Add();
}

FGLTFJsonImage* FGLTFJsonBuilder::AddImage()
{
	return JsonRoot.Images.Add();
}

FGLTFJsonMaterial* FGLTFJsonBuilder::AddMaterial()
{
	return JsonRoot.Materials.Add();
}

FGLTFJsonMesh* FGLTFJsonBuilder::AddMesh()
{
	return JsonRoot.Meshes.Add();
}

FGLTFJsonNode* FGLTFJsonBuilder::AddNode()
{
	return JsonRoot.Nodes.Add();
}

FGLTFJsonSampler* FGLTFJsonBuilder::AddSampler()
{
	return JsonRoot.Samplers.Add();
}

FGLTFJsonScene* FGLTFJsonBuilder::AddScene()
{
	return JsonRoot.Scenes.Add();
}

FGLTFJsonSkin* FGLTFJsonBuilder::AddSkin()
{
	return JsonRoot.Skins.Add();
}

FGLTFJsonTexture* FGLTFJsonBuilder::AddTexture()
{
	return JsonRoot.Textures.Add();
}

FGLTFJsonLight* FGLTFJsonBuilder::AddLight()
{
	return JsonRoot.Lights.Add();
}

FGLTFJsonLightMap* FGLTFJsonBuilder::AddLightMap()
{
	return JsonRoot.LightMaps.Add();
}

FGLTFJsonLightIES* FGLTFJsonBuilder::AddLightIES()
{
	return JsonRoot.LightIESs.Add();
}

FGLTFJsonLightIESInstance* FGLTFJsonBuilder::AddLightIESInstance()
{
	return JsonRoot.LightIESInstances.Add();
}

FGLTFJsonMaterialVariant* FGLTFJsonBuilder::AddMaterialVariant()
{
	return JsonRoot.MaterialVariants.Add();
}

const FGLTFJsonRoot& FGLTFJsonBuilder::GetRoot() const
{
	return JsonRoot;
}

FString FGLTFJsonBuilder::GetGeneratorString()
{
	return TEXT(EPIC_PRODUCT_NAME) TEXT(" ") ENGINE_VERSION_STRING;
}

FString FGLTFJsonBuilder::GetCopyrightString()
{
	return GetDefault<UGeneralProjectSettings>()->CopyrightNotice;
}


void FGLTFJsonBuilder::ValidateAndFixGLTFJson()
{
	//Sort through Meshes, remove empty ones (without indices/attributes/primitives)
	bool bRemovedElement = false;
	TSet<int32> RemoveOriginalIndices;
	for (int32 MeshIndex = 0, OriginalMeshIndex = 0; MeshIndex < JsonRoot.Meshes.Num(); MeshIndex++, OriginalMeshIndex++)
	{
		if (!JsonRoot.Meshes[MeshIndex]->HasValue())
		{
			RemoveOriginalIndices.Add(OriginalMeshIndex);
			bRemovedElement = true;
		}
	}

	if (bRemovedElement)
	{
		for (size_t NodeIndex = 0; NodeIndex < JsonRoot.Nodes.Num(); NodeIndex++)
		{
			if (JsonRoot.Nodes[NodeIndex]->Mesh)
			{
				if (RemoveOriginalIndices.Contains(JsonRoot.Nodes[NodeIndex]->Mesh->Index))
				{
					JsonRoot.Nodes[NodeIndex]->Mesh = nullptr;
				}
			}
		}

		TArray<int32> RemoveOriginalIndicesSorted = RemoveOriginalIndices.Array();
		RemoveOriginalIndicesSorted.Sort([](int32 A, int32 B) {
			return A > B;
			});

		for (int32 RemoveMeshIndex : RemoveOriginalIndicesSorted)
		{
			JsonRoot.Meshes.Remove(RemoveMeshIndex);
		}

		if (bRemovedElement)
		{
			JsonRoot.Meshes.FixElementIndices();
		}
	}

	
}