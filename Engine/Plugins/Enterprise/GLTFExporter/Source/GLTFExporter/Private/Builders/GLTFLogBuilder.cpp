// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builders/GLTFLogBuilder.h"
#include "GLTFExporterModule.h"
#include "Misc/FeedbackContext.h"
#if WITH_EDITOR
#include "Interfaces/IPluginManager.h"
#include "MessageLogModule.h"
#include "IMessageLogListing.h"
#endif

FGLTFLogBuilder::FGLTFLogBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions)
	: FGLTFBuilder(FileName, ExportOptions)
{
#if WITH_EDITOR
	if (!FApp::IsUnattended())
	{
		const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(GLTFEXPORTER_MODULE_NAME);
		const FString& PluginFriendlyName = Plugin->GetDescriptor().FriendlyName;

		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		LogListing = MessageLogModule.GetLogListing(GLTFEXPORTER_MODULE_NAME);
		LogListing->SetLabel(FText::FromString(PluginFriendlyName));
	}
#endif
}

void FGLTFLogBuilder::LogSuggestion(const FString& Message)
{
	Suggestions.Add(Message);
	PrintToLog(ELogLevel::Suggestion, Message);
}

void FGLTFLogBuilder::LogWarning(const FString& Message)
{
	Warnings.Add(Message);
	PrintToLog(ELogLevel::Warning, Message);
}

void FGLTFLogBuilder::LogError(const FString& Message)
{
	Errors.Add(Message);
	PrintToLog(ELogLevel::Error, Message);
}

const TArray<FString>& FGLTFLogBuilder::GetLoggedSuggestions() const
{
	return Suggestions;
}

const TArray<FString>& FGLTFLogBuilder::GetLoggedWarnings() const
{
	return Warnings;
}

const TArray<FString>& FGLTFLogBuilder::GetLoggedErrors() const
{
	return Errors;
}

bool FGLTFLogBuilder::HasLoggedMessages() const
{
	return Suggestions.Num() + Warnings.Num() + Errors.Num() > 0;
}

void FGLTFLogBuilder::OpenLog() const
{
#if WITH_EDITOR
	if (LogListing != nullptr)
	{
		LogListing->Open();
	}
#endif
}

void FGLTFLogBuilder::ClearLog()
{
	Suggestions.Empty();
	Warnings.Empty();
	Errors.Empty();

#if WITH_EDITOR
	if (LogListing != nullptr)
	{
		LogListing->ClearMessages();
	}
#endif
}

void FGLTFLogBuilder::PrintToLog(ELogLevel Level, const FString& Message) const
{
#if !NO_LOGGING
	ELogVerbosity::Type Verbosity;

	switch (Level)
	{
		case ELogLevel::Suggestion: Verbosity = ELogVerbosity::Display; break;
		case ELogLevel::Warning:    Verbosity = ELogVerbosity::Warning; break;
		case ELogLevel::Error:      Verbosity = ELogVerbosity::Error; break;
		default:
			checkNoEntry();
			return;
	}

	GWarn->Log(LogGLTFExporter.GetCategoryName(), Verbosity, Message);
#endif

#if WITH_EDITOR
	if (LogListing != nullptr)
	{
		EMessageSeverity::Type Severity;

		switch (Level)
		{
			case ELogLevel::Suggestion: Severity = EMessageSeverity::Info; break;
			case ELogLevel::Warning:    Severity = EMessageSeverity::Warning; break;
			case ELogLevel::Error:      Severity = EMessageSeverity::Error; break;
			default:
				checkNoEntry();
				return;
		}

		LogListing->AddMessage(FTokenizedMessage::Create(Severity, FText::FromString(Message)), false);
	}
#endif
}
