// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builders/GLTFTaskBuilder.h"
#include "Misc/FeedbackContext.h"
#include "Misc/ScopedSlowTask.h"

FGLTFTaskBuilder::FGLTFTaskBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions)
	: FGLTFLogBuilder(FileName, ExportOptions)
	, PriorityIndexLock(INDEX_NONE)
{
}

void FGLTFTaskBuilder::ProcessSlowTasks(FFeedbackContext* Context)
{
	const int32 PriorityCount = static_cast<int32>(EGLTFTaskPriority::MAX);
	for (int32 PriorityIndex = 0; PriorityIndex < PriorityCount; PriorityIndex++)
	{
		PriorityIndexLock = PriorityIndex;
		const EGLTFTaskPriority Priority = static_cast<EGLTFTaskPriority>(PriorityIndex);

		TArray<TUniquePtr<FGLTFDelayedTask>>* Tasks = TasksByPriority.Find(Priority);
		if (Tasks == nullptr)
		{
			continue;
		}

		if (Context != nullptr)
		{
			const FText MessageFormat = GetPriorityMessageFormat(Priority);
			const float AmountOfWork = static_cast<float>(Tasks->Num());
			FScopedSlowTask Progress(AmountOfWork, FText::Format(MessageFormat, FText()), true, *Context);
			Progress.MakeDialog();

			for (TUniquePtr<FGLTFDelayedTask>& Task : *Tasks)
			{
				const FText Name = FText::FromString(Task->GetName());
				const FText Message = FText::Format(MessageFormat, Name);
				Progress.EnterProgressFrame(1, Message);

				Task->Process();
			}
		}
		else
		{
			for (TUniquePtr<FGLTFDelayedTask>& Task : *Tasks)
			{
				Task->Process();
			}
		}

		//Merge over TasksByPrioritySecondary to TasksByPriority
		//This works here, because 'PriorityIndexLock' only allows addition to higher PriorityIndices.
		for (TPair<EGLTFTaskPriority, TArray<TUniquePtr<FGLTFDelayedTask>>>& Entry : TasksByPrioritySecondary)
		{
			TArray<TUniquePtr<FGLTFDelayedTask>>& TasksToCopyInto = TasksByPriority.FindOrAdd(Entry.Key);
			for (TUniquePtr<FGLTFDelayedTask>& EntryTask : Entry.Value)
			{
				TasksToCopyInto.Add(MoveTemp(EntryTask));
			}
		}
		TasksByPrioritySecondary.Empty();
	}

	TasksByPriority.Empty();
	PriorityIndexLock = INDEX_NONE;
}

bool FGLTFTaskBuilder::ScheduleSlowTask(TUniquePtr<FGLTFDelayedTask> Task)
{
	const EGLTFTaskPriority Priority = Task->Priority;
	if (static_cast<int32>(Priority) >= static_cast<int32>(EGLTFTaskPriority::MAX))
	{
		checkNoEntry();
		return false;
	}

	if (static_cast<int32>(Priority) <= PriorityIndexLock)
	{
		checkNoEntry();
		return false;
	}

	if (PriorityIndexLock == INDEX_NONE)
	{
		TasksByPriority.FindOrAdd(Priority).Add(MoveTemp(Task));
	}
	else
	{
		TasksByPrioritySecondary.FindOrAdd(Priority).Add(MoveTemp(Task));
	}
	
	return true;
}

FText FGLTFTaskBuilder::GetPriorityMessageFormat(EGLTFTaskPriority Priority)
{
	switch (Priority)
	{
		case EGLTFTaskPriority::Animation: return NSLOCTEXT("GLTFExporter", "AnimationTaskMessage", "Animation(s): {0}");
		case EGLTFTaskPriority::Mesh:      return NSLOCTEXT("GLTFExporter", "MeshTaskMessage", "Mesh(es): {0}");
		case EGLTFTaskPriority::Material:  return NSLOCTEXT("GLTFExporter", "MaterialTaskMessage", "Material(s): {0}");
		case EGLTFTaskPriority::Texture:   return NSLOCTEXT("GLTFExporter", "TextureTaskMessage", "Texture(s): {0}");
		default:
			checkNoEntry();
			return {};
	}
}
