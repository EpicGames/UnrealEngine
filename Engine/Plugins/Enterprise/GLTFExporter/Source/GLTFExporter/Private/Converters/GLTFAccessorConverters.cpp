// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFAccessorConverters.h"
#include "Utilities/GLTFCoreUtilities.h"
#include "Converters/GLTFBufferAdapter.h"
#include "Builders/GLTFConvertBuilder.h"
#include "Rendering/ColorVertexBuffer.h"
#include "Rendering/PositionVertexBuffer.h"
#include "Rendering/SkinWeightVertexBuffer.h"
#include "Rendering/StaticMeshVertexBuffer.h"

namespace
{
	template <typename DestinationType, typename SourceType>
	struct TGLTFVertexNormalUtilities
	{
		static DestinationType Convert(const SourceType& Normal)
		{
			return FGLTFCoreUtilities::ConvertNormal(Normal);
		}

		static DestinationType ConvertDelta(const SourceType& NormalDelta)
		{
			return FGLTFCoreUtilities::ConvertNormalDelta(NormalDelta);
		}
	};

	template <typename SourceType>
	struct TGLTFVertexNormalUtilities<FGLTFVector3, SourceType>
	{
		static FGLTFVector3 Convert(const SourceType& Normal)
		{
			return FGLTFCoreUtilities::ConvertNormal(Normal.ToFVector3f());
		}

		static FGLTFVector3 ConvertDelta(const SourceType& NormalDelta)
		{
            return FGLTFCoreUtilities::ConvertNormalDelta(NormalDelta.ToFVector3f());
		}
	};

	template <typename DestinationType, typename SourceType>
	struct TGLTFVertexTangentUtilities
	{
		static DestinationType Convert(const SourceType& TangentX, const SourceType& TangentZ)
		{
			return FGLTFCoreUtilities::ConvertTangent(TangentX, TangentZ);
		}
	};

	template <typename SourceType>
	struct TGLTFVertexTangentUtilities<FGLTFVector4, SourceType>
	{
		static FGLTFVector4 Convert(const SourceType& TangentX, const SourceType& TangentZ)
		{
			return FGLTFCoreUtilities::ConvertTangent(TangentX.ToFVector3f(), TangentZ.ToFVector4f());
		}
	};

	template <typename IndexType, typename IndexBufferType>
	FGLTFJsonAccessor* ConvertIndexBuffer(const TArray<IndexBufferType>& IndexBuffer, FGLTFConvertBuilder& Builder)
	{
		const uint32 IndexCount = IndexBuffer.Num();
		if (IndexCount == 0)
		{
			return nullptr;
		}

		TArray<IndexType> Indices;
		Indices.AddUninitialized(IndexCount);

		for (uint32 Index = 0; Index < IndexCount; ++Index)
		{
			Indices[Index] = static_cast<IndexType>(IndexBuffer[Index]);
		}

		FGLTFJsonAccessor* JsonAccessor = Builder.AddAccessor();
		JsonAccessor->BufferView = Builder.AddBufferView(Indices, EGLTFJsonBufferTarget::ElementArrayBuffer, sizeof(IndexType));
		JsonAccessor->ComponentType = FGLTFCoreUtilities::GetComponentType<IndexType>();
		JsonAccessor->Count = IndexCount;
		JsonAccessor->Type = EGLTFJsonAccessorType::Scalar;

		return JsonAccessor;
	}

	template <typename ComponentType, EGLTFJsonAccessorType AccessorType, class ContainerClass>
	FGLTFJsonAccessor* ConvertToAccessor(FGLTFConvertBuilder& Builder, ContainerClass& Container, bool bNormalized = false)
	{
		if (Container.Num() == 0)
		{
			return nullptr;
		}

		FGLTFJsonAccessor* JsonAccessor = Builder.AddAccessor();
		JsonAccessor->BufferView = Builder.AddBufferView(Container, EGLTFJsonBufferTarget::ArrayBuffer);
		JsonAccessor->ComponentType = FGLTFCoreUtilities::GetComponentType<ComponentType>();
		JsonAccessor->Count = Container.Num();
		JsonAccessor->Type = AccessorType;
		JsonAccessor->bNormalized = bNormalized;

		return JsonAccessor;
	}
}

FGLTFJsonAccessor* FGLTFPositionBufferConverter::Convert(const FGLTFMeshSection* MeshSection, const FPositionVertexBuffer* VertexBuffer)
{
	if (VertexBuffer == nullptr || VertexBuffer->GetNumVertices() == 0)
	{
		return nullptr;
	}

	const TUniquePtr<IGLTFBufferAdapter> SourceBuffer = IGLTFBufferAdapter::GetPositions(VertexBuffer);
	const uint8* SourceData = SourceBuffer->GetData();

	if (SourceData == nullptr)
	{
		// TODO: report error
		return nullptr;
	}

	const TArray<uint32>& IndexMap = MeshSection->IndexMap;
	const uint32 VertexCount = IndexMap.Num() > 0 ? IndexMap.Num() : VertexBuffer->GetNumVertices();
	const uint32 Stride = VertexBuffer->GetStride();

	TArray<FGLTFVector3> Positions;
	Positions.AddUninitialized(VertexCount);

    if (IndexMap.Num() > 0)
    {
        for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
        {
            const uint32 MappedVertexIndex = IndexMap[VertexIndex];
            const FVector3f& Position = *reinterpret_cast<const FVector3f*>(SourceData + Stride * MappedVertexIndex);
            Positions[VertexIndex] = FGLTFCoreUtilities::ConvertPosition(Position, Builder.ExportOptions->ExportUniformScale);
        }
    }
    else
    {
        for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
        {
            const FVector3f& Position = *reinterpret_cast<const FVector3f*>(SourceData + Stride * VertexIndex);
            Positions[VertexIndex] = FGLTFCoreUtilities::ConvertPosition(Position, Builder.ExportOptions->ExportUniformScale);
        }
    }
	
	FGLTFJsonAccessor* JsonAccessor = Builder.AddAccessor();
	JsonAccessor->BufferView = Builder.AddBufferView(Positions, EGLTFJsonBufferTarget::ArrayBuffer);
	JsonAccessor->ComponentType = EGLTFJsonComponentType::Float;
	JsonAccessor->Count = VertexCount;
	JsonAccessor->Type = EGLTFJsonAccessorType::Vec3;

	if (VertexCount > 0)
	{
		JsonAccessor->MinMaxLength = 3;

		for (int32 ComponentIndex = 0; ComponentIndex < JsonAccessor->MinMaxLength; ComponentIndex++)
		{
			JsonAccessor->Min[ComponentIndex] = Positions[0].Components[ComponentIndex];
			JsonAccessor->Max[ComponentIndex] = Positions[0].Components[ComponentIndex];
		}

		for (uint32 VertexIndex = 1; VertexIndex < VertexCount; ++VertexIndex)
		{
			const FGLTFVector3& Position = Positions[VertexIndex];
			for (int32 ComponentIndex = 0; ComponentIndex < JsonAccessor->MinMaxLength; ComponentIndex++)
			{
				JsonAccessor->Min[ComponentIndex] = FMath::Min(JsonAccessor->Min[ComponentIndex], Position.Components[ComponentIndex]);
				JsonAccessor->Max[ComponentIndex] = FMath::Max(JsonAccessor->Max[ComponentIndex], Position.Components[ComponentIndex]);
			}
		}
	}

	return JsonAccessor;
}

FGLTFJsonAccessor* FGLTFPositionDeltaBufferConverter::Convert(const FGLTFMeshSection* MeshSection, const TMap<uint32, FVector3f>* TargetPositionDeltas)
{
    if (!TargetPositionDeltas)
    {
        return nullptr;
    }

    const TArray<uint32>& IndexMap = MeshSection->IndexMap;
    const uint32 VertexCount = IndexMap.Num() > 0 ? IndexMap.Num() : TargetPositionDeltas->Num();

    TArray<FGLTFVector3> Positions;
    Positions.AddUninitialized(VertexCount);

    if (IndexMap.Num() > 0)
    {
        for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
        {
            const uint32 MappedVertexIndex = IndexMap[VertexIndex];
            const FVector3f* Delta = TargetPositionDeltas->Find(MappedVertexIndex);
            FVector3f Position = Delta ? *Delta : FVector3f();

            Positions[VertexIndex] = FGLTFCoreUtilities::ConvertPosition(Position, Builder.ExportOptions->ExportUniformScale);
        }
    }
    else
    {
        for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
        {
            const FVector3f* Delta = TargetPositionDeltas->Find(VertexIndex);
            FVector3f Position = Delta ? *Delta : FVector3f();

            Positions[VertexIndex] = FGLTFCoreUtilities::ConvertPosition(Position, Builder.ExportOptions->ExportUniformScale);
        }
    }

    FGLTFJsonAccessor* JsonAccessor = Builder.AddAccessor();
    JsonAccessor->BufferView = Builder.AddBufferView(Positions, EGLTFJsonBufferTarget::ArrayBuffer);
    JsonAccessor->ComponentType = EGLTFJsonComponentType::Float;
    JsonAccessor->Count = VertexCount;
    JsonAccessor->Type = EGLTFJsonAccessorType::Vec3;

    if (VertexCount > 0)
    {
        // Calculate accurate bounding box based on raw vertex values
        JsonAccessor->MinMaxLength = 3;

        for (int32 ComponentIndex = 0; ComponentIndex < JsonAccessor->MinMaxLength; ComponentIndex++)
        {
            JsonAccessor->Min[ComponentIndex] = Positions[0].Components[ComponentIndex];
            JsonAccessor->Max[ComponentIndex] = Positions[0].Components[ComponentIndex];
        }

        for (uint32 VertexIndex = 1; VertexIndex < VertexCount; ++VertexIndex)
        {
            const FGLTFVector3& Position = Positions[VertexIndex];
            for (int32 ComponentIndex = 0; ComponentIndex < JsonAccessor->MinMaxLength; ComponentIndex++)
            {
                JsonAccessor->Min[ComponentIndex] = FMath::Min(JsonAccessor->Min[ComponentIndex], Position.Components[ComponentIndex]);
                JsonAccessor->Max[ComponentIndex] = FMath::Max(JsonAccessor->Max[ComponentIndex], Position.Components[ComponentIndex]);
            }
        }
    }

    return JsonAccessor;
}

FGLTFJsonAccessor* FGLTFColorBufferConverter::Convert(const FGLTFMeshSection* MeshSection, const FColorVertexBuffer* VertexColorBufferBuffer)
{
	if (VertexColorBufferBuffer == nullptr || VertexColorBufferBuffer->GetNumVertices() == 0)
	{
		return nullptr;
	}

	const TUniquePtr<IGLTFBufferAdapter> SourceBuffer = IGLTFBufferAdapter::GetColors(VertexColorBufferBuffer);
	const uint8* SourceData = SourceBuffer->GetData();

	if (SourceData == nullptr)
	{
		// TODO: report error
		return nullptr;
	}

	const TArray<uint32>& IndexMap = MeshSection->IndexMap;
	const uint32 VertexCount = IndexMap.Num() > 0 ? IndexMap.Num() : VertexColorBufferBuffer->GetNumVertices();
	const uint32 Stride = VertexColorBufferBuffer->GetStride();

	TArray<FGLTFUInt8Color4> Colors;
	Colors.AddUninitialized(VertexCount);

	if (IndexMap.Num() > 0)
	{
		for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
		{
			const uint32 MappedVertexIndex = IndexMap[VertexIndex];
			const FColor& Color = *reinterpret_cast<const FColor*>(SourceData + Stride * MappedVertexIndex);
			Colors[VertexIndex] = FGLTFCoreUtilities::ConvertColor(Color);
		}
	}
	else
	{
		for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
		{
			const FColor& Color = *reinterpret_cast<const FColor*>(SourceData + Stride * VertexIndex);
			Colors[VertexIndex] = FGLTFCoreUtilities::ConvertColor(Color);
		}
	}



	FGLTFJsonAccessor* JsonAccessor = Builder.AddAccessor();
	JsonAccessor->BufferView = Builder.AddBufferView(Colors, EGLTFJsonBufferTarget::ArrayBuffer);
	JsonAccessor->ComponentType = EGLTFJsonComponentType::UInt8;
	JsonAccessor->Count = VertexCount;
	JsonAccessor->Type = EGLTFJsonAccessorType::Vec4;
	JsonAccessor->bNormalized = true;

	return JsonAccessor;
}

FGLTFJsonAccessor* FGLTFNormalBufferConverter::Convert(const FGLTFMeshSection* MeshSection, const FStaticMeshVertexBuffer* VertexBuffer)
{
	if (VertexBuffer == nullptr || VertexBuffer->GetNumVertices() == 0)
	{
		return nullptr;
	}

	const TUniquePtr<IGLTFBufferAdapter> SourceBuffer = IGLTFBufferAdapter::GetTangents(VertexBuffer);
	const uint8* SourceData = SourceBuffer->GetData();

	if (SourceData == nullptr)
	{
		// TODO: report error
		return nullptr;
	}

	FGLTFJsonBufferView* BufferView;
	EGLTFJsonComponentType ComponentType;

	const bool bMeshQuantization = Builder.ExportOptions->bUseMeshQuantization;
	const bool bHighPrecision = VertexBuffer->GetUseHighPrecisionTangentBasis();

	if (bMeshQuantization)
	{
		Builder.AddExtension(EGLTFJsonExtension::KHR_MeshQuantization, true);

		if (bHighPrecision)
		{
			ComponentType = EGLTFJsonComponentType::Int16;
			BufferView = ConvertBufferView<FGLTFInt16Vector4, FPackedRGBA16N>(MeshSection, SourceData);
			BufferView->ByteStride = sizeof(FGLTFInt16Vector4);
		}
		else
		{
			ComponentType = EGLTFJsonComponentType::Int8;
			BufferView = ConvertBufferView<FGLTFInt8Vector4, FPackedNormal>(MeshSection, SourceData);
			BufferView->ByteStride = sizeof(FGLTFInt8Vector4);
		}
	}
	else
	{
		ComponentType = EGLTFJsonComponentType::Float;
		BufferView = bHighPrecision
			? ConvertBufferView<FGLTFVector3, FPackedRGBA16N>(MeshSection, SourceData)
			: ConvertBufferView<FGLTFVector3, FPackedNormal>(MeshSection, SourceData);
	}

	FGLTFJsonAccessor* JsonAccessor = Builder.AddAccessor();
	JsonAccessor->BufferView = BufferView;
	JsonAccessor->ComponentType = ComponentType;
	JsonAccessor->Count = MeshSection->IndexMap.Num();
	JsonAccessor->Type = EGLTFJsonAccessorType::Vec3;
	JsonAccessor->bNormalized = bMeshQuantization;

	return JsonAccessor;
}

template <typename DestinationType, typename SourceType>
FGLTFJsonBufferView* FGLTFNormalBufferConverter::ConvertBufferView(const FGLTFMeshSection* MeshSection, const void* SourceData)
{
	const TArray<uint32>& IndexMap = MeshSection->IndexMap;
	const uint32 VertexCount = IndexMap.Num();

	typedef TStaticMeshVertexTangentDatum<SourceType> TangentDatumType;
	const TangentDatumType* TangentData = static_cast<const TangentDatumType*>(SourceData);

	TArray<DestinationType> Normals;
    Normals.SetNumZeroed(VertexCount);

    for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
    {
        const uint32 MappedVertexIndex = IndexMap[VertexIndex];
        const TangentDatumType& TangentDatum = TangentData[MappedVertexIndex];
        Normals[VertexIndex] = TGLTFVertexNormalUtilities<DestinationType, SourceType>::Convert(TangentDatum.TangentZ);
    }

	return Builder.AddBufferView(Normals, EGLTFJsonBufferTarget::ArrayBuffer);
}


FGLTFJsonAccessor* FGLTFNormalDeltaBufferConverter::Convert(const FGLTFMeshSection* MeshSection, const TMap<uint32, FVector3f>* TargetNormalDeltas, bool bHighPrecision)
{
    FGLTFJsonBufferView* BufferView;
    EGLTFJsonComponentType ComponentType;

    const bool bMeshQuantization = Builder.ExportOptions->bUseMeshQuantization;
    if (bMeshQuantization)
    {
        Builder.AddExtension(EGLTFJsonExtension::KHR_MeshQuantization, true);

        if (bHighPrecision)
        {
            ComponentType = EGLTFJsonComponentType::Int16;
            BufferView = ConvertBufferView<FGLTFInt16Vector4, FPackedRGBA16N>(MeshSection, TargetNormalDeltas);
            BufferView->ByteStride = sizeof(FGLTFInt16Vector4);
        }
        else
        {
            ComponentType = EGLTFJsonComponentType::Int8;
            BufferView = ConvertBufferView<FGLTFInt8Vector4, FPackedNormal>(MeshSection, TargetNormalDeltas);
            BufferView->ByteStride = sizeof(FGLTFInt8Vector4);
        }
    }
    else
    {
        ComponentType = EGLTFJsonComponentType::Float;
        BufferView = bHighPrecision
            ? ConvertBufferView<FGLTFVector3, FPackedRGBA16N>(MeshSection, TargetNormalDeltas)
            : ConvertBufferView<FGLTFVector3, FPackedNormal>(MeshSection, TargetNormalDeltas);
    }

    FGLTFJsonAccessor* JsonAccessor = Builder.AddAccessor();
    JsonAccessor->BufferView = BufferView;
    JsonAccessor->ComponentType = ComponentType;
    JsonAccessor->Count = MeshSection->IndexMap.Num();
    JsonAccessor->Type = EGLTFJsonAccessorType::Vec3;
    JsonAccessor->bNormalized = bMeshQuantization;

    return JsonAccessor;
}

template <typename DestinationType, typename SourceType>
FGLTFJsonBufferView* FGLTFNormalDeltaBufferConverter::ConvertBufferView(const FGLTFMeshSection* MeshSection, const TMap<uint32, FVector3f>* TargetNormalDeltas)
{
    const TArray<uint32>& IndexMap = MeshSection->IndexMap;
    const uint32 VertexCount = IndexMap.Num();

    TArray<DestinationType> Normals;
    Normals.SetNumZeroed(VertexCount);

    if (TargetNormalDeltas && TargetNormalDeltas->Num())
    {
        for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
        {
            const uint32 MappedVertexIndex = IndexMap[VertexIndex];

            const FVector3f* TargetNormalDelta = TargetNormalDeltas->Find(MappedVertexIndex);
            if (TargetNormalDelta)
            {
                Normals[VertexIndex] = TGLTFVertexNormalUtilities<DestinationType, SourceType>::ConvertDelta(*TargetNormalDelta);
            }
        }
    }

    return Builder.AddBufferView(Normals, EGLTFJsonBufferTarget::ArrayBuffer);
}

FGLTFJsonAccessor* FGLTFTangentBufferConverter::Convert(const FGLTFMeshSection* MeshSection, const FStaticMeshVertexBuffer* VertexBuffer)
{
	if (VertexBuffer == nullptr || VertexBuffer->GetNumVertices() == 0)
	{
		return nullptr;
	}

	const TUniquePtr<IGLTFBufferAdapter> SourceBuffer = IGLTFBufferAdapter::GetTangents(VertexBuffer);
	const uint8* SourceData = SourceBuffer->GetData();

	if (SourceData == nullptr)
	{
		// TODO: report error
		return nullptr;
	}

	FGLTFJsonBufferView* BufferView;
	EGLTFJsonComponentType ComponentType;

	const bool bMeshQuantization = Builder.ExportOptions->bUseMeshQuantization;
	const bool bHighPrecision = VertexBuffer->GetUseHighPrecisionTangentBasis();

	if (bMeshQuantization)
	{
		Builder.AddExtension(EGLTFJsonExtension::KHR_MeshQuantization, true);

		if (bHighPrecision)
		{
			ComponentType = EGLTFJsonComponentType::Int16;
			BufferView = ConvertBufferView<FGLTFInt16Vector4, FPackedRGBA16N>(MeshSection, SourceData);
		}
		else
		{
			ComponentType = EGLTFJsonComponentType::Int8;
			BufferView = ConvertBufferView<FGLTFInt8Vector4, FPackedNormal>(MeshSection, SourceData);
		}
	}
	else
	{
		ComponentType = EGLTFJsonComponentType::Float;
		BufferView = bHighPrecision
			? ConvertBufferView<FGLTFVector4, FPackedRGBA16N>(MeshSection, SourceData)
			: ConvertBufferView<FGLTFVector4, FPackedNormal>(MeshSection, SourceData);
	}

	FGLTFJsonAccessor* JsonAccessor = Builder.AddAccessor();
	JsonAccessor->BufferView = BufferView;
	JsonAccessor->ComponentType = ComponentType;
	JsonAccessor->Count = MeshSection->IndexMap.Num();
	JsonAccessor->Type = EGLTFJsonAccessorType::Vec4;
	JsonAccessor->bNormalized = bMeshQuantization;

	return JsonAccessor;
}

template <typename DestinationType, typename SourceType>
FGLTFJsonBufferView* FGLTFTangentBufferConverter::ConvertBufferView(const FGLTFMeshSection* MeshSection, const void* SourceData)
{
	const TArray<uint32>& IndexMap = MeshSection->IndexMap;
	const uint32 VertexCount = IndexMap.Num();

	typedef TStaticMeshVertexTangentDatum<SourceType> TangentDatumType;
	const TangentDatumType* VertexData = static_cast<const TangentDatumType*>(SourceData);

	TArray<DestinationType> Tangents;
	Tangents.AddUninitialized(VertexCount);

	for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
	{
		const uint32 MappedVertexIndex = IndexMap[VertexIndex];
		const TangentDatumType& TangentDatum = VertexData[MappedVertexIndex];
		Tangents[VertexIndex] = TGLTFVertexTangentUtilities<DestinationType, SourceType>::Convert(TangentDatum.TangentX, TangentDatum.TangentZ);
	}

	return Builder.AddBufferView(Tangents, EGLTFJsonBufferTarget::ArrayBuffer);
}

FGLTFJsonAccessor* FGLTFUVBufferConverter::Convert(const FGLTFMeshSection* MeshSection, const FStaticMeshVertexBuffer* VertexBuffer, uint32 UVIndex)
{
	if (VertexBuffer == nullptr || VertexBuffer->GetNumVertices() == 0)
	{
		return nullptr;
	}

	const uint32 UVCount = VertexBuffer->GetNumTexCoords();
	if (UVIndex >= UVCount)
	{
		// TODO: report warning
		return nullptr;
	}

	// TODO: report warning or add support for half float precision UVs, i.e. !VertexBuffer->GetUseFullPrecisionUVs()?

	const TUniquePtr<IGLTFBufferAdapter> SourceBuffer = IGLTFBufferAdapter::GetUVs(VertexBuffer);
	const uint8* SourceData = SourceBuffer->GetData();

	if (SourceData == nullptr)
	{
		// TODO: report error
		return nullptr;
	}

	return VertexBuffer->GetUseFullPrecisionUVs()
		? Convert<FVector2f>(MeshSection, VertexBuffer, UVIndex, SourceData)
		: Convert<FVector2DHalf>(MeshSection, VertexBuffer, UVIndex, SourceData);
}

template <typename SourceType>
FGLTFJsonAccessor* FGLTFUVBufferConverter::Convert(const FGLTFMeshSection* MeshSection, const FStaticMeshVertexBuffer* VertexBuffer, uint32 UVIndex, const uint8* SourceData) const
{
	const TArray<uint32>& IndexMap = MeshSection->IndexMap;
	const uint32 VertexCount = IndexMap.Num();

	TArray<FGLTFVector2> UVs;
	UVs.AddUninitialized(VertexCount);

	const uint32 UVCount = VertexBuffer->GetNumTexCoords();
	const SourceType* UVData = reinterpret_cast<const SourceType*>(SourceData);

	for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
	{
		const uint32 MappedVertexIndex = IndexMap[VertexIndex];
		const SourceType& UV = UVData[UVCount * MappedVertexIndex + UVIndex];
		UVs[VertexIndex] = FGLTFCoreUtilities::ConvertUV(UV);
	}

	return ConvertToAccessor<float, EGLTFJsonAccessorType::Vec2>(Builder, UVs);
}

FGLTFJsonAccessor* FGLTFBoneIndexBufferConverter::Convert(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, uint32 InfluenceOffset)
{
	if (VertexBuffer == nullptr || VertexBuffer->GetNumVertices() == 0)
	{
		return nullptr;
	}

	const TUniquePtr<IGLTFBufferAdapter> SourceBuffer = IGLTFBufferAdapter::GetInfluences(VertexBuffer);
	const uint8* SourceData = SourceBuffer->GetData();

	if (SourceData == nullptr)
	{
		// TODO: report error
		return nullptr;
	}

	return MeshSection->MaxBoneIndex > UINT8_MAX
		? Convert<uint16>(MeshSection, VertexBuffer, InfluenceOffset, SourceData)
		: Convert<uint8>(MeshSection, VertexBuffer, InfluenceOffset, SourceData);
}

template <typename DestinationType>
FGLTFJsonAccessor* FGLTFBoneIndexBufferConverter::Convert(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, uint32 InfluenceOffset, const uint8* SourceData) const
{
	return VertexBuffer->Use16BitBoneIndex()
		? Convert<DestinationType, uint16>(MeshSection, VertexBuffer, InfluenceOffset, SourceData)
		: Convert<DestinationType, uint8>(MeshSection, VertexBuffer, InfluenceOffset, SourceData);
}

template <typename DestinationType, typename SourceType>
FGLTFJsonAccessor* FGLTFBoneIndexBufferConverter::Convert(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, uint32 InfluenceOffset, const uint8* SourceData) const
{
	if (VertexBuffer->GetVariableBonesPerVertex())
	{
		const TUniquePtr<IGLTFBufferAdapter> LookupBuffer = IGLTFBufferAdapter::GetLookups(VertexBuffer);
		const uint32* LookupData = reinterpret_cast<const uint32*>(LookupBuffer->GetData());

		if (LookupData == nullptr)
		{
			// TODO: report error
			return nullptr;
		}

		return Convert<DestinationType, SourceType>(MeshSection, VertexBuffer, InfluenceOffset, SourceData, [LookupData](uint32 VertexIndex, uint32& VertexDataOffset, uint32& VertexInfluenceCount)
		{
			const uint32 Value = LookupData[VertexIndex];
			VertexDataOffset = Value >> 8;
			VertexInfluenceCount = Value & 0xff;
		});
	}

	const uint32 MaxBoneInfluences = VertexBuffer->GetMaxBoneInfluences();
	return Convert<DestinationType, SourceType>(MeshSection, VertexBuffer, InfluenceOffset, SourceData, [MaxBoneInfluences](uint32 VertexIndex, uint32& VertexDataOffset, uint32& VertexInfluenceCount)
	{
		VertexDataOffset = (sizeof(SourceType) + sizeof(uint8)) * MaxBoneInfluences * VertexIndex;
		VertexInfluenceCount = MaxBoneInfluences;
	});
}

template <typename DestinationType, typename SourceType, typename CallbackType>
FGLTFJsonAccessor* FGLTFBoneIndexBufferConverter::Convert(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, uint32 InfluenceOffset, const uint8* SourceData, CallbackType GetVertexInfluenceOffsetCount) const
{
	const TArray<uint32>& IndexMap = MeshSection->IndexMap;
	const uint32 VertexCount = IndexMap.Num();

	struct FVertexBoneIndices
	{
		DestinationType Index[4];
	};

	TArray<FVertexBoneIndices> BoneIndices;
	BoneIndices.AddUninitialized(VertexCount);

	for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
	{
		const TArray<FBoneIndexType>& BoneMap = MeshSection->BoneMaps[MeshSection->BoneMapLookup[VertexIndex]];
		const uint32 MappedVertexIndex = IndexMap[VertexIndex];
		FVertexBoneIndices& VertexBoneIndices = BoneIndices[VertexIndex];

		uint32 VertexDataOffset;
		uint32 VertexInfluenceCount;
		GetVertexInfluenceOffsetCount(MappedVertexIndex, VertexDataOffset, VertexInfluenceCount);

		const SourceType* VertexBoneIndexData = reinterpret_cast<const SourceType*>(SourceData + VertexDataOffset);
		const int32 InfluenceCount = FMath::Min(static_cast<int32>(VertexInfluenceCount - InfluenceOffset), 4);

		for (int32 InfluenceIndex = 0; InfluenceIndex < InfluenceCount; ++InfluenceIndex)
		{
			const uint32 UnmappedBoneIndex = VertexBoneIndexData[InfluenceOffset + InfluenceIndex];
			VertexBoneIndices.Index[InfluenceIndex] = static_cast<DestinationType>(BoneMap[UnmappedBoneIndex]);
		}

		for (int32 InfluenceIndex = InfluenceCount; InfluenceIndex < 4; ++InfluenceIndex)
		{
			VertexBoneIndices.Index[InfluenceIndex] = 0;
		}
	}

	return ConvertToAccessor<DestinationType, EGLTFJsonAccessorType::Vec4>(Builder, BoneIndices);
}

FGLTFJsonAccessor* FGLTFBoneWeightBufferConverter::Convert(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, uint32 InfluenceOffset)
{
	if (VertexBuffer == nullptr || VertexBuffer->GetNumVertices() == 0)
	{
		return nullptr;
	}

	const TUniquePtr<IGLTFBufferAdapter> SourceBuffer = IGLTFBufferAdapter::GetInfluences(VertexBuffer);
	const uint8* SourceData = SourceBuffer->GetData();

	if (SourceData == nullptr)
	{
		// TODO: report error
		return nullptr;
	}

	return MeshSection->MaxBoneIndex > UINT8_MAX
		? Convert<uint16>(MeshSection, VertexBuffer, InfluenceOffset, SourceData)
		: Convert<uint8>(MeshSection, VertexBuffer, InfluenceOffset, SourceData);
}

template <typename BoneIndexType>
FGLTFJsonAccessor* FGLTFBoneWeightBufferConverter::Convert(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, uint32 InfluenceOffset, const uint8* SourceData) const
{
	if (VertexBuffer->GetVariableBonesPerVertex())
	{
		const TUniquePtr<IGLTFBufferAdapter> LookupBuffer = IGLTFBufferAdapter::GetLookups(VertexBuffer);
		const uint32* LookupData = reinterpret_cast<const uint32*>(LookupBuffer->GetData());

		if (LookupData == nullptr)
		{
			// TODO: report error
			return nullptr;
		}

		return Convert<BoneIndexType>(MeshSection, VertexBuffer, InfluenceOffset, SourceData, [LookupData](uint32 VertexIndex, uint32& VertexDataOffset, uint32& VertexInfluenceCount)
		{
			const uint32 Value = LookupData[VertexIndex];
			VertexDataOffset = Value >> 8;
			VertexInfluenceCount = Value & 0xff;
		});
	}

	const uint32 MaxBoneInfluences = VertexBuffer->GetMaxBoneInfluences();
	return Convert<BoneIndexType>(MeshSection, VertexBuffer, InfluenceOffset, SourceData, [MaxBoneInfluences](uint32 VertexIndex, uint32& VertexDataOffset, uint32& VertexInfluenceCount)
	{
		VertexDataOffset = (sizeof(BoneIndexType) + sizeof(uint8)) * MaxBoneInfluences * VertexIndex;
		VertexInfluenceCount = MaxBoneInfluences;
	});
}

template <typename BoneIndexType, typename CallbackType>
FGLTFJsonAccessor* FGLTFBoneWeightBufferConverter::Convert(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, uint32 InfluenceOffset, const uint8* SourceData, CallbackType GetVertexInfluenceOffsetCount) const
{
	const TArray<uint32>& IndexMap = MeshSection->IndexMap;
	const uint32 VertexCount = IndexMap.Num();

	struct FVertexBoneWeights
	{
		uint8 Weights[4];
	};

	TArray<FVertexBoneWeights> BoneWeights;
	BoneWeights.AddUninitialized(VertexCount);

	for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
	{
		const uint32 MappedVertexIndex = IndexMap[VertexIndex];
		FVertexBoneWeights& VertexBoneWeights = BoneWeights[VertexIndex];

		uint32 VertexDataOffset;
		uint32 VertexInfluenceCount;
		GetVertexInfluenceOffsetCount(MappedVertexIndex, VertexDataOffset, VertexInfluenceCount);

		const uint8* VertexBoneWeightsData = SourceData + VertexDataOffset + sizeof(BoneIndexType) * VertexInfluenceCount;
		const int32 InfluenceCount = FMath::Min(static_cast<int32>(VertexInfluenceCount - InfluenceOffset), 4);

		for (int32 InfluenceIndex = 0; InfluenceIndex < InfluenceCount; ++InfluenceIndex)
		{
			VertexBoneWeights.Weights[InfluenceIndex] = VertexBoneWeightsData[InfluenceOffset + InfluenceIndex];
		}

		for (int32 InfluenceIndex = InfluenceCount; InfluenceIndex < 4; ++InfluenceIndex)
		{
			VertexBoneWeights.Weights[InfluenceIndex] = 0;
		}
	}

	return ConvertToAccessor<uint8, EGLTFJsonAccessorType::Vec4>(Builder, BoneWeights, true);
}

FGLTFJsonAccessor* FGLTFIndexBufferConverter::Convert(const FGLTFMeshSection* MeshSection)
{
	const TArray<uint32>& IndexBuffer = MeshSection->IndexBuffer;
	const uint32 MaxVertexIndex = MeshSection->IndexMap.Num() - 1;
	// NOTE: Even if a maximum value (i.e. UINT8_MAX or UINT16_MAX) would fit inside the type (i.e. uint8 or uint16), it's not allowed per gltf spec
	// since the maximum values trigger primitive restart in some graphics APIs and would require client implementations to rebuild the index buffer.
	if (MaxVertexIndex < UINT8_MAX) return ConvertIndexBuffer<uint8>(IndexBuffer, Builder);
	if (MaxVertexIndex < UINT16_MAX) return ConvertIndexBuffer<uint16>(IndexBuffer, Builder);
	if (MaxVertexIndex == UINT32_MAX)
	{
		Builder.LogWarning(
			FString::Printf(TEXT("Maximum value (%d) used in indices accessor for mesh %s, this may not be supported on some glTF viewers"),
				UINT32_MAX,
				*MeshSection->ToString()
			));
	}
	return ConvertIndexBuffer<uint32>(IndexBuffer, Builder);
}


FGLTFJsonAccessor* FGLTFPositionBufferConverterRaw::Convert(FGLTFPositionArray VertexBuffer)
{
	const uint32 VertexCount = VertexBuffer.Num();

	TArray<FGLTFVector3> Positions;
	FVector3f Min, Max;

	Positions.AddUninitialized(VertexCount);

	const size_t MinMaxLength = 3;
	if (VertexCount > 0)
	{
		Positions[0] = FGLTFCoreUtilities::ConvertPosition(VertexBuffer[0], Builder.ExportOptions->ExportUniformScale);

		for (size_t ComponentIndex = 0; ComponentIndex < MinMaxLength; ComponentIndex++)
		{
			Min[ComponentIndex] = Positions[0].Components[ComponentIndex];
			Max[ComponentIndex] = Positions[0].Components[ComponentIndex];
		}

		for (uint32 PositionIndex = 1; PositionIndex < VertexCount; ++PositionIndex)
		{
			Positions[PositionIndex] = FGLTFCoreUtilities::ConvertPosition(VertexBuffer[PositionIndex], Builder.ExportOptions->ExportUniformScale);

			for (size_t ComponentIndex = 0; ComponentIndex < MinMaxLength; ComponentIndex++)
			{
				Min[ComponentIndex] = FMath::Min(Min[ComponentIndex], Positions[PositionIndex].Components[ComponentIndex]);
				Max[ComponentIndex] = FMath::Max(Max[ComponentIndex], Positions[PositionIndex].Components[ComponentIndex]);
			}
		}
	}

	FGLTFJsonAccessor* JsonAccessor = ConvertToAccessor<float, EGLTFJsonAccessorType::Vec3>(Builder, Positions);

	JsonAccessor->MinMaxLength = MinMaxLength;
	for (int32 ComponentIndex = 0; ComponentIndex < JsonAccessor->MinMaxLength; ComponentIndex++)
	{
		JsonAccessor->Min[ComponentIndex] = Min[ComponentIndex];
		JsonAccessor->Max[ComponentIndex] = Max[ComponentIndex];
	}

	return JsonAccessor;
}

FGLTFJsonAccessor* FGLTFIndexBufferConverterRaw::Convert(FGLTFIndexArray IndexBuffer, FString MeshName)
{
	const uint32 MaxVertexIndex = IndexBuffer.Num() - 1;
	// NOTE: Even if a maximum value (i.e. UINT8_MAX or UINT16_MAX) would fit inside the type (i.e. uint8 or uint16), it's not allowed per gltf spec
	// since the maximum values trigger primitive restart in some graphics APIs and would require client implementations to rebuild the index buffer.
	if (MaxVertexIndex < UINT8_MAX) return ConvertIndexBuffer<uint8>(IndexBuffer, Builder);
	if (MaxVertexIndex < UINT16_MAX) return ConvertIndexBuffer<uint16>(IndexBuffer, Builder);
	if (MaxVertexIndex == UINT32_MAX)
	{
		Builder.LogWarning(
			FString::Printf(TEXT("Maximum value (%d) used in indices accessor for mesh %s, this may not be supported on some glTF viewers"),
				UINT32_MAX,
				*MeshName
			));
	}
	return ConvertIndexBuffer<uint32>(IndexBuffer, Builder);
}


FGLTFJsonAccessor* FGLTFNormalBufferConverterRaw::Convert(FGLTFNormalArray NormalsSource, bool bNormalize)
{
	const uint32 VertexCount = NormalsSource.Num();

	TArray<FGLTFVector3> Normals;
	Normals.AddUninitialized(VertexCount);

    if (bNormalize)
    {
        for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
        {
            Normals[VertexIndex] = FGLTFCoreUtilities::ConvertNormal(NormalsSource[VertexIndex]);
        }
    }
    else
    {
        for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
        {
            Normals[VertexIndex] = FGLTFCoreUtilities::ConvertVector(NormalsSource[VertexIndex]);
        }
    }
	

	return ConvertToAccessor<float, EGLTFJsonAccessorType::Vec3>(Builder, Normals);
}

FGLTFJsonAccessor* FGLTFUVBufferConverterRaw::Convert(FGLTFUVArray UVSource)
{
	const uint32 VertexCount = UVSource.Num();

	TArray<FGLTFVector2> UVs;
	UVs.AddUninitialized(VertexCount);

	for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
	{
		UVs[VertexIndex] = FGLTFCoreUtilities::ConvertUV(UVSource[VertexIndex]);
	}

	return ConvertToAccessor<float, EGLTFJsonAccessorType::Vec2>(Builder, UVs);
}

FGLTFJsonAccessor* FGLTFColorBufferConverterRaw::Convert(FGLTFColorArray VertexColorBuffer)
{
	const uint32 VertexCount = VertexColorBuffer.Num();

	TArray<FGLTFUInt8Color4> Colors;
	Colors.AddUninitialized(VertexCount);

	for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
	{
		Colors[VertexIndex] = FGLTFCoreUtilities::ConvertColor(VertexColorBuffer[VertexIndex]);
	}

	return ConvertToAccessor<uint8, EGLTFJsonAccessorType::Vec4>(Builder, Colors, true);
}

FGLTFJsonAccessor* FGLTFTangentBufferConverterRaw::Convert(FGLTFTangentArray TangentsSource)
{
	const uint32 VertexCount = TangentsSource.Num();

	TArray<FGLTFVector4> Tangents;
	Tangents.AddUninitialized(VertexCount);

	for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
	{
		Tangents[VertexIndex] = FGLTFCoreUtilities::ConvertTangent(TangentsSource[VertexIndex]);
	}
	
	return ConvertToAccessor<float, EGLTFJsonAccessorType::Vec4>(Builder, Tangents);
}

FGLTFJsonAccessor* FGLTFBoneIndexBufferConverterRaw::Convert(FGLTFJointInfluenceArray BoneIndices)
{
	return ConvertToAccessor<FBoneIndexType, EGLTFJsonAccessorType::Vec4>(Builder, BoneIndices);
}

FGLTFJsonAccessor* FGLTFBoneWeightBufferConverterRaw::Convert(FGLTFJointWeightArray Weights)
{
	return ConvertToAccessor<uint16, EGLTFJsonAccessorType::Vec4>(Builder, Weights, true);
}