// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFAnimationConverters.h"
#include "Builders/GLTFConvertBuilder.h"
#include "Tasks/GLTFDelayedAnimationTasks.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "LevelSequenceActor.h"

FGLTFJsonAnimation* FGLTFAnimationConverter::Convert(FGLTFJsonNode* RootNode, const USkeletalMesh* SkeletalMesh, const UAnimSequence* AnimSequence)
{
	if (AnimSequence->GetNumberOfSampledKeys() < 0)
	{
		// TODO: report warning
		return nullptr;
	}

	const USkeleton* AnimSkeleton = AnimSequence->GetSkeleton();
	if (AnimSkeleton == nullptr)
	{
		// TODO: report error
		return nullptr;
	}

	const USkeleton* MeshSkeleton = SkeletalMesh->GetSkeleton();
	if (AnimSkeleton != MeshSkeleton)
	{
		// TODO: report error
		return nullptr;
	}

	FGLTFJsonAnimation* JsonAnimation = Builder.AddAnimation();
	Builder.ScheduleSlowTask<FGLTFDelayedAnimSequenceTask>(Builder, RootNode, SkeletalMesh, AnimSequence, JsonAnimation);
	return JsonAnimation;
}

FGLTFJsonAnimation* FGLTFAnimationDataConverter::Convert(FGLTFJsonNode* RootNode, const USkeletalMeshComponent* SkeletalMeshComponent)
{
	const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
	if (SkeletalMesh == nullptr)
	{
		return nullptr;
	}

	const UAnimSequence* AnimSequence = Cast<UAnimSequence>(SkeletalMeshComponent->AnimationData.AnimToPlay);
	if (AnimSequence == nullptr)
	{
		return nullptr;
	}

	const EAnimationMode::Type AnimationMode = SkeletalMeshComponent->GetAnimationMode();
	if (AnimationMode != EAnimationMode::AnimationSingleNode)
	{
		return nullptr;
	}

	return Builder.AddUniqueAnimation(RootNode, SkeletalMesh, AnimSequence);
}

FGLTFJsonAnimation* FGLTFLevelSequenceConverter::Convert(const ULevel* Level, const ULevelSequence* LevelSequence)
{
	FGLTFJsonAnimation* JsonAnimation = Builder.AddAnimation();
	Builder.ScheduleSlowTask<FGLTFDelayedLevelSequenceTask>(Builder, Level, LevelSequence, JsonAnimation);
	return JsonAnimation;
}

FGLTFJsonAnimation* FGLTFLevelSequenceDataConverter::Convert(const ALevelSequenceActor* LevelSequenceActor)
{
	const ULevel* Level = LevelSequenceActor->GetLevel();
	if (Level == nullptr)
	{
		return nullptr;
	}

	const ULevelSequence* LevelSequence = LevelSequenceActor->GetSequence();
	if (LevelSequence == nullptr)
	{
#if WITH_EDITORONLY_DATA
		// TODO: find a nicer way to load level sequence synchronously without relying on a deprecated engine api
		LevelSequence = Cast<ULevelSequence>(LevelSequenceActor->LevelSequence_DEPRECATED.TryLoad());
		if (LevelSequence == nullptr)
		{
			return nullptr;
		}
#else
		return nullptr;
#endif
	}

	return Builder.AddUniqueAnimation(Level, LevelSequence);
}
