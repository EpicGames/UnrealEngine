// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFBoneUtilities.h"
#include "Animation/AnimationPoseData.h"
#include "Animation/AnimSequence.h"
#include "Animation/AttributesRuntime.h"
#include "ReferenceSkeleton.h"
#include "BonePose.h"

FTransform FGLTFBoneUtilities::GetBindTransform(const FReferenceSkeleton& ReferenceSkeleton, int32 BoneIndex)
{
	const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton.GetRefBoneInfo();
	const TArray<FTransform>& BonePoses = ReferenceSkeleton.GetRefBonePose();

	FTransform BindTransform = BonePoses[BoneIndex];
	int32 ParentBoneIndex = BoneInfos[BoneIndex].ParentIndex;

	while (ParentBoneIndex != INDEX_NONE)
	{
		BindTransform *= BonePoses[ParentBoneIndex];
		ParentBoneIndex = BoneInfos[ParentBoneIndex].ParentIndex;
	}

	return BindTransform;
}

void FGLTFBoneUtilities::GetBoneIndices(const FReferenceSkeleton& ReferenceSkeleton, TArray<FBoneIndexType>& OutBoneIndices)
{
	const int32 BoneCount = ReferenceSkeleton.GetNum();
	OutBoneIndices.AddUninitialized(BoneCount);

	for (int32 BoneIndex = 0; BoneIndex < BoneCount; ++BoneIndex)
	{
		OutBoneIndices[BoneIndex] = static_cast<FBoneIndexType>(BoneIndex);
	}
}

void FGLTFBoneUtilities::GetFrameTimestamps(const UAnimSequence* AnimSequence, TArray<float>& OutFrameTimestamps)
{
	const int32 FrameCount = AnimSequence->GetNumberOfSampledKeys();
	OutFrameTimestamps.AddUninitialized(FrameCount);

	const float SequenceLength = AnimSequence->GetPlayLength();
	const float FrameLength = FrameCount > 1 ? SequenceLength / (FrameCount - 1) : 0;

	for (int32 FrameIndex = 0; FrameIndex < FrameCount; ++FrameIndex)
	{
		OutFrameTimestamps[FrameIndex] = FMath::Clamp(FrameIndex * FrameLength, 0.0f, SequenceLength);
	}
}

void FGLTFBoneUtilities::GetBoneTransformsByFrame(const UObject* SkeletalMeshOrSkeleton, const UAnimSequence* AnimSequence, const TArray<FBoneIndexType>& BoneIndices, const TArray<float>& FrameTimestamps, TArray<TArray<FTransform>>& OutBoneTransformsByFrame)
{
	// Make sure to free stack allocations made by FCompactPose, FBlendedCurve, and FStackCustomAttributes when end of scope
	FMemMark Mark(FMemStack::Get());

	FBoneContainer BoneContainer;
	BoneContainer.InitializeTo(BoneIndices, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *const_cast<UObject*>(SkeletalMeshOrSkeleton));

	const int32 FrameCount = FrameTimestamps.Num();
	OutBoneTransformsByFrame.AddDefaulted(FrameCount);

	FCompactPose Pose;
	Pose.SetBoneContainer(&BoneContainer);

	FBlendedCurve Curve;
	Curve.InitFrom(BoneContainer);

	UE::Anim::FStackAttributeContainer Attributes;
	FAnimationPoseData PoseData(Pose, Curve, Attributes);

	for (int32 Frame = 0; Frame < FrameCount; ++Frame)
	{
		const FAnimExtractContext ExtractionContext(static_cast<double>(FrameTimestamps[Frame])); // TODO: set bExtractRootMotion?
		AnimSequence->GetBonePose(PoseData, ExtractionContext);
		Pose.CopyBonesTo(OutBoneTransformsByFrame[Frame]);
	}

	// Clear all stack allocations to allow FMemMark to free them
	Pose.Empty();
	Curve.Empty();
	Attributes.Empty();
}
