// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BoneIndices.h"

class UAnimSequence;
struct FReferenceSkeleton;

struct FGLTFBoneUtilities
{
	static FTransform GetBindTransform(const FReferenceSkeleton& ReferenceSkeleton, int32 BoneIndex);

	static void GetBoneIndices(const FReferenceSkeleton& ReferenceSkeleton, TArray<FBoneIndexType>& OutBoneIndices);

	static void GetFrameTimestamps(const UAnimSequence* AnimSequence, TArray<float>& OutFrameTimestamps);

	static void GetBoneTransformsByFrame(const UObject* SkeletalMeshOrSkeleton, const UAnimSequence* AnimSequence, const TArray<FBoneIndexType>& BoneIndices, const TArray<float>& FrameTimestamps, TArray<TArray<FTransform>>& OutBoneTransformsByFrame);
};
