// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFBufferAdapter.h"
#include "Converters/GLTFBufferUtilities.h"
#include "RawIndexBuffer.h"
#include "Rendering/ColorVertexBuffer.h"
#include "Rendering/PositionVertexBuffer.h"
#include "Rendering/StaticMeshVertexBuffer.h"
#include "Rendering/SkinWeightVertexBuffer.h"

class FGLTFBufferAdapterCPU final : public IGLTFBufferAdapter
{
public:

	FGLTFBufferAdapterCPU(const void* Data)
		: Data(Data)
	{
	}

	virtual const uint8* GetData() override
	{
		return static_cast<const uint8*>(Data);
	}

	const void* Data;
};

class FGLTFBufferAdapterGPU final : public IGLTFBufferAdapter
{
public:

	FGLTFBufferAdapterGPU(FRHIBuffer* RHIBuffer)
	{
		FGLTFBufferUtilities::ReadRHIBuffer(RHIBuffer, DataBuffer);
	}

	virtual const uint8* GetData() override
	{
		return DataBuffer.Num() > 0 ? DataBuffer.GetData() : nullptr;
	}

	TArray<uint8> DataBuffer;
};

TUniquePtr<IGLTFBufferAdapter> IGLTFBufferAdapter::GetIndices(const FRawStaticIndexBuffer* IndexBuffer)
{
	const void* IndexData = FGLTFBufferUtilities::GetCPUBuffer(IndexBuffer);
	if (IndexData != nullptr && FGLTFBufferUtilities::HasCPUAccess(IndexBuffer)) return MakeUnique<FGLTFBufferAdapterCPU>(IndexData);
	return MakeUnique<FGLTFBufferAdapterGPU>(IndexBuffer->IndexBufferRHI);
}

TUniquePtr<IGLTFBufferAdapter> IGLTFBufferAdapter::GetIndices(const FRawStaticIndexBuffer16or32Interface* IndexBuffer)
{
	const void* IndexData =FGLTFBufferUtilities::GetCPUBuffer(IndexBuffer);
	if (IndexData != nullptr && FGLTFBufferUtilities::HasCPUAccess(IndexBuffer)) return MakeUnique<FGLTFBufferAdapterCPU>(IndexData);
	return MakeUnique<FGLTFBufferAdapterGPU>(IndexBuffer->IndexBufferRHI);
}

TUniquePtr<IGLTFBufferAdapter> IGLTFBufferAdapter::GetPositions(const FPositionVertexBuffer* VertexBuffer)
{
	const void* PositionData = VertexBuffer->GetVertexData();
	if (PositionData != nullptr && FGLTFBufferUtilities::HasCPUAccess(VertexBuffer)) return MakeUnique<FGLTFBufferAdapterCPU>(PositionData);
	return MakeUnique<FGLTFBufferAdapterGPU>(VertexBuffer->VertexBufferRHI);
}

TUniquePtr<IGLTFBufferAdapter> IGLTFBufferAdapter::GetColors(const FColorVertexBuffer* VertexBuffer)
{
	const void* ColorData = VertexBuffer->GetVertexData();
	if (ColorData != nullptr && FGLTFBufferUtilities::HasCPUAccess(VertexBuffer)) return MakeUnique<FGLTFBufferAdapterCPU>(ColorData);
	return MakeUnique<FGLTFBufferAdapterGPU>(VertexBuffer->VertexBufferRHI);
}

TUniquePtr<IGLTFBufferAdapter> IGLTFBufferAdapter::GetTangents(const FStaticMeshVertexBuffer* VertexBuffer)
{
	const void* TangentData = VertexBuffer->GetTangentData();
	if (TangentData != nullptr && FGLTFBufferUtilities::HasCPUAccess(VertexBuffer)) return MakeUnique<FGLTFBufferAdapterCPU>(TangentData);
	return MakeUnique<FGLTFBufferAdapterGPU>(VertexBuffer->TangentsVertexBuffer.VertexBufferRHI);
}

TUniquePtr<IGLTFBufferAdapter> IGLTFBufferAdapter::GetUVs(const FStaticMeshVertexBuffer* VertexBuffer)
{
	const void* UVData = VertexBuffer->GetTexCoordData();
	if (UVData != nullptr && FGLTFBufferUtilities::HasCPUAccess(VertexBuffer)) return MakeUnique<FGLTFBufferAdapterCPU>(UVData);
	return MakeUnique<FGLTFBufferAdapterGPU>(VertexBuffer->TexCoordVertexBuffer.VertexBufferRHI);
}

TUniquePtr<IGLTFBufferAdapter> IGLTFBufferAdapter::GetInfluences(const FSkinWeightVertexBuffer* VertexBuffer)
{
	const FSkinWeightDataVertexBuffer* InfluenceBuffer = VertexBuffer->GetDataVertexBuffer();
	const void* InfluenceData = InfluenceBuffer->GetWeightData();
	if (InfluenceData != nullptr && FGLTFBufferUtilities::HasCPUAccess(VertexBuffer)) return MakeUnique<FGLTFBufferAdapterCPU>(InfluenceData);
	return MakeUnique<FGLTFBufferAdapterGPU>(InfluenceBuffer->VertexBufferRHI);
}

TUniquePtr<IGLTFBufferAdapter> IGLTFBufferAdapter::GetLookups(const FSkinWeightVertexBuffer* VertexBuffer)
{
	const FSkinWeightLookupVertexBuffer* LookupBuffer = VertexBuffer->GetLookupVertexBuffer();
	const void* LookupData = LookupBuffer->GetLookupData();
	if (LookupData != nullptr && FGLTFBufferUtilities::HasCPUAccess(VertexBuffer)) return MakeUnique<FGLTFBufferAdapterCPU>(LookupData);
	return MakeUnique<FGLTFBufferAdapterGPU>(LookupBuffer->VertexBufferRHI);
}
