// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FRawStaticIndexBuffer;
class FRawStaticIndexBuffer16or32Interface;
class FPositionVertexBuffer;
class FColorVertexBuffer;
class FStaticMeshVertexBuffer;
class FSkinWeightVertexBuffer;

class IGLTFBufferAdapter
{
public:

	virtual ~IGLTFBufferAdapter() = default;

	virtual const uint8* GetData() = 0;

	static TUniquePtr<IGLTFBufferAdapter> GetIndices(const FRawStaticIndexBuffer* IndexBuffer);
	static TUniquePtr<IGLTFBufferAdapter> GetIndices(const FRawStaticIndexBuffer16or32Interface* IndexBuffer);
	static TUniquePtr<IGLTFBufferAdapter> GetPositions(const FPositionVertexBuffer* VertexBuffer);
	static TUniquePtr<IGLTFBufferAdapter> GetColors(const FColorVertexBuffer* VertexBuffer);
	static TUniquePtr<IGLTFBufferAdapter> GetTangents(const FStaticMeshVertexBuffer* VertexBuffer);
	static TUniquePtr<IGLTFBufferAdapter> GetUVs(const FStaticMeshVertexBuffer* VertexBuffer);
	static TUniquePtr<IGLTFBufferAdapter> GetInfluences(const FSkinWeightVertexBuffer* VertexBuffer);
	static TUniquePtr<IGLTFBufferAdapter> GetLookups(const FSkinWeightVertexBuffer* VertexBuffer);
};
