// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFBufferUtilities.h"
#include "Rendering/SkeletalMeshRenderData.h"

bool FGLTFBufferUtilities::HasCPUAccess(const FRawStaticIndexBuffer* IndexBuffer)
{
#if WITH_EDITOR
	return true;
#else
	return IndexBuffer->GetAllowCPUAccess();
#endif
}

bool FGLTFBufferUtilities::HasCPUAccess(const FRawStaticIndexBuffer16or32Interface* IndexBuffer)
{
#if WITH_EDITOR
	return true;
#else
	return IndexBuffer->GetNeedsCPUAccess();
#endif
}

bool FGLTFBufferUtilities::HasCPUAccess(const FPositionVertexBuffer* VertexBuffer)
{
#if WITH_EDITOR
	return true;
#else
	return VertexBuffer->GetAllowCPUAccess();
#endif
}

bool FGLTFBufferUtilities::HasCPUAccess(const FColorVertexBuffer* VertexBuffer)
{
#if WITH_EDITOR
	return true;
#else
	return VertexBuffer->GetAllowCPUAccess();
#endif
}

bool FGLTFBufferUtilities::HasCPUAccess(const FStaticMeshVertexBuffer* VertexBuffer)
{
#if WITH_EDITOR
	return true;
#else
	return const_cast<FStaticMeshVertexBuffer*>(VertexBuffer)->GetAllowCPUAccess();
#endif
}

bool FGLTFBufferUtilities::HasCPUAccess(const FSkinWeightVertexBuffer* VertexBuffer)
{
#if WITH_EDITOR
	return true;
#else
	return VertexBuffer->GetNeedsCPUAccess();
#endif
}

const void* FGLTFBufferUtilities::GetCPUBuffer(const FRawStaticIndexBuffer* IndexBuffer)
{
	return IndexBuffer->Is32Bit() ? static_cast<const void*>(IndexBuffer->AccessStream32()) : static_cast<const void*>(IndexBuffer->AccessStream16());
}

const void* FGLTFBufferUtilities::GetCPUBuffer(const FRawStaticIndexBuffer16or32Interface* IndexBuffer)
{
	return IndexBuffer->GetResourceDataSize() > 0 ? const_cast<FRawStaticIndexBuffer16or32Interface*>(IndexBuffer)->GetPointerTo(0) : nullptr;
}

void FGLTFBufferUtilities::ReadRHIBuffer(FRHIBuffer* SourceBuffer, TArray<uint8>& OutData)
{
	OutData.Empty();

	if (SourceBuffer == nullptr)
	{
		return;
	}

	const uint32 NumBytes = SourceBuffer->GetSize();
	if (NumBytes == 0)
	{
		return;
	}

	const EBufferUsageFlags Usage = SourceBuffer->GetUsage();
	if ((Usage & BUF_Static) != BUF_Static)
	{
		return; // Some RHI implementations only support reading static buffers
	}

	OutData.AddUninitialized(NumBytes);
	void *DstData = OutData.GetData();

	ENQUEUE_RENDER_COMMAND(ReadRHIBuffer)(
		[SourceBuffer, NumBytes, DstData](FRHICommandListImmediate& RHICmdList)
		{
			const void* SrcData = RHICmdList.LockBuffer(SourceBuffer, 0, NumBytes, RLM_ReadOnly);
			FMemory::Memcpy(DstData, SrcData, NumBytes);
			RHICmdList.UnlockBuffer(SourceBuffer);
		}
	);

	FlushRenderingCommands();
}
