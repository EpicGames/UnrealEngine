// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FColorVertexBuffer;
class FPositionVertexBuffer;
class FRawStaticIndexBuffer;
class FRawStaticIndexBuffer16or32Interface;
class FRHIBuffer;
class FSkinWeightVertexBuffer;
class FSkinWeightDataVertexBuffer;
class FSkinWeightLookupVertexBuffer;
class FStaticMeshVertexBuffer;

struct FGLTFBufferUtilities
{
	static bool HasCPUAccess(const FRawStaticIndexBuffer* IndexBuffer);
	static bool HasCPUAccess(const FRawStaticIndexBuffer16or32Interface* IndexBuffer);
	static bool HasCPUAccess(const FPositionVertexBuffer* VertexBuffer);
	static bool HasCPUAccess(const FColorVertexBuffer* VertexBuffer);
	static bool HasCPUAccess(const FStaticMeshVertexBuffer* VertexBuffer);
	static bool HasCPUAccess(const FSkinWeightVertexBuffer* VertexBuffer);

	static const void* GetCPUBuffer(const FRawStaticIndexBuffer* IndexBuffer);
	static const void* GetCPUBuffer(const FRawStaticIndexBuffer16or32Interface* IndexBuffer);

	static void ReadRHIBuffer(FRHIBuffer* SourceBuffer, TArray<uint8>& OutData);
};
