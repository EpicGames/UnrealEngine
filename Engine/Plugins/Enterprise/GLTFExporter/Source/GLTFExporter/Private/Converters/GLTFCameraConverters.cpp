// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFCameraConverters.h"
#include "Builders/GLTFContainerBuilder.h"
#include "Utilities/GLTFCoreUtilities.h"
#include "Converters/GLTFNameUtilities.h"
#include "Camera/CameraComponent.h"
#include "GameFramework/Actor.h"

FGLTFJsonCamera* FGLTFCameraConverter::Convert(const UCameraComponent* CameraComponent)
{
	const EGLTFJsonCameraType Type = FGLTFCoreUtilities::ConvertCameraType(CameraComponent->ProjectionMode);
	if (Type == EGLTFJsonCameraType::None)
	{
		// TODO: report error (unsupported camera type)
		return nullptr;
	}

	FGLTFJsonCamera* JsonCamera = Builder.AddCamera();
	JsonCamera->Name = FGLTFNameUtilities::GetName(CameraComponent);
	JsonCamera->Type = Type;

	FMinimalViewInfo DesiredView;
	const_cast<UCameraComponent*>(CameraComponent)->GetCameraView(0, DesiredView);
	const float ExportScale = Builder.ExportOptions->ExportUniformScale;

	switch (JsonCamera->Type)
	{
		case EGLTFJsonCameraType::Orthographic:
			if (!DesiredView.bConstrainAspectRatio)
			{
				Builder.LogWarning(FString::Printf(TEXT("Aspect ratio for orthographic camera component %s (in actor %s) will be constrainted in glTF"), *CameraComponent->GetName(), *CameraComponent->GetOwner()->GetName()));
			}
			JsonCamera->Orthographic.XMag = FGLTFCoreUtilities::ConvertLength(DesiredView.OrthoWidth, ExportScale);
			JsonCamera->Orthographic.YMag = FGLTFCoreUtilities::ConvertLength(DesiredView.OrthoWidth / DesiredView.AspectRatio, ExportScale); // TODO: is this correct?
			JsonCamera->Orthographic.ZFar = FGLTFCoreUtilities::ConvertLength(DesiredView.OrthoFarClipPlane, ExportScale);
			JsonCamera->Orthographic.ZNear = FGLTFCoreUtilities::ConvertLength(DesiredView.OrthoNearClipPlane, ExportScale);
			break;

		case EGLTFJsonCameraType::Perspective:
			if (DesiredView.bConstrainAspectRatio)
			{
				JsonCamera->Perspective.AspectRatio = DesiredView.AspectRatio;
			}
			JsonCamera->Perspective.YFov = FGLTFCoreUtilities::ConvertFieldOfView(DesiredView.FOV, DesiredView.AspectRatio);
			JsonCamera->Perspective.ZNear = FGLTFCoreUtilities::ConvertLength(DesiredView.GetFinalPerspectiveNearClipPlane(), ExportScale);
			break;

		default:
			checkNoEntry();
			break;
	}

	return JsonCamera;
}
