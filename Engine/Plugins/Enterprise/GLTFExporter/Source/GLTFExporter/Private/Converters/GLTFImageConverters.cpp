// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFImageConverters.h"
#include "Converters/GLTFImageUtilities.h"
#include "Builders/GLTFConvertBuilder.h"
#include "Misc/FileHelper.h"

FGLTFJsonImage* FGLTFImageConverter::Convert(TGLTFSuperfluous<FString> Name, bool bIgnoreAlpha, FIntPoint Size, TGLTFSharedArray<FColor> Pixels)
{
	const TSharedPtr<FGLTFMemoryArchive> CompressedData = MakeShared<FGLTFMemoryArchive>();

	const EGLTFJsonMimeType MimeType = GetMimeType(Pixels->GetData(), Size, bIgnoreAlpha);
	switch (MimeType)
	{
		case EGLTFJsonMimeType::None:
			return nullptr;

		case EGLTFJsonMimeType::PNG:
			FGLTFImageUtilities::CompressToPNG(Pixels->GetData(), Size, *CompressedData);
			break;

		case EGLTFJsonMimeType::JPEG:
			FGLTFImageUtilities::CompressToJPEG(Pixels->GetData(), Size, Builder.ExportOptions->TextureImageQuality, *CompressedData);
			break;

		default:
			checkNoEntry();
			break;
	}

	FGLTFJsonImage* JsonImage = Builder.AddImage();

	if (Builder.bIsGLB)
	{
		JsonImage->Name = Name;
		JsonImage->MimeType = MimeType;
		JsonImage->BufferView = Builder.AddBufferView(CompressedData->GetData(), CompressedData->Num());
	}
	else
	{
		const TCHAR* Extension = FGLTFImageUtilities::GetFileExtension(MimeType);
		JsonImage->URI = Builder.AddExternalFile(Name + Extension, CompressedData);
	}

	return JsonImage;
}

EGLTFJsonMimeType FGLTFImageConverter::GetMimeType(const FColor* Pixels, FIntPoint Size, bool bIgnoreAlpha) const
{
	switch (Builder.ExportOptions->TextureImageFormat)
	{
		case EGLTFTextureImageFormat::None: return EGLTFJsonMimeType::None;
		case EGLTFTextureImageFormat::PNG: return EGLTFJsonMimeType::PNG;
		case EGLTFTextureImageFormat::JPEG: return bIgnoreAlpha || FGLTFImageUtilities::NoAlphaNeeded(Pixels, Size) ? EGLTFJsonMimeType::JPEG : EGLTFJsonMimeType::PNG;
		default:
			checkNoEntry();
			return EGLTFJsonMimeType::None;
	}
}
