// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFImageUtilities.h"
#include "Modules/ModuleManager.h"
#include "IImageWrapperModule.h"
#include "IImageWrapper.h"

const TCHAR* FGLTFImageUtilities::GetFileExtension(EGLTFJsonMimeType MimeType)
{
	switch (MimeType)
	{
		case EGLTFJsonMimeType::PNG:  return TEXT(".png");
		case EGLTFJsonMimeType::JPEG: return TEXT(".jpg");
		default:
			checkNoEntry();
		return TEXT("");
	}
}

FString FGLTFImageUtilities::GetUniqueFilename(const FString& BaseFilename, const FString& FileExtension, const TSet<FString>& UniqueFilenames)
{
	FString Filename = BaseFilename + FileExtension;
	if (!UniqueFilenames.Contains(Filename))
	{
		return Filename;
	}

	int32 Suffix = 1;
	do
	{
		Filename = BaseFilename + TEXT('_') + FString::FromInt(Suffix) + FileExtension;
		Suffix++;
	}
	while (UniqueFilenames.Contains(Filename));

	return Filename;
}

bool FGLTFImageUtilities::NoAlphaNeeded(const FColor* Pixels, FIntPoint Size)
{
	const int64 Count = Size.X * Size.Y;
	for (int64 Index = 0; Index < Count; ++Index)
	{
		const FColor& Pixel = Pixels[Index];
		if (Pixel.A < 255)
		{
			return false;
		}
	}

	return true;
}

bool FGLTFImageUtilities::CompressToPNG(const FColor* InPixels, FIntPoint InSize, TArray64<uint8>& OutCompressedData)
{
	return CompressToFormat(InPixels, InSize, EImageFormat::PNG, 0, OutCompressedData);
}

bool FGLTFImageUtilities::CompressToJPEG(const FColor* InPixels, FIntPoint InSize, int32 InCompressionQuality, TArray64<uint8>& OutCompressedData)
{
	// NOTE: Tests have shown that explicitly checking for greyscale images (and using EImageFormat::GrayscaleJPEG) results in the same as EImageFormat::JPEG
	return CompressToFormat(InPixels, InSize, EImageFormat::JPEG, InCompressionQuality, OutCompressedData);
}

bool FGLTFImageUtilities::CompressToFormat(const FColor* InPixels, FIntPoint InSize, EImageFormat InCompressionFormat, int32 InCompressionQuality, TArray64<uint8>& OutCompressedData)
{
	const int64 ByteLength = InSize.X * InSize.Y * sizeof(FColor);
	return CompressToFormat(InPixels, ByteLength, InSize.X, InSize.Y, ERGBFormat::BGRA, 8, InCompressionFormat, InCompressionQuality, OutCompressedData);
}

bool FGLTFImageUtilities::CompressToFormat(const void* InRawData, int64 InRawSize, int32 InWidth, int32 InHeight, ERGBFormat InRGBFormat, int32 InBitDepth, EImageFormat InCompressionFormat, int32 InCompressionQuality, TArray64<uint8>& OutCompressedData)
{
	IImageWrapperModule& ImageWrapperModule = FModuleManager::Get().LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
	const TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(InCompressionFormat);

	if (ImageWrapper.IsValid() && ImageWrapper->SetRaw(InRawData, InRawSize, InWidth, InHeight, InRGBFormat, InBitDepth))
	{
		const TArray64<uint8>& CompressedData = ImageWrapper->GetCompressed(InCompressionQuality);
		OutCompressedData = MoveTemp(const_cast<TArray64<uint8>&>(CompressedData)); // safe to do this since we don't keep image wrapper
		return true;
	}

	return false;
}
