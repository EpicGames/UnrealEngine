// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Json/GLTFJsonEnums.h"

enum class ERGBFormat : int8;
enum class EImageFormat : int8;

struct FGLTFImageUtilities
{
	static const TCHAR* GetFileExtension(EGLTFJsonMimeType MimeType);

	static FString GetUniqueFilename(const FString& BaseFilename, const FString& FileExtension, const TSet<FString>& UniqueFilenames);

	static bool NoAlphaNeeded(const FColor* Pixels, FIntPoint Size);

	static bool CompressToPNG(const FColor* InPixels, FIntPoint InSize, TArray64<uint8>& OutCompressedData);
	static bool CompressToJPEG(const FColor* InPixels, FIntPoint InSize, int32 InCompressionQuality, TArray64<uint8>& OutCompressedData);

	static bool CompressToFormat(const FColor* InPixels, FIntPoint InSize, EImageFormat InCompressionFormat, int32 InCompressionQuality, TArray64<uint8>& OutCompressedData);
	static bool CompressToFormat(const void* InRawData, int64 InRawSize, int32 InWidth, int32 InHeight, ERGBFormat InRGBFormat, int32 InBitDepth, EImageFormat InCompressionFormat, int32 InCompressionQuality, TArray64<uint8>& OutCompressedData);
};
