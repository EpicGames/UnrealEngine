// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFLightConverters.h"
#include "Builders/GLTFContainerBuilder.h"
#include "Utilities/GLTFCoreUtilities.h"
#include "Converters/GLTFNameUtilities.h"
#include "Components/LightComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/SpotLightComponent.h"

#if WITH_EDITORONLY_DATA
#include "InterchangeAssetImportData.h"
#include "InterchangeTextureLightProfileFactoryNode.h"
#endif

FGLTFJsonLight* FGLTFLightConverter::Convert(const ULightComponent* LightComponent)
{
	const EGLTFJsonLightType Type = FGLTFCoreUtilities::ConvertLightType(LightComponent->GetLightType());
	if (Type == EGLTFJsonLightType::None)
	{
		// TODO: report error (unsupported light component type)
		return nullptr;
	}

	FGLTFJsonLight* Light = Builder.AddLight();
	Light->Name = FGLTFNameUtilities::GetName(LightComponent);
	Light->Type = Type;

	if (!LightComponent->IESTexture || !LightComponent->bUseIESBrightness)
	{
		Light->Intensity = LightComponent->Intensity;
	}
	
	Light->Color = FGLTFCoreUtilities::ConvertColor3(LightComponent->LightColor);

	if (const UPointLightComponent* PointLightComponent = Cast<UPointLightComponent>(LightComponent))
	{
		Light->Range = FGLTFCoreUtilities::ConvertLength(PointLightComponent->AttenuationRadius, Builder.ExportOptions->ExportUniformScale);
	}

	if (const USpotLightComponent* SpotLightComponent = Cast<USpotLightComponent>(LightComponent))
	{
		Light->Spot.InnerConeAngle = FGLTFCoreUtilities::ConvertLightAngle(SpotLightComponent->InnerConeAngle);
		Light->Spot.OuterConeAngle = FGLTFCoreUtilities::ConvertLightAngle(SpotLightComponent->OuterConeAngle);

		// KHR_lights_punctual requires that InnerConeAngle must be greater than or equal to 0 and less than OuterConeAngle.
		Light->Spot.InnerConeAngle = FMath::Clamp(Light->Spot.InnerConeAngle, 0.0f, nextafterf(Light->Spot.OuterConeAngle, 0.0f));
		// KHR_lights_punctual requires that OuterConeAngle must be greater than InnerConeAngle and less than or equal to PI / 2.0.
		Light->Spot.OuterConeAngle = FMath::Clamp(Light->Spot.OuterConeAngle, nextafterf(Light->Spot.InnerConeAngle, HALF_PI), HALF_PI);
	}

	return Light;
}

FGLTFJsonLightIES* FGLTFLightIESConverter::Convert(const ULightComponent* LightComponent)
{
#if WITH_EDITORONLY_DATA
	if (!LightComponent->IESTexture)
	{
		return nullptr;
	}


	FGLTFJsonLightIES* LightIES = Builder.AddLightIES();

	UTextureLightProfile* TextureLightProfile = LightComponent->IESTexture.Get();

	LightIES->Name = TextureLightProfile->GetName();

	if (TextureLightProfile->AssetImportData)
	{
		UInterchangeAssetImportData* InterchangeAssetImportData = Cast<UInterchangeAssetImportData>(TextureLightProfile->AssetImportData.Get());
		if (UInterchangeTextureLightProfileFactoryNode* LightProfileFactoryNode = Cast<UInterchangeTextureLightProfileFactoryNode>(InterchangeAssetImportData->GetNodeContainer()->GetFactoryNode(InterchangeAssetImportData->NodeUniqueID)))
		{
			const TSharedPtr<FGLTFMemoryArchive> IESFileContent = MakeShared<FGLTFMemoryArchive>();
			LightProfileFactoryNode->GetIESSourceFileContents(*IESFileContent);

			if (Builder.bIsGLB)
			{
				LightIES->BufferView = Builder.AddBufferView(IESFileContent->GetData(), IESFileContent->Num());
			}
			else
			{
				LightIES->URI = Builder.AddExternalFile(LightIES->Name + TEXT(".IES"), IESFileContent);
			}
		}
	}

	return LightIES;
#else
	Builder.LogWarning(FString::Printf(
		TEXT("[%s] IES Light Export is not supported at Runtime."),
		*LightComponent->GetName()));

	return nullptr;
#endif
}

FGLTFJsonLightIESInstance* FGLTFLightIESInstanceConverter::Convert(const ULightComponent* LightComponent)
{
	if (!LightComponent->IESTexture)
	{
		return nullptr;
	}

	//IES Light:
	FGLTFJsonLightIES* IESLight = Builder.AddUniqueLightIES(LightComponent);

	if (!IESLight)
	{
		//If the IES Light is null, then null the IES Instance as well.
		return nullptr;
	}

	FGLTFJsonLightIESInstance* LightIESInstance = Builder.AddLightIESInstance();

	//Multiplier
	if (LightComponent->bUseIESBrightness)
	{
		LightIESInstance->Multiplier = LightComponent->IESBrightnessScale;
	}

	//Set the IES Light:
	LightIESInstance->LightIES = IESLight;

	return LightIESInstance;
}