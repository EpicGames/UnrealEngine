// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFLightMapConverters.h"
#include "Converters/GLTFMeshUtilities.h"
#include "Builders/GLTFContainerBuilder.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "Engine/MapBuildDataRegistry.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshResources.h"

FGLTFJsonLightMap* FGLTFLightMapConverter::Convert(const UStaticMeshComponent* StaticMeshComponent)
{
	if (Builder.ExportOptions->TextureImageFormat == EGLTFTextureImageFormat::None)
	{
		return nullptr;
	}

	const UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();

	if (StaticMesh == nullptr)
	{
		return nullptr;
	}

	const int32 LODIndex = FGLTFMeshUtilities::GetLOD(StaticMesh, StaticMeshComponent, Builder.ExportOptions->DefaultLevelOfDetail);
	const FStaticMeshLODResources& LODResources = StaticMesh->GetLODForExport(LODIndex);
	const int32 CoordinateIndex = StaticMesh->GetLightMapCoordinateIndex();

	if (CoordinateIndex < 0 || CoordinateIndex >= LODResources.GetNumTexCoords())
	{
		return nullptr;
	}

	constexpr int32 LightMapLODIndex = 0; // TODO: why is this zero?

	if (!StaticMeshComponent->LODData.IsValidIndex(LightMapLODIndex))
	{
		return nullptr;
	}

	const FStaticMeshComponentLODInfo& ComponentLODInfo = StaticMeshComponent->LODData[LightMapLODIndex];
	const FMeshMapBuildData* MeshMapBuildData = StaticMeshComponent->GetMeshMapBuildData(ComponentLODInfo);

	if (MeshMapBuildData == nullptr || MeshMapBuildData->LightMap == nullptr)
	{
		return nullptr;
	}

	const FLightMap2D* LightMap2D = MeshMapBuildData->LightMap->GetLightMap2D();

	if (LightMap2D == nullptr)
	{
		return nullptr;
	}

	const FLightMapInteraction LightMapInteraction = LightMap2D->GetInteraction(GMaxRHIFeatureLevel);
	const ULightMapTexture2D* Texture = LightMapInteraction.GetTexture(true);
	FGLTFJsonTexture* JsonTexture = Builder.AddUniqueTexture(Texture);

	if (JsonTexture == nullptr)
	{
		return nullptr;
	}

	const FVector2f CoordinateBias = FVector2f(LightMap2D->GetCoordinateBias());
	const FVector2f CoordinateScale = FVector2f(LightMap2D->GetCoordinateScale());
	const FVector4f& LightMapAdd = LightMapInteraction.GetAddArray()[0];
	const FVector4f& LightMapScale = LightMapInteraction.GetScaleArray()[0];

	FGLTFJsonLightMap* JsonLightMap = Builder.AddLightMap();
	StaticMeshComponent->GetName(JsonLightMap->Name); // TODO: use better name (similar to light and camera)
	JsonLightMap->Texture.Index = JsonTexture;
	JsonLightMap->Texture.TexCoord = CoordinateIndex;
	JsonLightMap->LightMapScale = { LightMapScale.X, LightMapScale.Y, LightMapScale.Z, LightMapScale.W };
	JsonLightMap->LightMapAdd = { LightMapAdd.X, LightMapAdd.Y, LightMapAdd.Z, LightMapAdd.W };
	JsonLightMap->CoordinateScaleBias = { CoordinateScale.X, CoordinateScale.Y, CoordinateBias.X, CoordinateBias.Y };

	return JsonLightMap;
}
