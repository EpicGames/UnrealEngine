// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFMaterialConverters.h"
#include "Converters/GLTFMaterialUtilities.h"
#include "Builders/GLTFConvertBuilder.h"
#include "Tasks/GLTFDelayedMaterialTasks.h"

void FGLTFMaterialConverter::Sanitize(const UMaterialInterface*& Material, const FGLTFMeshData*& MeshData, FGLTFIndexArray& SectionIndices)
{
	Material = Builder.ResolveProxy(Material);

	if (MeshData == nullptr ||
		Builder.ExportOptions->BakeMaterialInputs != EGLTFMaterialBakeMode::UseMeshData ||
		!FGLTFMaterialUtilities::NeedsMeshData(Material))
	{
		MeshData = nullptr;
		SectionIndices = {};
	}

#if WITH_EDITOR
	if (MeshData != nullptr)
	{
		const FMeshDescription& MeshDescription = MeshData->GetParent()->Description;

		const float DegenerateUVPercentage = UVDegenerateChecker.GetOrAdd(&MeshDescription, SectionIndices, MeshData->BakeUsingTexCoord);
		if (FMath::IsNearlyEqual(DegenerateUVPercentage, 1))
		{
			FString SectionString = TEXT("mesh section");
			SectionString += SectionIndices.Num() > 1 ? TEXT("s ") : TEXT(" ");
			SectionString += FString::JoinBy(SectionIndices, TEXT(", "), FString::FromInt);

			Builder.LogWarning(FString::Printf(
				TEXT("Material %s uses mesh data from %s but the lightmap UVs (channel %d) are nearly 100%% degenerate (in %s). Simple baking will be used as fallback"),
				*Material->GetName(),
				*MeshData->GetParent()->Name,
				MeshData->BakeUsingTexCoord,
				*SectionString));

			MeshData = nullptr;
			SectionIndices = {};
		}
	}
#endif
}

FGLTFJsonMaterial* FGLTFMaterialConverter::Convert(const UMaterialInterface* Material, const FGLTFMeshData* MeshData, FGLTFIndexArray SectionIndices)
{
	if (Material != FGLTFMaterialUtilities::GetDefaultMaterial())
	{
		FGLTFJsonMaterial* JsonMaterial = Builder.AddMaterial();
		Builder.ScheduleSlowTask<FGLTFDelayedMaterialTask>(Builder, UVOverlapChecker, UVBoundsCalculator, Material, MeshData, SectionIndices, JsonMaterial);
		return JsonMaterial;
	}

	return nullptr; // use default gltf definition
}
