// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Converters/GLTFMaterialProxyFactory.h"
#include "Converters/GLTFMaterialUtilities.h"
#include "Converters/GLTFImageUtilities.h"
#include "Utilities/GLTFProxyMaterialUtilities.h"
#include "Materials/GLTFProxyMaterialInfo.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Misc/PackageName.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "HAL/FileManager.h"
#include "ImageUtils.h"

FGLTFMaterialProxyFactory::FGLTFMaterialProxyFactory(const UGLTFProxyOptions* Options)
	: Builder(TEXT(""), CreateExportOptions(Options))
{
	Builder.Texture2DConverter = CreateTextureConverter();
	Builder.ImageConverter = CreateImageConverter();
}

UMaterialInterface* FGLTFMaterialProxyFactory::Create(UMaterialInterface* OriginalMaterial)
{
	if (FGLTFProxyMaterialUtilities::IsProxyMaterial(OriginalMaterial))
	{
		return OriginalMaterial;
	}

	if (FGLTFMaterialUtilities::NeedsMeshData(OriginalMaterial))
	{
		Builder.LogWarning(FString::Printf(
			TEXT("Material %s uses mesh data but prebaking will only use a simple quad as mesh data currently"),
			*OriginalMaterial->GetName()));
	}

	FGLTFJsonMaterial* JsonMaterial = Builder.AddUniqueMaterial(OriginalMaterial);
	if (JsonMaterial == nullptr)
	{
		// TODO: report error
		return nullptr;
	}

	MakeDirectory(RootPath);
	Builder.ProcessSlowTasks();

	UMaterialInstanceConstant* ProxyMaterial = CreateInstancedMaterial(OriginalMaterial, JsonMaterial->ShadingModel);
	if (ProxyMaterial != nullptr)
	{
		FGLTFProxyMaterialUtilities::SetProxyMaterial(OriginalMaterial, ProxyMaterial);
		SetBaseProperties(ProxyMaterial, OriginalMaterial);
		SetProxyParameters(ProxyMaterial, *JsonMaterial);
	}

	return ProxyMaterial;
}

void FGLTFMaterialProxyFactory::OpenLog()
{
	if (Builder.HasLoggedMessages())
	{
		Builder.OpenLog();
	}
}

void FGLTFMaterialProxyFactory::SetBaseProperties(UMaterialInstanceConstant* ProxyMaterial, UMaterialInterface* OriginalMaterial)
{
	const UMaterial* BaseMaterial = ProxyMaterial->GetMaterial();

	const bool bTwoSided = OriginalMaterial->IsTwoSided();
	if (bTwoSided != BaseMaterial->IsTwoSided())
	{
		ProxyMaterial->BasePropertyOverrides.bOverride_TwoSided = true;
		ProxyMaterial->BasePropertyOverrides.TwoSided = bTwoSided;
	}

	const bool bIsThinSurface = OriginalMaterial->IsThinSurface();
	if (bIsThinSurface != BaseMaterial->IsThinSurface())
	{
		ProxyMaterial->BasePropertyOverrides.bOverride_bIsThinSurface = true;
		ProxyMaterial->BasePropertyOverrides.bIsThinSurface = bIsThinSurface;
	}

	const EBlendMode BlendMode = OriginalMaterial->GetBlendMode();
	if (BlendMode != BaseMaterial->GetBlendMode())
	{
		ProxyMaterial->BasePropertyOverrides.bOverride_BlendMode = true;
		ProxyMaterial->BasePropertyOverrides.BlendMode = BlendMode;
	}

	const float OpacityMaskClipValue = OriginalMaterial->GetOpacityMaskClipValue();
	if (OpacityMaskClipValue != BaseMaterial->GetOpacityMaskClipValue())
	{
		ProxyMaterial->BasePropertyOverrides.bOverride_OpacityMaskClipValue = true;
		ProxyMaterial->BasePropertyOverrides.OpacityMaskClipValue = OpacityMaskClipValue;
	}
}

void FGLTFMaterialProxyFactory::SetProxyParameters(UMaterialInstanceConstant* ProxyMaterial, const FGLTFJsonMaterial& JsonMaterial)
{
	SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::BaseColor, JsonMaterial.PBRMetallicRoughness.BaseColorTexture);
	SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::BaseColorFactor, JsonMaterial.PBRMetallicRoughness.BaseColorFactor);

	if (JsonMaterial.ShadingModel != EGLTFJsonShadingModel::None && 
		JsonMaterial.ShadingModel != EGLTFJsonShadingModel::Unlit &&
		(int)JsonMaterial.ShadingModel < (int)EGLTFJsonShadingModel::NumShadingModels)
	{
		SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::Emissive, JsonMaterial.EmissiveTexture);
		SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::EmissiveFactor, JsonMaterial.EmissiveFactor);
		SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::EmissiveStrength, JsonMaterial.EmissiveStrength);

		SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::MetallicRoughness, JsonMaterial.PBRMetallicRoughness.MetallicRoughnessTexture);
		SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::MetallicFactor, JsonMaterial.PBRMetallicRoughness.MetallicFactor);
		SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::RoughnessFactor, JsonMaterial.PBRMetallicRoughness.RoughnessFactor);

		SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::Normal, JsonMaterial.NormalTexture);
		SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::NormalScale, JsonMaterial.NormalTexture.Scale);

		SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::Occlusion, JsonMaterial.OcclusionTexture);
		SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::OcclusionStrength, JsonMaterial.OcclusionTexture.Strength);

		SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::SpecularFactor, JsonMaterial.Specular.Factor);
		SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::SpecularTexture, JsonMaterial.Specular.Texture);

		SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::IOR, JsonMaterial.IOR.Value);

		if (JsonMaterial.ShadingModel == EGLTFJsonShadingModel::ClearCoat)
		{
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::ClearCoat, JsonMaterial.ClearCoat.ClearCoatTexture);
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::ClearCoatFactor, JsonMaterial.ClearCoat.ClearCoatFactor);

			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::ClearCoatRoughness, JsonMaterial.ClearCoat.ClearCoatRoughnessTexture);
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::ClearCoatRoughnessFactor, JsonMaterial.ClearCoat.ClearCoatRoughnessFactor);

			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::ClearCoatNormal, JsonMaterial.ClearCoat.ClearCoatNormalTexture);
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::ClearCoatNormalScale, JsonMaterial.ClearCoat.ClearCoatNormalTexture.Scale);
		}
		else if (JsonMaterial.ShadingModel == EGLTFJsonShadingModel::Sheen)
		{
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::SheenColorFactor, JsonMaterial.Sheen.ColorFactor);
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::SheenColorTexture, JsonMaterial.Sheen.ColorTexture);
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::SheenRoughnessFactor, JsonMaterial.Sheen.RoughnessFactor);
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::SheenRoughnessTexture, JsonMaterial.Sheen.RoughnessTexture);
		}
		else if (JsonMaterial.ShadingModel == EGLTFJsonShadingModel::Transmission)
		{
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::TransmissionFactor, JsonMaterial.Transmission.Factor);
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::TransmissionTexture, JsonMaterial.Transmission.Texture);
		}

		if (JsonMaterial.ShadingModel != EGLTFJsonShadingModel::SpecularGlossiness)
		{
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::IridescenceFactor, JsonMaterial.Iridescence.IridescenceFactor);
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::IridescenceTexture, JsonMaterial.Iridescence.IridescenceTexture);
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::IridescenceIOR, JsonMaterial.Iridescence.IridescenceIOR);
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::IridescenceThicknessMinimum, JsonMaterial.Iridescence.IridescenceThicknessMinimum);
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::IridescenceThicknessMaximum, JsonMaterial.Iridescence.IridescenceThicknessMaximum);
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::IridescenceThicknessTexture, JsonMaterial.Iridescence.IridescenceThicknessTexture);

			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::AnisotropyStrength, JsonMaterial.Anisotropy.AnisotropyStrength);
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::AnisotropyRotation, JsonMaterial.Anisotropy.AnisotropyRotation);
			SetProxyParameter(ProxyMaterial, FGLTFProxyMaterialInfo::AnisotropyTexture, JsonMaterial.Anisotropy.AnisotropyTexture);
		}
	}
}

void FGLTFMaterialProxyFactory::SetProxyParameter(UMaterialInstanceConstant* ProxyMaterial, const TGLTFProxyMaterialParameterInfo<float>& ParameterInfo, float Scalar)
{
	ParameterInfo.Set(ProxyMaterial, Scalar, true);
}

void FGLTFMaterialProxyFactory::SetProxyParameter(UMaterialInstanceConstant* ProxyMaterial, const TGLTFProxyMaterialParameterInfo<FLinearColor>& ParameterInfo, const FGLTFJsonColor3& Color)
{
	ParameterInfo.Set(ProxyMaterial, FLinearColor(Color.R, Color.G, Color.B), true);
}

void FGLTFMaterialProxyFactory::SetProxyParameter(UMaterialInstanceConstant* ProxyMaterial, const TGLTFProxyMaterialParameterInfo<FLinearColor>& ParameterInfo, const FGLTFJsonColor4& Color)
{
	ParameterInfo.Set(ProxyMaterial, FLinearColor(Color.R, Color.G, Color.B, Color.A), true);
}

void FGLTFMaterialProxyFactory::SetProxyParameter(UMaterialInstanceConstant* ProxyMaterial, const FGLTFProxyMaterialTextureParameterInfo& ParameterInfo, const FGLTFJsonTextureInfo& TextureInfo)
{
	UTexture* Texture = FindOrCreateTexture(TextureInfo.Index, ParameterInfo);
	if (Texture == nullptr)
	{
		return;
	}

	ParameterInfo.Texture.Set(ProxyMaterial, Texture, true);
	ParameterInfo.UVIndex.Set(ProxyMaterial, static_cast<float>(TextureInfo.TexCoord), true);
	ParameterInfo.UVOffset.Set(ProxyMaterial, FLinearColor(TextureInfo.Transform.Offset.X, TextureInfo.Transform.Offset.Y, 0, 0), true);
	ParameterInfo.UVScale.Set(ProxyMaterial, FLinearColor(TextureInfo.Transform.Scale.X, TextureInfo.Transform.Scale.Y, 0, 0), true);
	ParameterInfo.UVRotation.Set(ProxyMaterial, TextureInfo.Transform.Rotation, true);
	ParameterInfo.TilingMethod.Set(ProxyMaterial, FLinearColor(FGLTFMaterialUtilities::FromTextureWrap(TextureInfo.Index->Sampler->WrapS), FGLTFMaterialUtilities::FromTextureWrap(TextureInfo.Index->Sampler->WrapT), 0, 0), true);
}

UTexture2D* FGLTFMaterialProxyFactory::FindOrCreateTexture(FGLTFJsonTexture* JsonTexture, const FGLTFProxyMaterialTextureParameterInfo& ParameterInfo)
{
	if (JsonTexture == nullptr)
	{
		return nullptr;
	}

	// TODO: fix potential conflict when same texture used for different material properties that have different encoding (sRGB vs Linear, Normalmap compression etc)

	UTexture2D** FoundPtr = Textures.Find(JsonTexture);
	if (FoundPtr != nullptr)
	{
		return *FoundPtr;
	}

	const FGLTFImageData* ImageData = Images.Find(JsonTexture->Source);
	if (ImageData == nullptr)
	{
		// TODO: report error
		return nullptr;
	}

	const FGLTFJsonSampler& JsonSampler = *JsonTexture->Sampler;
	UTexture2D* Texture = CreateTexture(ImageData, JsonSampler, ParameterInfo);
	Textures.Add(JsonTexture, Texture);
	return Texture;
}

UTexture2D* FGLTFMaterialProxyFactory::CreateTexture(const FGLTFImageData* ImageData, const FGLTFJsonSampler& JsonSampler, const FGLTFProxyMaterialTextureParameterInfo& ParameterInfo)
{
	const bool bSRGB = ParameterInfo == FGLTFProxyMaterialInfo::BaseColor || ParameterInfo == FGLTFProxyMaterialInfo::Emissive;
	const bool bNormalMap = ParameterInfo == FGLTFProxyMaterialInfo::Normal || ParameterInfo == FGLTFProxyMaterialInfo::ClearCoatNormal;

	FCreateTexture2DParameters TexParams;
	TexParams.bUseAlpha = !ImageData->bIgnoreAlpha;
	TexParams.CompressionSettings = bNormalMap ? TC_Normalmap : TC_Default;
	TexParams.bDeferCompression = true;
	TexParams.bSRGB = bSRGB;
	TexParams.TextureGroup = bNormalMap ? TEXTUREGROUP_WorldNormalMap : TEXTUREGROUP_World;
	TexParams.SourceGuidHash = FGuid();

	const FString BaseName = TEXT("T_GLTF_") + ImageData->Filename;
	UPackage* Package = FindOrCreatePackage(BaseName);

	UTexture2D* Texture = FImageUtils::CreateTexture2D(ImageData->Size.X, ImageData->Size.Y, *ImageData->Pixels, Package, BaseName, RF_Public | RF_Standalone, TexParams);
	
	Texture->PreEditChange(nullptr);
	Texture->Filter = ConvertFilter(JsonSampler.MagFilter);
	Texture->AddressX = ConvertWrap(JsonSampler.WrapS);
	Texture->AddressY = ConvertWrap(JsonSampler.WrapT);
	Texture->PostEditChange();

	return Texture;
}

UMaterialInstanceConstant* FGLTFMaterialProxyFactory::CreateInstancedMaterial(UMaterialInterface* OriginalMaterial, EGLTFJsonShadingModel ShadingModel)
{
	const FString BaseName = TEXT("MI_GLTF_") + OriginalMaterial->GetName();
	UPackage* Package = FindOrCreatePackage(BaseName);
	return FGLTFProxyMaterialUtilities::CreateProxyMaterial(ShadingModel, Package, *BaseName, RF_Public | RF_Standalone);
}

UPackage* FGLTFMaterialProxyFactory::FindOrCreatePackage(const FString& BaseName)
{
	const FString PackageName = RootPath / BaseName;
	UPackage* Package = CreatePackage(*PackageName);
	Package->FullyLoad();
	Package->Modify();
	// TODO: do we need to delete any old assets in the package?
	return Package;
}

TUniquePtr<IGLTFTexture2DConverter> FGLTFMaterialProxyFactory::CreateTextureConverter()
{
	class FGLTFCustomTexture2DConverter final: public IGLTFTexture2DConverter
	{
	public:

		FGLTFMaterialProxyFactory& Factory;

		FGLTFCustomTexture2DConverter(FGLTFMaterialProxyFactory& Factory)
			: Factory(Factory)
		{
		}

	protected:

		virtual void Sanitize(const UTexture2D*& Texture2D, bool& bToSRGB, TextureAddress& WrapS, TextureAddress& WrapT) override
		{
			bToSRGB = false; // ignore
		}

		virtual FGLTFJsonTexture* Convert(const UTexture2D* Texture2D, bool bToSRGB, TextureAddress WrapS, TextureAddress WrapT) override
		{
			FGLTFJsonTexture* Texture = Factory.Builder.AddTexture();
			Factory.Textures.Add(Texture, const_cast<UTexture2D*>(Texture2D));
			return Texture;
		}
	};

	return MakeUnique<FGLTFCustomTexture2DConverter>(*this);
}

TUniquePtr<IGLTFImageConverter> FGLTFMaterialProxyFactory::CreateImageConverter()
{
	class FGLTFCustomImageConverter final: public IGLTFImageConverter
	{
	public:

		FGLTFMaterialProxyFactory& Factory;
		TSet<FString> UniqueFilenames;

		FGLTFCustomImageConverter(FGLTFMaterialProxyFactory& Factory)
			: Factory(Factory)
		{
		}

	protected:

		virtual FGLTFJsonImage* Convert(TGLTFSuperfluous<FString> Name, bool bIgnoreAlpha, FIntPoint Size, TGLTFSharedArray<FColor> Pixels) override
		{
			const FString Filename = FGLTFImageUtilities::GetUniqueFilename(Name, TEXT(""), UniqueFilenames);
			UniqueFilenames.Add(Filename);

			FGLTFJsonImage* Image = Factory.Builder.AddImage();
			Factory.Images.Add(Image, { Filename, bIgnoreAlpha, Size, Pixels });
			return Image;
		}
	};

	return MakeUnique<FGLTFCustomImageConverter>(*this);
}

bool FGLTFMaterialProxyFactory::MakeDirectory(const FString& PackagePath)
{
	const FString DirPath = FPaths::ConvertRelativePathToFull(FPackageName::LongPackageNameToFilename(PackagePath + TEXT("/")));
	if (DirPath.IsEmpty())
	{
		return false;
	}

	bool bResult = true;
	if (!IFileManager::Get().DirectoryExists(*DirPath))
	{
		bResult = IFileManager::Get().MakeDirectory(*DirPath, true);
	}

	if (bResult)
	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		AssetRegistryModule.Get().AddPath(PackagePath);
	}

	return bResult;
}

UGLTFExportOptions* FGLTFMaterialProxyFactory::CreateExportOptions(const UGLTFProxyOptions* ProxyOptions)
{
	UGLTFExportOptions* ExportOptions = NewObject<UGLTFExportOptions>();
	ExportOptions->ResetToDefault();
	ExportOptions->bExportProxyMaterials = false;
	ExportOptions->BakeMaterialInputs = ProxyOptions->bBakeMaterialInputs ? EGLTFMaterialBakeMode::Simple : EGLTFMaterialBakeMode::Disabled;
	ExportOptions->bExportThinTranslucentMaterials = ProxyOptions->bUseThinTranslucentShadingModel;
	ExportOptions->DefaultMaterialBakeSize = ProxyOptions->DefaultMaterialBakeSize;
	ExportOptions->DefaultMaterialBakeFilter = ProxyOptions->DefaultMaterialBakeFilter;
	ExportOptions->DefaultMaterialBakeTiling = ProxyOptions->DefaultMaterialBakeTiling;
	ExportOptions->DefaultInputBakeSettings = ProxyOptions->DefaultInputBakeSettings;
	ExportOptions->bAdjustNormalmaps = false;
	return ExportOptions;
}

TextureAddress FGLTFMaterialProxyFactory::ConvertWrap(EGLTFJsonTextureWrap Wrap)
{
	switch (Wrap)
	{
		case EGLTFJsonTextureWrap::Repeat:         return TA_Wrap;
		case EGLTFJsonTextureWrap::MirroredRepeat: return TA_Mirror;
		case EGLTFJsonTextureWrap::ClampToEdge:    return TA_Clamp;
		default:
			checkNoEntry();
			return TA_MAX;
	}
}

TextureFilter FGLTFMaterialProxyFactory::ConvertFilter(EGLTFJsonTextureFilter Filter)
{
	switch (Filter)
	{
		case EGLTFJsonTextureFilter::Nearest:              return TF_Nearest;
		case EGLTFJsonTextureFilter::NearestMipmapNearest: return TF_Nearest;
		case EGLTFJsonTextureFilter::LinearMipmapNearest:  return TF_Bilinear;
		case EGLTFJsonTextureFilter::NearestMipmapLinear:  return TF_Bilinear;
		case EGLTFJsonTextureFilter::Linear:               return TF_Trilinear;
		case EGLTFJsonTextureFilter::LinearMipmapLinear:   return TF_Trilinear;
		default:
			checkNoEntry();
			return TF_MAX;
	}
}

#endif
