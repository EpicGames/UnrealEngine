// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Builders/GLTFConvertBuilder.h"
#include "Options/GLTFProxyOptions.h"
#include "Options/GLTFExportOptions.h"
#include "Materials/GLTFProxyMaterialParameterInfo.h"

class UMaterialInstanceConstant;

class FGLTFMaterialProxyFactory
{
public:

	FGLTFMaterialProxyFactory(const UGLTFProxyOptions* Options = nullptr);

	UMaterialInterface* Create(UMaterialInterface* OriginalMaterial);
	void OpenLog();

	FString RootPath;

private:

	struct FGLTFImageData
	{
		FString Filename;
		bool bIgnoreAlpha;
		FIntPoint Size;
		TGLTFSharedArray<FColor> Pixels;
	};

	void SetBaseProperties(UMaterialInstanceConstant* ProxyMaterial, UMaterialInterface* OriginalMaterial);
	void SetProxyParameters(UMaterialInstanceConstant* ProxyMaterial, const FGLTFJsonMaterial& JsonMaterial);

	void SetProxyParameter(UMaterialInstanceConstant* ProxyMaterial, const TGLTFProxyMaterialParameterInfo<float>& ParameterInfo, float Scalar);
	void SetProxyParameter(UMaterialInstanceConstant* ProxyMaterial, const TGLTFProxyMaterialParameterInfo<FLinearColor>& ParameterInfo, const FGLTFJsonColor3& Color);
	void SetProxyParameter(UMaterialInstanceConstant* ProxyMaterial, const TGLTFProxyMaterialParameterInfo<FLinearColor>& ParameterInfo, const FGLTFJsonColor4& Color);
	void SetProxyParameter(UMaterialInstanceConstant* ProxyMaterial, const FGLTFProxyMaterialTextureParameterInfo& ParameterInfo, const FGLTFJsonTextureInfo& TextureInfo);

	UTexture2D* FindOrCreateTexture(FGLTFJsonTexture* JsonTexture, const FGLTFProxyMaterialTextureParameterInfo& ParameterInfo);
	UTexture2D* CreateTexture(const FGLTFImageData* ImageData, const FGLTFJsonSampler& JsonSampler, const FGLTFProxyMaterialTextureParameterInfo& ParameterInfo);

	UMaterialInstanceConstant* CreateInstancedMaterial(UMaterialInterface* OriginalMaterial, EGLTFJsonShadingModel ShadingModel);

	UPackage* FindOrCreatePackage(const FString& BaseName);

	TUniquePtr<IGLTFTexture2DConverter> CreateTextureConverter();
	TUniquePtr<IGLTFImageConverter> CreateImageConverter();

	static bool MakeDirectory(const FString& String);

	static UGLTFExportOptions* CreateExportOptions(const UGLTFProxyOptions* ProxyOptions);

	static TextureAddress ConvertWrap(EGLTFJsonTextureWrap Wrap);
	static TextureFilter ConvertFilter(EGLTFJsonTextureFilter Filter);

	FGLTFConvertBuilder Builder;

	TMap<FGLTFJsonTexture*, UTexture2D*> Textures;
	TMap<FGLTFJsonImage*, FGLTFImageData> Images;
};

#endif
