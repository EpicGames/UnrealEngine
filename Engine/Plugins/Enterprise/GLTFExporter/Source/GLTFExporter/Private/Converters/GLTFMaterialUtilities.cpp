// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFMaterialUtilities.h"
#include "Converters/GLTFTextureUtilities.h"
#include "Converters/GLTFNameUtilities.h"
#include "Converters/GLTFProxyMaterialCompiler.h"
#include "Utilities/GLTFProxyMaterialUtilities.h"
#include "Materials/Material.h"
#include "Modules/ModuleManager.h"
#include "Engine/RendererSettings.h"
#include "Materials/MaterialExpressionTextureObject.h"
#include "Materials/MaterialExpressionTextureObjectParameter.h"
#if WITH_EDITOR
#include "IMaterialBakingModule.h"
#include "MaterialBakingStructures.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#endif

#include "Materials/MaterialInstance.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionTextureSampleParameter2D.h"

#include "Utilities/GLTFCoreUtilities.h"

#if GLTF_EXPORT_ENABLE
#include "Gltf/InterchangeGLTFMaterial.h"
#endif

UMaterialInterface* FGLTFMaterialUtilities::GetDefaultMaterial()
{
	static UMaterial* DefaultMaterial = FGLTFProxyMaterialUtilities::GetBaseMaterial(EGLTFJsonShadingModel::Default);
	return DefaultMaterial;
}

bool FGLTFMaterialUtilities::IsClearCoatBottomNormalEnabled()
{
	return GetDefault<URendererSettings>()->bClearCoatEnableSecondNormal != 0;
}

#if WITH_EDITOR


bool FGLTFMaterialUtilities::IsNormalMap(const FMaterialPropertyEx& Property)
{
	return Property == MP_Normal || Property == FMaterialPropertyEx::ClearCoatBottomNormal;
}

bool FGLTFMaterialUtilities::IsSRGB(const FMaterialPropertyEx& Property)
{
	return Property == MP_BaseColor || Property == MP_EmissiveColor || Property == MP_SubsurfaceColor || Property == FMaterialPropertyEx::TransmittanceColor;
}

FGuid FGLTFMaterialUtilities::GetAttributeID(const FMaterialPropertyEx& Property)
{
	return Property.IsCustomOutput()
		? FMaterialAttributeDefinitionMap::GetCustomAttributeID(Property.CustomOutput.ToString())
		: FMaterialAttributeDefinitionMap::GetID(Property.Type);
}

FGuid FGLTFMaterialUtilities::GetAttributeIDChecked(const FMaterialPropertyEx& Property)
{
	const FGuid AttributeID = GetAttributeID(Property);
	check(AttributeID != FMaterialAttributeDefinitionMap::GetDefaultID());
	return AttributeID;
}

FVector4f FGLTFMaterialUtilities::GetPropertyDefaultValue(const FMaterialPropertyEx& Property)
{
	return FMaterialAttributeDefinitionMap::GetDefaultValue(GetAttributeIDChecked(Property));
}

FVector4f FGLTFMaterialUtilities::GetPropertyMask(const FMaterialPropertyEx& Property)
{
	switch (FMaterialAttributeDefinitionMap::GetValueType(GetAttributeIDChecked(Property)))
	{
		case MCT_Float:
		case MCT_Float1: return FVector4f(1, 0, 0, 0);
		case MCT_Float2: return FVector4f(1, 1, 0, 0);
		case MCT_Float3: return FVector4f(1, 1, 1, 0);
		case MCT_Float4: return FVector4f(1, 1, 1, 1);
		default:
			checkNoEntry();
			return FVector4f();
	}
}

const FExpressionInput* FGLTFMaterialUtilities::GetInputForProperty(const UMaterialInterface* Material, const FMaterialPropertyEx& Property)
{
	if (Property.IsCustomOutput())
	{
		const FString FunctionName = Property.CustomOutput.ToString();
		const UMaterialExpressionCustomOutput* CustomOutput = GetCustomOutputByName(Material, FunctionName);
		if (CustomOutput == nullptr)
		{
			return nullptr;
		}

		// Assume custom outputs always have a single input (which is true for all supported custom outputs)
		return const_cast<UMaterialExpressionCustomOutput*>(CustomOutput)->GetInput(0);
	}

	UMaterial* UnderlyingMaterial = const_cast<UMaterial*>(Material->GetMaterial());
	return UnderlyingMaterial->GetExpressionInputForProperty(Property.Type);
}

UMaterialExpressionCustomOutput* FGLTFMaterialUtilities::GetCustomOutputByName(const UMaterialInterface* Material, const FString& FunctionName)
{
	for (const TObjectPtr<UMaterialExpression>& Expression : Material->GetMaterial()->GetExpressions())
	{
		UMaterialExpressionCustomOutput* CustomOutput = Cast<UMaterialExpressionCustomOutput>(Expression);
		if (CustomOutput != nullptr && CustomOutput->GetFunctionName() == FunctionName)
		{
			return CustomOutput;
		}
	}

	return nullptr;
}

FGLTFPropertyBakeOutput FGLTFMaterialUtilities::BakeMaterialProperty(const FIntPoint& OutputSize, const FMaterialPropertyEx& Property, const UMaterialInterface* Material, const FBox2f& TexCoordBounds, int32 TexCoordIndex, const FGLTFMeshData* MeshData, const FGLTFIndexArray& MeshSectionIndices, bool bFillAlpha, bool bAdjustNormalmaps)
{
	FMeshData MeshSet;
	MeshSet.TextureCoordinateBox = FBox2D(TexCoordBounds);
	MeshSet.TextureCoordinateIndex = TexCoordIndex;
	MeshSet.MaterialIndices = MeshSectionIndices; // NOTE: MaterialIndices is actually section indices
	if (MeshData != nullptr)
	{
		MeshSet.MeshDescription = const_cast<FMeshDescription*>(&MeshData->Description);
		MeshSet.LightMap = MeshData->LightMap;
		MeshSet.LightMapIndex = MeshData->LightMapTexCoord;
		MeshSet.LightmapResourceCluster = MeshData->LightMapResourceCluster;
		MeshSet.PrimitiveData = MeshData->PrimitiveData;
	}

	FMaterialDataEx MatSet;
	MatSet.Material = const_cast<UMaterialInterface*>(Material);
	MatSet.PropertySizes.Add(Property, OutputSize);
	MatSet.bTangentSpaceNormal = true;

	TArray<FMeshData*> MeshSettings;
	TArray<FMaterialDataEx*> MatSettings;
	MeshSettings.Add(&MeshSet);
	MatSettings.Add(&MatSet);

	TArray<FBakeOutputEx> BakeOutputs;
	IMaterialBakingModule& Module = FModuleManager::Get().LoadModuleChecked<IMaterialBakingModule>("MaterialBaking");

	Module.SetLinearBake(true);
	Module.BakeMaterials(MatSettings, MeshSettings, BakeOutputs);
	const bool bIsLinearBake = Module.IsLinearBake(Property);
	Module.SetLinearBake(false);

	FBakeOutputEx& BakeOutput = BakeOutputs[0];

	TGLTFSharedArray<FColor> BakedPixels = MakeShared<TArray<FColor>>(MoveTemp(BakeOutput.PropertyData.FindChecked(Property)));
	const FIntPoint BakedSize = BakeOutput.PropertySizes.FindChecked(Property);
	const float EmissiveScale = BakeOutput.EmissiveScale;

	if (bFillAlpha)
	{
		// NOTE: alpha is 0 by default after baking a property, but we prefer 255 (1.0).
		// It makes it easier to view the exported textures.
		for (FColor& Pixel: *BakedPixels)
		{
			Pixel.A = 255;
		}
	}

	if (bAdjustNormalmaps && IsNormalMap(Property))
	{
		// TODO: add support for adjusting normals in baking module instead
		FGLTFTextureUtilities::FlipGreenChannel(*BakedPixels);
	}

	bool bFromSRGB = !bIsLinearBake;
	FGLTFPropertyBakeOutput PropertyBakeOutput(BakedPixels, BakedSize, EmissiveScale, bFromSRGB);

	bool bToSRGB = IsSRGB(Property);
	FGLTFTextureUtilities::TransformColorSpace(*BakedPixels, bFromSRGB, bToSRGB);

	return PropertyBakeOutput;
}

FGLTFJsonTexture* FGLTFMaterialUtilities::AddTexture(FGLTFConvertBuilder& Builder, TGLTFSharedArray<FColor>& Pixels, const FIntPoint& TextureSize, bool bIgnoreAlpha, bool bIsNormalMap, const FString& TextureName, TextureAddress TextureAddress, TextureFilter TextureFilter)
{
	// TODO: reuse same texture index when image is the same
	FGLTFJsonTexture* JsonTexture = Builder.AddTexture();
	JsonTexture->Name = TextureName;
	JsonTexture->Sampler = Builder.AddUniqueSampler(TextureAddress, TextureFilter);
	JsonTexture->Source = Builder.AddUniqueImage(Pixels, TextureSize, bIgnoreAlpha, TextureName);

	return JsonTexture;
}

FLinearColor FGLTFMaterialUtilities::GetMask(const FExpressionInput& ExpressionInput)
{
	return FLinearColor(
		static_cast<float>(ExpressionInput.MaskR),
		static_cast<float>(ExpressionInput.MaskG),
		static_cast<float>(ExpressionInput.MaskB),
		static_cast<float>(ExpressionInput.MaskA)
	);
}

uint32 FGLTFMaterialUtilities::GetMaskComponentCount(const FExpressionInput& ExpressionInput)
{
	return ExpressionInput.MaskR + ExpressionInput.MaskG + ExpressionInput.MaskB + ExpressionInput.MaskA;
}

bool FGLTFMaterialUtilities::TryGetMaxTextureSize(const UMaterialInterface* Material, const FMaterialPropertyEx& Property, FIntPoint& OutMaxSize)
{
	TArray<UMaterialExpressionTextureSample*> TextureSamples;
	GetAllInputExpressionsOfType(Material, Property, TextureSamples);

	if (TextureSamples.Num() == 0)
	{
		return false;
	}

	FIntPoint MaxSize = { 0, 0 };

	for (const UMaterialExpressionTextureSample* TextureSample : TextureSamples)
	{
		const UTexture* Texture = GetTextureFromSample(Material, TextureSample);
		if (Texture == nullptr || !FGLTFTextureUtilities::Is2D(Texture))
		{
			return false;
		}

		FGLTFTextureUtilities::FullyLoad(Texture);
		FIntPoint TextureSize = FGLTFTextureUtilities::GetInGameSize(Texture);

		MaxSize = MaxSize.ComponentMax(TextureSize);
	}

	OutMaxSize = MaxSize;
	return true;
}

bool FGLTFMaterialUtilities::TryGetMaxTextureSize(const UMaterialInterface* Material, const FMaterialPropertyEx& PropertyA, const FMaterialPropertyEx& PropertyB, FIntPoint& OutMaxSize)
{
	FIntPoint MaxSizeA;
	if (!TryGetMaxTextureSize(Material, PropertyA, MaxSizeA))
	{
		return false;
	}

	FIntPoint MaxSizeB;
	if (!TryGetMaxTextureSize(Material, PropertyB, MaxSizeB))
	{
		return false;
	}

	OutMaxSize = MaxSizeA.ComponentMax(MaxSizeB);
	return true;
}

UTexture* FGLTFMaterialUtilities::GetTextureFromSample(const UMaterialInterface* Material, const UMaterialExpressionTextureSample* SampleExpression)
{
	if (const UMaterialExpressionTextureSampleParameter2D* SampleParameter = ExactCast<UMaterialExpressionTextureSampleParameter2D>(SampleExpression))
	{
		UTexture* ParameterValue = SampleParameter->Texture;

		if (!Material->GetTextureParameterValue(SampleParameter->GetParameterName(), ParameterValue))
		{
			return nullptr;
		}

		return ParameterValue;
	}

	if (const UMaterialExpressionTextureSample* Sample = ExactCast<UMaterialExpressionTextureSample>(SampleExpression))
	{
		UMaterialExpression* ObjectExpression = Sample->TextureObject.Expression;
		if (ObjectExpression == nullptr)
		{
			return Sample->Texture;
		}

		if (const UMaterialExpressionTextureObjectParameter* ObjectParameter = ExactCast<UMaterialExpressionTextureObjectParameter>(ObjectExpression))
		{
			UTexture* ParameterValue = ObjectParameter->Texture;

			if (!Material->GetTextureParameterValue(ObjectParameter->GetParameterName(), ParameterValue))
			{
				return nullptr;
			}

			return ParameterValue;
		}

		if (const UMaterialExpressionTextureObject* Object = ExactCast<UMaterialExpressionTextureObject>(ObjectExpression))
		{
			return Object->Texture;
		}

		return nullptr;
	}

	return nullptr;
}

bool FGLTFMaterialUtilities::TryGetTextureCoordinateIndex(const UMaterialExpressionTextureSample* TextureSample, int32& OutTexCoord, FGLTFJsonTextureTransform& OutTransform)
{
	const UMaterialExpression* Expression = TextureSample->Coordinates.Expression;
	if (Expression == nullptr)
	{
		OutTexCoord = TextureSample->ConstCoordinate;
		OutTransform = {};
		return true;
	}

	if (const UMaterialExpressionTextureCoordinate* TextureCoordinate = Cast<UMaterialExpressionTextureCoordinate>(Expression))
	{
		OutTexCoord = TextureCoordinate->CoordinateIndex;
		OutTransform.Offset.X = TextureCoordinate->UnMirrorU ? TextureCoordinate->UTiling * 0.5f : 0.0f;
		OutTransform.Offset.Y = TextureCoordinate->UnMirrorV ? TextureCoordinate->VTiling * 0.5f : 0.0f;
		OutTransform.Scale.X = TextureCoordinate->UTiling * (TextureCoordinate->UnMirrorU ? 0.5f : 1.0f);
		OutTransform.Scale.Y = TextureCoordinate->VTiling * (TextureCoordinate->UnMirrorV ? 0.5f : 1.0f);
		OutTransform.Rotation = 0;
		return true;
	}

	// TODO: add support for advanced expression tree (ex UMaterialExpressionTextureCoordinate -> UMaterialExpressionMultiply -> UMaterialExpressionAdd)

	return false;
}

void FGLTFMaterialUtilities::GetAllTextureCoordinateIndices(const UMaterialInterface* Material, const FMaterialPropertyEx& Property, FGLTFIndexArray& OutTexCoords)
{
	FMaterialAnalysisResult Analysis;
	AnalyzeMaterialProperty(Material, Property, Analysis);

	const TBitArray<>& TexCoords = Analysis.TextureCoordinates;
	for (int32 Index = 0; Index < TexCoords.Num(); Index++)
	{
		if (TexCoords[Index])
		{
			OutTexCoords.Add(Index);
		}
	}
}

void FGLTFMaterialUtilities::AnalyzeMaterialProperty(const UMaterialInterface* InMaterial, const FMaterialPropertyEx& InProperty, FMaterialAnalysisResult& OutAnalysis)
{
	if (GetInputForProperty(InMaterial, InProperty) == nullptr)
	{
		OutAnalysis = FMaterialAnalysisResult();
		return;
	}

	UMaterial* BaseMaterial = const_cast<UMaterial*>(InMaterial->GetMaterial());
	bool bRequiresPrimitiveData = false;

	// To extend and improve the analysis for glTF's specific use-case, we compile using FGLTFProxyMaterialCompiler
	BaseMaterial->AnalyzeMaterialCompilationInCallback([InProperty, BaseMaterial, &bRequiresPrimitiveData](FMaterialCompiler* Compiler)
	{
		FGLTFProxyMaterialCompiler ProxyCompiler(Compiler);

		if (InProperty.IsCustomOutput())
		{
			UMaterialExpressionCustomOutput* CustomOutput = GetCustomOutputByName(BaseMaterial, InProperty.CustomOutput.ToString());
			ProxyCompiler.SetMaterialProperty(MP_MAX, CustomOutput->GetShaderFrequency(0), false);
			CustomOutput->Compile(&ProxyCompiler, 0);
		}
		else
		{
			ProxyCompiler.SetMaterialProperty(InProperty.Type, SF_NumFrequencies, false);
			BaseMaterial->CompileProperty(&ProxyCompiler, InProperty.Type);
		}

		bRequiresPrimitiveData = ProxyCompiler.UsesPrimitiveData();
	}, OutAnalysis);

	// Also make sure the analysis takes into account primitive data
	OutAnalysis.bRequiresVertexData |= bRequiresPrimitiveData;
}

FMaterialShadingModelField FGLTFMaterialUtilities::EvaluateShadingModelExpression(const UMaterialInterface* Material)
{
	FMaterialAnalysisResult Analysis;
	AnalyzeMaterialProperty(Material, MP_ShadingModel, Analysis);
	return Analysis.ShadingModels;
}

template <typename ExpressionType>
void FGLTFMaterialUtilities::GetAllInputExpressionsOfType(const UMaterialInterface* Material, const FMaterialPropertyEx& Property, TArray<ExpressionType*>& OutExpressions)
{
	const FExpressionInput* Input = GetInputForProperty(Material, Property);
	if (Input == nullptr)
	{
		return;
	}

	UMaterialExpression* InputExpression = Input->Expression;
	if (InputExpression == nullptr)
	{
		return;
	}

	TArray<UMaterialExpression*> AllInputExpressions;
	InputExpression->GetAllInputExpressions(AllInputExpressions);

	for (UMaterialExpression* Expression : AllInputExpressions)
	{
		if (ExpressionType* ExpressionOfType = Cast<ExpressionType>(Expression))
		{
			OutExpressions.Add(ExpressionOfType);
		}

		if (UMaterialFunctionInterface* MaterialFunction = UMaterial::GetExpressionFunctionPointer(Expression))
		{
			MaterialFunction->GetAllExpressionsOfType<ExpressionType>(OutExpressions);
		}
		else if (TOptional<UMaterial::FLayersInterfaces> LayersInterfaces = UMaterial::GetExpressionLayers(Expression))
		{
			for (UMaterialFunctionInterface* Layer : LayersInterfaces->Layers)
			{
				if (Layer != nullptr)
				{
					Layer->GetAllExpressionsOfType<ExpressionType>(OutExpressions);
				}
			}

			for (UMaterialFunctionInterface* Blend : LayersInterfaces->Blends)
			{
				if (Blend != nullptr)
				{
					Blend->GetAllExpressionsOfType<ExpressionType>(OutExpressions);
				}
			}
		}
	}
}

#endif

EMaterialShadingModel FGLTFMaterialUtilities::GetRichestShadingModel(const FMaterialShadingModelField& ShadingModels)
{
	if (ShadingModels.HasShadingModel(MSM_ClearCoat))
	{
		return MSM_ClearCoat;
	}

	if (ShadingModels.HasShadingModel(MSM_DefaultLit))
	{
		return MSM_DefaultLit;
	}

	if (ShadingModels.HasShadingModel(MSM_Unlit))
	{
		return MSM_Unlit;
	}

	// TODO: add more shading models when conversion supported

	return ShadingModels.GetFirstShadingModel();
}

FString FGLTFMaterialUtilities::ShadingModelsToString(const FMaterialShadingModelField& ShadingModels)
{
	FString Result;

	for (uint32 Index = 0; Index < MSM_NUM; Index++)
	{
		const EMaterialShadingModel ShadingModel = static_cast<EMaterialShadingModel>(Index);
		if (ShadingModels.HasShadingModel(ShadingModel))
		{
			FString Name = FGLTFNameUtilities::GetName(ShadingModel);
			Result += Result.IsEmpty() ? Name : TEXT(", ") + Name;
		}
	}

	return Result;
}

bool FGLTFMaterialUtilities::NeedsMeshData(const UMaterialInterface* Material)
{
#if WITH_EDITOR
	if (Material != nullptr && !FGLTFProxyMaterialUtilities::IsProxyMaterial(Material))
	{
		// TODO: only analyze properties that will be needed for this specific material
		const TArray<FMaterialPropertyEx> Properties =
		{
			MP_BaseColor,
			MP_EmissiveColor,
			MP_Opacity,
			MP_OpacityMask,
			MP_Metallic,
			MP_Roughness,
			MP_Normal,
			MP_AmbientOcclusion,
			MP_CustomData0,
			MP_CustomData1,
			FMaterialPropertyEx::ClearCoatBottomNormal,
			// TODO: add TransmittanceColor when supported
			// TODO: add Refraction when supported
			// TODO: add Specular when supported
		};

		bool bNeedsMeshData = false;
		FMaterialAnalysisResult Analysis;

		// TODO: optimize baking by separating need for vertex data and primitive data

		for (const FMaterialPropertyEx& Property: Properties)
		{
			AnalyzeMaterialProperty(Material, Property, Analysis);
			bNeedsMeshData |= Analysis.bRequiresVertexData;
		}

		return bNeedsMeshData;
	}
#endif

	return false;
}

bool FGLTFMaterialUtilities::NeedsMeshData(const TArray<const UMaterialInterface*>& Materials)
{
#if WITH_EDITOR
	for (const UMaterialInterface* Material: Materials)
	{
		if (NeedsMeshData(Material))
		{
			return true;
		}
	}
#endif

	return false;
}

EMaterialShadingModel FGLTFMaterialUtilities::GetShadingModel(const UMaterialInterface* Material, FString& OutMessage)
{
	const FMaterialShadingModelField Possibilities = Material->GetShadingModels();
	const int32 PossibilitiesCount = Possibilities.CountShadingModels();

	if (PossibilitiesCount == 0)
	{
		const EMaterialShadingModel ShadingModel = MSM_DefaultLit;
		OutMessage = FString::Printf(
			TEXT("No shading model defined for material %s, will export as %s"),
			*Material->GetName(),
			*FGLTFNameUtilities::GetName(ShadingModel));
		return ShadingModel;
	}

	if (PossibilitiesCount > 1)
	{
#if WITH_EDITOR
		if (!FApp::CanEverRender())
		{
			const EMaterialShadingModel ShadingModel = FGLTFMaterialUtilities::GetRichestShadingModel(Possibilities);
			OutMessage = FString::Printf(
				TEXT("Can't evaluate shading model expression in material %s because renderer missing, will export as %s"),
				*Material->GetName(),
				*FGLTFNameUtilities::GetName(ShadingModel));
			return ShadingModel;
		}

		if (Material->IsShadingModelFromMaterialExpression())
		{
			const FMaterialShadingModelField Evaluation = FGLTFMaterialUtilities::EvaluateShadingModelExpression(Material);
			const int32 EvaluationCount = Evaluation.CountShadingModels();

			if (EvaluationCount == 0)
			{
				const EMaterialShadingModel ShadingModel = FGLTFMaterialUtilities::GetRichestShadingModel(Possibilities);
				OutMessage = FString::Printf(
					TEXT("Evaluation of shading model expression in material %s returned none, will export as %s"),
					*Material->GetName(),
					*FGLTFNameUtilities::GetName(ShadingModel));
				return ShadingModel;
			}

			if (EvaluationCount > 1)
			{
				const EMaterialShadingModel ShadingModel = FGLTFMaterialUtilities::GetRichestShadingModel(Evaluation);
				OutMessage = FString::Printf(
					TEXT("Evaluation of shading model expression in material %s is inconclusive (%s), will export as %s"),
					*Material->GetName(),
					*FGLTFMaterialUtilities::ShadingModelsToString(Evaluation),
					*FGLTFNameUtilities::GetName(ShadingModel));
				return ShadingModel;
			}

			return Evaluation.GetFirstShadingModel();
		}

		// we should never end up here
#else
		const EMaterialShadingModel ShadingModel = FGLTFMaterialUtilities::GetRichestShadingModel(Possibilities);
		OutMessage = FString::Printf(
			TEXT("Can't evaluate shading model expression in material %s without editor, will export as %s"),
			*Material->GetName(),
			*FGLTFNameUtilities::GetName(ShadingModel));
		return ShadingModel;
#endif
	}

	return Possibilities.GetFirstShadingModel();
}

//To acquire TextureAddress from glTF Imported material TilingMethod values:
TextureAddress FGLTFMaterialUtilities::ToTextureAddress(float Value)
{
	//0: wrap
	//1: clamp
	//2: mirror
	int ValueInt = FMath::RoundToInt(Value);

	switch (ValueInt)
	{
		case 0:     return TextureAddress::TA_Wrap;
		case 1:     return TextureAddress::TA_Clamp;
		case 2:	    return TextureAddress::TA_Mirror;
		default:    return TextureAddress::TA_MAX;
	}
}
float FGLTFMaterialUtilities::FromTextureWrap(EGLTFJsonTextureWrap Value)
{
	//0: wrap
	//1: clamp
	//2: mirror

	switch (Value)
	{
		case EGLTFJsonTextureWrap::Repeat:           return 0;
		case EGLTFJsonTextureWrap::ClampToEdge:      return 1;
		case EGLTFJsonTextureWrap::MirroredRepeat:   return 2;
	}

	return 1;
}

FGLTFImportMaterialMatchMakingHelper::FGLTFImportMaterialMatchMakingHelper(FGLTFConvertBuilder& InBuilder,
	const UMaterialInterface* InMaterial,
	FGLTFJsonMaterial& InJsonMaterial)
	: Builder(InBuilder)
	, JsonMaterial(InJsonMaterial)
	, Material(InMaterial)
	, bMaterialInstance(InMaterial ? InMaterial->IsA(UMaterialInstance::StaticClass()) : false)
	, bIsGLTFImportedMaterial(false)
{
#if GLTF_EXPORT_ENABLE
	if (!Material)
	{
		return;
	}

	using namespace UE::Interchange::GLTFMaterials;

	const UMaterial* TopMostParentMaterial = Material->GetMaterial();

	if (bMaterialInstance)
	{
		FString PathName = TopMostParentMaterial->GetPathName();

		TMap<FString, EShadingModel> MaterialPathsToShadingModels = GetMaterialPathsToShadingModels();

		if (MaterialPathsToShadingModels.Contains(PathName))
		{
			bIsGLTFImportedMaterial = true;

			JsonMaterial.ShadingModel = EGLTFJsonShadingModel(MaterialPathsToShadingModels[PathName]);
		}
	}
	else
	{
#if WITH_EDITOR
		//Acquire glTF Importer's MaterialFunctionPaths
		TMap<FString, EShadingModel> MaterialFunctionPathsToShadingModels = GetMaterialFunctionPathsToShadingModels();

		//Acquire the Material's expressions and check if any matches a MaterialFunctionCall which matches the glTF Importer's list:
		TConstArrayView<TObjectPtr<UMaterialExpression>> MaterialExpressions = TopMostParentMaterial->GetExpressions();

		UMaterialExpressionMaterialFunctionCall* GLTFImportedMaterialFunction = nullptr;
		EShadingModel GLTFImportedShadingModel = EShadingModel::DEFAULT;

		for (const TObjectPtr<UMaterialExpression>& MaterialExpression : MaterialExpressions)
		{
			if (UMaterialExpressionMaterialFunctionCall* FunctionCallExpression = Cast<UMaterialExpressionMaterialFunctionCall>(MaterialExpression))
			{
				if (FunctionCallExpression->MaterialFunction)
				{
					FString PathName = FunctionCallExpression->MaterialFunction->GetPathName();

					if (MaterialFunctionPathsToShadingModels.Contains(PathName))
					{
						//Material imported by glTF Importer
						GLTFImportedMaterialFunction = FunctionCallExpression;
						GLTFImportedShadingModel = MaterialFunctionPathsToShadingModels[PathName];

						break;
					}
				}
			}
		}

		if (!GLTFImportedMaterialFunction)
		{
			return;
		}

		bIsGLTFImportedMaterial = true;

		JsonMaterial.ShadingModel = EGLTFJsonShadingModel(GLTFImportedShadingModel);

		//If Exported Material is a MaterialInstance then we will use the GetParameter APIs
		if (!bMaterialInstance)
		{
			for (FExpressionInputIterator It{ GLTFImportedMaterialFunction}; It; ++It)
			{
				if (!It->IsConnected())
				{
					continue;
				}

				FString Name = It->InputName.ToString();
				UMaterialExpression* InputExpression = It->GetTracedInput().Expression;

				Inputs.Add(Name, InputExpression);
			}
		}
#endif
	}
#endif
}

bool FGLTFImportMaterialMatchMakingHelper::GetValue(const FString& InputKey, float& OutValue)
{
	if (bMaterialInstance)
	{
		if (!Material->GetScalarParameterValue(*InputKey, OutValue, true))
		{
			return false;
		}
	}
	else
	{
		if (!Inputs.Contains(InputKey))
		{
			return false;
		}

		UMaterialExpression* InputExpression = Inputs[InputKey];

		if (UMaterialExpressionConstant* ConstExpr1 = Cast<UMaterialExpressionConstant>(InputExpression))
		{
			OutValue = ConstExpr1->R;
		}
		else
		{
			return false;
		}
	}

	return true;
}

bool FGLTFImportMaterialMatchMakingHelper::GetValue(const FString& InputKey, FGLTFJsonColor3& OutValue)
{
	FLinearColor Value;

	if (bMaterialInstance)
	{
		if (!Material->GetVectorParameterValue(*InputKey, Value, true))
		{
			return false;
		}
	}
	else
	{
		if (!Inputs.Contains(InputKey))
		{
			return false;
		}

		UMaterialExpression* InputExpression = Inputs[InputKey];

		if (UMaterialExpressionConstant3Vector* ConstExpr3 = Cast<UMaterialExpressionConstant3Vector>(InputExpression))
		{
			Value = ConstExpr3->Constant;
		}
		else
		{
			return false;
		}
	}

	OutValue = FGLTFCoreUtilities::ConvertColor3(Value);

	return true;
}

bool FGLTFImportMaterialMatchMakingHelper::GetValue(const FString& InputKey, FGLTFJsonColor4& OutValue, bool HandleAsColor)
{
#if GLTF_EXPORT_ENABLE
	using namespace UE::Interchange::GLTFMaterials::Inputs::PostFix;

	FLinearColor Value;

	if (bMaterialInstance)
	{
		if (!Material->GetVectorParameterValue(*InputKey, Value, true))
		{
			return false;
		}
	}
	else
	{
		if (!Inputs.Contains(InputKey))
		{
			if (HandleAsColor)
			{
				float Alpha = 1;
				FGLTFJsonColor3 RGB = FGLTFJsonColor3::White;
				if (GetValue(InputKey + Color_RGB, RGB) || GetValue(InputKey + Color_A, Alpha))
				{
					OutValue = { RGB.R, RGB.G, RGB.B, FMath::Clamp(Alpha, 0.0f, 1.0f) };
					return true;
				}
			}

			return false;
		}

		UMaterialExpression* InputExpression = Inputs[InputKey];


		if (UMaterialExpressionConstant4Vector* ConstExpr4 = Cast<UMaterialExpressionConstant4Vector>(InputExpression))
		{
			Value = ConstExpr4->Constant;
		}
		else
		{
			return false;
		}
	}

	if (HandleAsColor)
	{
		OutValue = FGLTFCoreUtilities::ConvertColor(Value);
	}
	else
	{
		OutValue = { Value.R, Value.G, Value.B, Value.A };
	}

	return true;
#else
	return false;
#endif
}

bool FGLTFImportMaterialMatchMakingHelper::GetValue(const FString& InputKey, FGLTFJsonTextureInfo& OutValue)
{
#if GLTF_EXPORT_ENABLE
	using namespace UE::Interchange::GLTFMaterials::Inputs;

	UTexture* Texture = nullptr;

	if (bMaterialInstance)
	{
		if (!Material->GetTextureParameterValue(*InputKey, Texture, true))
		{
			return false;
		}
	}
	else
	{

		if (!Inputs.Contains(InputKey))
		{
			return false;
		}

		UMaterialExpression* InputExpression = Inputs[InputKey];

		if (UMaterialExpressionTextureObject* TextureObject = Cast<UMaterialExpressionTextureObject>(InputExpression))
		{
			if (TextureObject->Texture)
			{
				Texture = TextureObject->Texture.Get();
			}
		}
		else
		{
			return false;
		}
	}

	if (!Texture)
	{
		return false;
	}

	TextureAddress TextureAddressX = TextureAddress::TA_Wrap;
	TextureAddress TextureAddressY = TextureAddress::TA_Wrap;

	FGLTFJsonColor4 TilingMethod = FGLTFColor4{ 0.f, 0.f, 0.f, 0.f };
	if (GetValue(InputKey + PostFix::TilingMethod, TilingMethod, false))
	{
		TextureAddressX = FGLTFMaterialUtilities::ToTextureAddress(TilingMethod.R);
		TextureAddressY = FGLTFMaterialUtilities::ToTextureAddress(TilingMethod.G);
	}

	const bool bSRGB = InputKey == BaseColorTexture || InputKey == EmissiveTexture;
	OutValue.Index = Builder.AddUniqueTexture(Texture, bSRGB, TextureAddressX, TextureAddressY);

	float UVIndex;
	if (GetValue(InputKey + PostFix::TexCoord, UVIndex))
	{
		OutValue.TexCoord = FMath::RoundToInt(UVIndex);
	}

	FGLTFJsonColor4 UVOffsetScale = FGLTFColor4{ 0.f, 0.f, 1.f, 1.f };
	if (GetValue(InputKey + PostFix::OffsetScale, UVOffsetScale, false))
	{
		OutValue.Transform.Offset.X = UVOffsetScale.R;
		OutValue.Transform.Offset.Y = UVOffsetScale.G;

		OutValue.Transform.Scale.X = UVOffsetScale.B;
		OutValue.Transform.Scale.Y = UVOffsetScale.A;
	}
	else
	{
		if (GetValue(InputKey + PostFix::OffsetX, UVOffsetScale.R))
		{
			OutValue.Transform.Offset.X = UVOffsetScale.R;
		}
		if (GetValue(InputKey + PostFix::OffsetY, UVOffsetScale.G))
		{
			OutValue.Transform.Offset.Y = UVOffsetScale.G;
		}

		if (GetValue(InputKey + PostFix::ScaleX, UVOffsetScale.B))
		{
			OutValue.Transform.Scale.X = UVOffsetScale.B;
		}
		if (GetValue(InputKey + PostFix::ScaleY, UVOffsetScale.A))
		{
			OutValue.Transform.Scale.Y = UVOffsetScale.A;
		}
	}

	float UVRotation;
	if (GetValue(InputKey + PostFix::Rotation, UVRotation))
	{
		OutValue.Transform.Rotation = UVRotation;
	}

	if (!Builder.ExportOptions->bExportTextureTransforms && !OutValue.Transform.IsExactlyDefault())
	{
		Builder.LogWarning(FString::Printf(
			TEXT("Texture coordinates [%d] in %s for material %s are transformed, but texture transform is disabled by export options"),
			OutValue.TexCoord,
			*InputKey,
			*Material->GetName()));
		OutValue.Transform = {};
	}

	return true;
#else
	return false;
#endif
}

void FGLTFImportMaterialMatchMakingHelper::Process()
{
#if GLTF_EXPORT_ENABLE
	using namespace UE::Interchange::GLTFMaterials;

	GetValue(Inputs::AlphaCutoff, JsonMaterial.AlphaCutoff);

	if (JsonMaterial.ShadingModel == EGLTFJsonShadingModel::SpecularGlossiness)
	{
		GetValue(Inputs::DiffuseTexture, JsonMaterial.PBRSpecularGlossiness.DiffuseTexture);
		GetValue(Inputs::DiffuseFactor, JsonMaterial.PBRSpecularGlossiness.DiffuseFactor, true);
	}
	else
	{
		GetValue(Inputs::BaseColorTexture, JsonMaterial.PBRMetallicRoughness.BaseColorTexture);
		GetValue(Inputs::BaseColorFactor, JsonMaterial.PBRMetallicRoughness.BaseColorFactor, true);
	}

	if (JsonMaterial.ShadingModel != EGLTFJsonShadingModel::Unlit)
	{
		if (JsonMaterial.ShadingModel != EGLTFJsonShadingModel::SpecularGlossiness)
		{
			GetValue(Inputs::MetallicRoughnessTexture, JsonMaterial.PBRMetallicRoughness.MetallicRoughnessTexture);
			GetValue(Inputs::MetallicFactor, JsonMaterial.PBRMetallicRoughness.MetallicFactor);
			GetValue(Inputs::RoughnessFactor, JsonMaterial.PBRMetallicRoughness.RoughnessFactor);

			GetValue(Inputs::SpecularFactor, JsonMaterial.Specular.Factor);
			GetValue(Inputs::SpecularTexture, JsonMaterial.Specular.Texture);

			GetValue(Inputs::IOR, JsonMaterial.IOR.Value);

			//Iridescence
			GetValue(Inputs::IridescenceFactor, JsonMaterial.Iridescence.IridescenceFactor);
			GetValue(Inputs::IridescenceTexture, JsonMaterial.Iridescence.IridescenceTexture);
			GetValue(Inputs::IridescenceIOR, JsonMaterial.Iridescence.IridescenceIOR);
			GetValue(Inputs::IridescenceThicknessMinimum, JsonMaterial.Iridescence.IridescenceThicknessMinimum);
			GetValue(Inputs::IridescenceThicknessMaximum, JsonMaterial.Iridescence.IridescenceThicknessMaximum);
			GetValue(Inputs::IridescenceThicknessTexture, JsonMaterial.Iridescence.IridescenceThicknessTexture);

			//Anisotropy
			GetValue(Inputs::AnisotropyStrength, JsonMaterial.Anisotropy.AnisotropyStrength);
			GetValue(Inputs::AnisotropyRotation, JsonMaterial.Anisotropy.AnisotropyRotation);
			GetValue(Inputs::AnisotropyTexture, JsonMaterial.Anisotropy.AnisotropyTexture);
		}

		//Importer does not support Emissive for Transmission at the moment
		GetValue(Inputs::EmissiveTexture, JsonMaterial.EmissiveTexture);
		GetValue(Inputs::EmissiveFactor, JsonMaterial.EmissiveFactor);
		GetValue(Inputs::EmissiveStrength, JsonMaterial.EmissiveStrength);

		GetValue(Inputs::NormalTexture, JsonMaterial.NormalTexture);
		GetValue(Inputs::NormalScale, JsonMaterial.NormalTexture.Scale);

		GetValue(Inputs::OcclusionTexture, JsonMaterial.OcclusionTexture);
		GetValue(Inputs::OcclusionStrength, JsonMaterial.OcclusionTexture.Strength);

		if (JsonMaterial.ShadingModel == EGLTFJsonShadingModel::ClearCoat)
		{
			GetValue(Inputs::ClearCoatTexture, JsonMaterial.ClearCoat.ClearCoatTexture);
			GetValue(Inputs::ClearCoatFactor, JsonMaterial.ClearCoat.ClearCoatFactor);

			GetValue(Inputs::ClearCoatRoughnessTexture, JsonMaterial.ClearCoat.ClearCoatRoughnessTexture);
			GetValue(Inputs::ClearCoatRoughnessFactor, JsonMaterial.ClearCoat.ClearCoatRoughnessFactor);

			GetValue(Inputs::ClearCoatNormalTexture, JsonMaterial.ClearCoat.ClearCoatNormalTexture);
			GetValue(Inputs::ClearCoatNormalScale, JsonMaterial.ClearCoat.ClearCoatNormalTexture.Scale);
		}
		else if (JsonMaterial.ShadingModel == EGLTFJsonShadingModel::Sheen)
		{
			GetValue(Inputs::SheenColorFactor, JsonMaterial.Sheen.ColorFactor);
			GetValue(Inputs::SheenColorTexture, JsonMaterial.Sheen.ColorTexture);
			GetValue(Inputs::SheenRoughnessFactor, JsonMaterial.Sheen.RoughnessFactor);
			GetValue(Inputs::SheenRoughnessTexture, JsonMaterial.Sheen.RoughnessTexture);
		}
		else if (JsonMaterial.ShadingModel == EGLTFJsonShadingModel::Transmission)
		{
			GetValue(Inputs::TransmissionFactor, JsonMaterial.Transmission.Factor);
			GetValue(Inputs::TransmissionTexture, JsonMaterial.Transmission.Texture);

			float AlphaMode;
			if (GetValue(Inputs::AlphaMode, AlphaMode))
			{
				JsonMaterial.AlphaMode = EGLTFJsonAlphaMode(FMath::RoundToInt(AlphaMode));
			}
		}
		else if (JsonMaterial.ShadingModel == EGLTFJsonShadingModel::SpecularGlossiness)
		{
			GetValue(Inputs::SpecFactor, JsonMaterial.PBRSpecularGlossiness.SpecularFactor);
			GetValue(Inputs::GlossinessFactor, JsonMaterial.PBRSpecularGlossiness.GlossinessFactor);
			GetValue(Inputs::SpecularGlossinessTexture, JsonMaterial.PBRSpecularGlossiness.SpecularGlossinessTexture);
		}
	}
#endif
}