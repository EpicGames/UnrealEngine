// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Builders/GLTFContainerBuilder.h"
#include "Converters/GLTFSharedArray.h"
#include "MaterialExpressionIO.h"

struct FMaterialAnalysisResult;
struct FMaterialPropertyEx;
class UMaterialExpressionTextureSample;
class UMaterialInstanceConstant;
class UMaterialInterface;
class UMaterialExpressionCustomOutput;

struct FGLTFPropertyBakeOutput
{
	FGLTFPropertyBakeOutput(const TGLTFSharedArray<FColor>& Pixels, FIntPoint Size, float EmissiveScale, bool bSRGB)
		: Pixels(Pixels)
		, Size(Size)
		, EmissiveScale(EmissiveScale)
		, bIsConstant(Pixels->Num() == 1)
	{
		if (bIsConstant)
		{
			const FColor& Pixel = (*Pixels)[0];
			ConstantValue = bSRGB ? FLinearColor(Pixel) : Pixel.ReinterpretAsLinear();
		}
	}

	TGLTFSharedArray<FColor> Pixels;
	FIntPoint Size;
	float EmissiveScale;
	bool bIsConstant;
	FLinearColor ConstantValue;
};

struct FGLTFMaterialUtilities
{
	static UMaterialInterface* GetDefaultMaterial();

	static bool IsClearCoatBottomNormalEnabled();

	static EMaterialShadingModel GetRichestShadingModel(const FMaterialShadingModelField& ShadingModels);
	static FString ShadingModelsToString(const FMaterialShadingModelField& ShadingModels);

	static bool NeedsMeshData(const UMaterialInterface* Material);
	static bool NeedsMeshData(const TArray<const UMaterialInterface*>& Materials);

	static EMaterialShadingModel GetShadingModel(const UMaterialInterface* Material, FString& OutMessage);

	static TextureAddress ToTextureAddress(float Value);
	static float FromTextureWrap(EGLTFJsonTextureWrap Value);

#if WITH_EDITOR
	static bool IsNormalMap(const FMaterialPropertyEx& Property);
	static bool IsSRGB(const FMaterialPropertyEx& Property);

	static FGuid GetAttributeID(const FMaterialPropertyEx& Property);
	static FGuid GetAttributeIDChecked(const FMaterialPropertyEx& Property);

	static FVector4f GetPropertyDefaultValue(const FMaterialPropertyEx& Property);
	static FVector4f GetPropertyMask(const FMaterialPropertyEx& Property);

	static const FExpressionInput* GetInputForProperty(const UMaterialInterface* Material, const FMaterialPropertyEx& Property);

	template <class InputType>
	static const FMaterialInput<InputType>* GetInputForProperty(const UMaterialInterface* Material, const FMaterialPropertyEx& Property)
	{
		const FExpressionInput* ExpressionInput = GetInputForProperty(Material, Property);
		return static_cast<const FMaterialInput<InputType>*>(ExpressionInput);
	}

	static UMaterialExpressionCustomOutput* GetCustomOutputByName(const UMaterialInterface* Material, const FString& FunctionName);

	static FGLTFPropertyBakeOutput BakeMaterialProperty(const FIntPoint& OutputSize, const FMaterialPropertyEx& Property, const UMaterialInterface* Material, const FBox2f& TexCoordBounds, int32 TexCoordIndex, const FGLTFMeshData* MeshData, const FGLTFIndexArray& MeshSectionIndices, bool bFillAlpha, bool bAdjustNormalmaps);

	static FGLTFJsonTexture* AddTexture(FGLTFConvertBuilder& Builder, TGLTFSharedArray<FColor>& Pixels, const FIntPoint& TextureSize, bool bIgnoreAlpha, bool bIsNormalMap, const FString& TextureName, TextureAddress TextureAddress, TextureFilter TextureFilter);

	static FLinearColor GetMask(const FExpressionInput& ExpressionInput);
	static uint32 GetMaskComponentCount(const FExpressionInput& ExpressionInput);

	static bool TryGetMaxTextureSize(const UMaterialInterface* Material, const FMaterialPropertyEx& Property, FIntPoint& OutMaxSize);
	static bool TryGetMaxTextureSize(const UMaterialInterface* Material, const FMaterialPropertyEx& PropertyA, const FMaterialPropertyEx& PropertyB, FIntPoint& OutMaxSize);

	static UTexture* GetTextureFromSample(const UMaterialInterface* Material, const UMaterialExpressionTextureSample* SampleExpression);

	static bool TryGetTextureCoordinateIndex(const UMaterialExpressionTextureSample* TextureSample, int32& OutTexCoord, FGLTFJsonTextureTransform& OutTransform);
	static void GetAllTextureCoordinateIndices(const UMaterialInterface* Material, const FMaterialPropertyEx& Property, FGLTFIndexArray& OutTexCoords);

	static void AnalyzeMaterialProperty(const UMaterialInterface* Material, const FMaterialPropertyEx& InProperty, FMaterialAnalysisResult& OutAnalysis);

	static FMaterialShadingModelField EvaluateShadingModelExpression(const UMaterialInterface* Material);
private:

	template<typename ExpressionType>
	static void GetAllInputExpressionsOfType(const UMaterialInterface* Material, const FMaterialPropertyEx& Property, TArray<ExpressionType*>& OutExpressions);
#endif
};

//Helper used to identify Interchange - glTF Importer created materials.
// And then helps to acquire the Values set by the glTF Importer
struct FGLTFImportMaterialMatchMakingHelper
{
	TMap<FString, UMaterialExpression*> Inputs;
	FGLTFConvertBuilder& Builder;
	FGLTFJsonMaterial& JsonMaterial;
	const UMaterialInterface* Material;
	bool bMaterialInstance; //uses Parameter acquisition approach if true, otherwise uses Inputs
	bool bIsGLTFImportedMaterial;

	FGLTFImportMaterialMatchMakingHelper(FGLTFConvertBuilder& InBuilder,
		const UMaterialInterface* InMaterial,
		FGLTFJsonMaterial& InJsonMaterial);

	bool GetValue(const FString& InputKey, float& OutValue);
	bool GetValue(const FString& InputKey, FGLTFJsonColor3& OutValue);
	bool GetValue(const FString& InputKey, FGLTFJsonColor4& OutValue, bool HandleAsColor);
	bool GetValue(const FString& InputKey, FGLTFJsonTextureInfo& OutValue);

	void Process();
};