// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFMaterialVariantConverters.h"
#include "Converters/GLTFVariantUtilities.h"
#include "Builders/GLTFContainerBuilder.h"
#include "Components/MeshComponent.h"
#include "VariantObjectBinding.h"
#include "PropertyValueMaterial.h"
#include "PropertyValue.h"
#include "Variant.h"

FGLTFJsonMaterialVariant* FGLTFMaterialVariantConverter::Convert(const UVariant* Variant)
{
	if (Variant == nullptr || Builder.ExportOptions->ExportMaterialVariants == EGLTFMaterialVariantMode::None)
	{
		return nullptr;
	}

	typedef TTuple<FGLTFJsonPrimitive*, FGLTFJsonMaterial*> FGLTFPrimitiveMaterial;
	TArray<TTuple<FGLTFJsonPrimitive*, FGLTFJsonMaterial*>> PrimitiveMaterials;

	for (const UVariantObjectBinding* Binding: Variant->GetBindings())
	{
		for (const UPropertyValue* Property: Binding->GetCapturedProperties())
		{
			if (!const_cast<UPropertyValue*>(Property)->Resolve() || !Property->HasRecordedData())
			{
				continue;
			}

			if (const UPropertyValueMaterial* MaterialProperty = Cast<UPropertyValueMaterial>(Property))
			{
				FGLTFJsonPrimitive* Primitive = nullptr;
				FGLTFJsonMaterial* JsonMaterial;

				if (TryParseMaterialProperty(Primitive, JsonMaterial, MaterialProperty))
				{
					PrimitiveMaterials.Add(MakeTuple(Primitive, JsonMaterial));
				}
			}
		}
	}

	if (PrimitiveMaterials.Num() < 1)
	{
		// TODO: add warning and / or allow unused material variants to be added?

		return nullptr;
	}

	FGLTFJsonMaterialVariant* MaterialVariant = Builder.AddMaterialVariant();
	// TODO: add warning if the variant name is not unique, i.e it's already used?
	// While material variants are technically allowed to use the same name, it may
	// cause confusion when trying to select the correct variant in a viewer.
	MaterialVariant->Name = Variant->GetDisplayText().ToString();

	for (const FGLTFPrimitiveMaterial& PrimitiveMaterial: PrimitiveMaterials)
	{
		FGLTFJsonPrimitive* Primitive = PrimitiveMaterial.Key;
		FGLTFJsonMaterial* Material = PrimitiveMaterial.Value;

		FGLTFJsonMaterialVariantMapping* ExistingMapping = Primitive->MaterialVariantMappings.FindByPredicate(
			[Material](const FGLTFJsonMaterialVariantMapping& Mapping)
			{
				return Mapping.Material == Material;
			});

		if (ExistingMapping != nullptr)
		{
			ExistingMapping->Variants.AddUnique(MaterialVariant);
		}
		else
		{
			FGLTFJsonMaterialVariantMapping Mapping;
			Mapping.Material = Material;
			Mapping.Variants.Add(MaterialVariant);

			Primitive->MaterialVariantMappings.Add(Mapping);
		}
	}

	return MaterialVariant;
}

bool FGLTFMaterialVariantConverter::TryParseMaterialProperty(FGLTFJsonPrimitive*& OutPrimitive, FGLTFJsonMaterial*& OutMaterial, const UPropertyValueMaterial* Property) const
{
	const UMeshComponent* Target = static_cast<UMeshComponent*>(Property->GetPropertyParentContainerAddress());
	if (Target == nullptr)
	{
		Builder.LogWarning(FString::Printf(
			TEXT("Variant property %s must belong to a mesh component, the property will be skipped"),
			*FGLTFVariantUtilities::GetLogContext(Property)));
		return false;
	}

	const AActor* Owner = Target->GetOwner();
	if (Owner == nullptr)
	{
		Builder.LogWarning(FString::Printf(
			TEXT("Variant property %s must belong to an actor, the property will be skipped"),
			*FGLTFVariantUtilities::GetLogContext(Property)));
		return false;
	}

	if (!Builder.IsSelectedActor(Owner))
	{
		Builder.LogWarning(FString::Printf(
			TEXT("Variant property %s doesn't belong to an actor selected for export, the property will be skipped"),
			*FGLTFVariantUtilities::GetLogContext(Property)));
		return false;
	}

	const TArray<FCapturedPropSegment>& CapturedPropSegments = Property->GetCapturedPropSegments();
	const int32 NumPropSegments = CapturedPropSegments.Num();

	if (NumPropSegments < 1)
	{
		Builder.LogWarning(FString::Printf(
			TEXT("Failed to parse material index for variant property %s, the property will be skipped"),
			*FGLTFVariantUtilities::GetLogContext(Property)));
		return false;
	}

	// NOTE: UPropertyValueMaterial::GetMaterial does *not* ensure that the recorded data has been loaded,
	// so we need to call UProperty::GetRecordedData first to make that happen.
	const_cast<UPropertyValueMaterial*>(Property)->GetRecordedData();

	const UMaterialInterface* Material = const_cast<UPropertyValueMaterial*>(Property)->GetMaterial();
	if (Material == nullptr)
	{
		// TODO: find way to determine whether the material is null because "None" was selected, or because it failed to resolve

		Builder.LogWarning(FString::Printf(
			TEXT("No material assigned, the property will be skipped. Context: %s"),
			*FGLTFVariantUtilities::GetLogContext(Property)));
		return false;
	}

	Builder.RegisterObjectVariant(Target, Property);
	Builder.RegisterObjectVariant(Owner, Property); // TODO: we don't need to register this on the actor

	const int32 MaterialIndex = CapturedPropSegments[NumPropSegments - 1].PropertyIndex;
	FGLTFJsonMesh* JsonMesh = Builder.AddUniqueMesh(Target);

	OutPrimitive = &JsonMesh->Primitives[MaterialIndex];
	OutMaterial = FGLTFVariantUtilities::AddUniqueMaterial(Builder, Material, Target, MaterialIndex);

	return true;
}
