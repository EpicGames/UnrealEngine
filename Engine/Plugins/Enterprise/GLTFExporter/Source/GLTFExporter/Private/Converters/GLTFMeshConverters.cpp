// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFMeshConverters.h"
#include "Converters/GLTFMeshUtilities.h"
#include "Converters/GLTFMaterialUtilities.h"
#include "Builders/GLTFConvertBuilder.h"
#include "Tasks/GLTFDelayedMeshTasks.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/SplineMeshComponent.h"
#include "LandscapeComponent.h"
#include "Utilities/MeshParsingUtilities.h"

void FGLTFStaticMeshConverter::Sanitize(const UStaticMesh*& StaticMesh, const UStaticMeshComponent*& StaticMeshComponent, FGLTFMaterialArray& Materials, int32& LODIndex)
{
	FGLTFMeshUtilities::ResolveMaterials(Materials, StaticMeshComponent, StaticMesh);
	Builder.ResolveProxies(Materials);

	LODIndex = Builder.SanitizeLOD(StaticMesh, StaticMeshComponent, LODIndex);

	if (StaticMeshComponent != nullptr)
	{
		const bool bUsesMeshData = Builder.ExportOptions->BakeMaterialInputs == EGLTFMaterialBakeMode::UseMeshData &&
			FGLTFMaterialUtilities::NeedsMeshData(Materials); // TODO: if this expensive, cache the results for each material

		const bool bIsReferencedByVariant = Builder.GetObjectVariants(StaticMeshComponent) != nullptr;

		// Only use the component if it's needed for baking or variants, since we would
		// otherwise export a copy of this mesh for each mesh-component.
		if (!bUsesMeshData && !bIsReferencedByVariant)
		{
			StaticMeshComponent = nullptr;
		}
	}
}

FGLTFJsonMesh* FGLTFStaticMeshConverter::Convert(const UStaticMesh* StaticMesh,  const UStaticMeshComponent* StaticMeshComponent, FGLTFMaterialArray Materials, int32 LODIndex)
{
#if !WITH_EDITOR
	if (StaticMesh && !StaticMesh->bAllowCPUAccess)
	{
		Builder.LogSuggestion(FString::Printf(
			TEXT("Export of mesh %s can in runtime be speed-up by checking 'Allow CPU Access' in asset settings"),
			*StaticMesh->GetName()));
	}
#endif

	int32 MaterialCount = 0;
	if (StaticMesh)
	{
#if WITH_EDITORONLY_DATA
		if (Builder.ExportOptions->bExportSourceModel)
		{
			MaterialCount = UE::MeshParser::Private::GetNumberOfValidPolygons(StaticMesh, LODIndex);
		}
		else
#endif
		{
			MaterialCount = FGLTFMeshUtilities::GetMaterials(StaticMesh).Num();
		}
	}

	FGLTFJsonMesh* JsonMesh = Builder.AddMesh();
	JsonMesh->Primitives.AddDefaulted(MaterialCount);

	Builder.ScheduleSlowTask<FGLTFDelayedStaticMeshTask>(Builder, MeshSectionConverter, StaticMesh, StaticMeshComponent, Materials, LODIndex, JsonMesh);
	return JsonMesh;
}

void FGLTFSkeletalMeshConverter::Sanitize(const USkeletalMesh*& SkeletalMesh, const USkeletalMeshComponent*& SkeletalMeshComponent, FGLTFMaterialArray& Materials, int32& LODIndex)
{
	FGLTFMeshUtilities::ResolveMaterials(Materials, SkeletalMeshComponent, SkeletalMesh);
	Builder.ResolveProxies(Materials);

	LODIndex = Builder.SanitizeLOD(SkeletalMesh, SkeletalMeshComponent, LODIndex);

	if (SkeletalMeshComponent != nullptr)
	{
		const bool bUsesMeshData = Builder.ExportOptions->BakeMaterialInputs == EGLTFMaterialBakeMode::UseMeshData &&
			FGLTFMaterialUtilities::NeedsMeshData(Materials); // TODO: if this expensive, cache the results for each material

		const bool bIsReferencedByVariant = Builder.GetObjectVariants(SkeletalMeshComponent) != nullptr;

		// Only use the component if it's needed for baking or variants, since we would
		// otherwise export a copy of this mesh for each mesh-component.
		if (!bUsesMeshData && !bIsReferencedByVariant)
		{
			SkeletalMeshComponent = nullptr;
		}
	}
}

FGLTFJsonMesh* FGLTFSkeletalMeshConverter::Convert(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent, FGLTFMaterialArray Materials, int32 LODIndex)
{
#if !WITH_EDITOR
	if (SkeletalMesh && !SkeletalMesh->GetLODInfo(LODIndex)->bAllowCPUAccess)
	{
		Builder.LogSuggestion(FString::Printf(
			TEXT("Export of mesh %s (LOD %d) can in runtime be speed-up by checking 'Allow CPU Access' in asset settings"),
			*SkeletalMesh->GetName(),
			LODIndex));
	}
#endif

	int32 MaterialCount = 0;
	if (SkeletalMesh)
	{
#if WITH_EDITORONLY_DATA
		if (Builder.ExportOptions->bExportSourceModel)
		{
			MaterialCount = UE::MeshParser::Private::GetNumberOfValidPolygons(SkeletalMesh, LODIndex);
		}
		else
#endif
		{
			MaterialCount = FGLTFMeshUtilities::GetMaterials(SkeletalMesh).Num();
		}
	}

	FGLTFJsonMesh* JsonMesh = Builder.AddMesh();
	JsonMesh->Primitives.AddDefaulted(MaterialCount);

	Builder.ScheduleSlowTask<FGLTFDelayedSkeletalMeshTask>(Builder, MeshSectionConverter, SkeletalMesh, SkeletalMeshComponent, Materials, LODIndex, JsonMesh);
	return JsonMesh;
}


void FGLTFSplineMeshConverter::Sanitize(const UStaticMesh*& StaticMesh, const USplineMeshComponent*& SplineMeshComponent, FGLTFMaterialArray& Materials, int32& LODIndex)
{
	//SplineMeshComponent is used for calculating the SplineTransform, hence we cannot null it out. (Contrary to FGLTFStaticMeshConverter::Sanitize and FGLTFSkeletalMeshConverter::Sanitize)

	FGLTFMeshUtilities::ResolveMaterials(Materials, SplineMeshComponent, StaticMesh);
	Builder.ResolveProxies(Materials);

	LODIndex = Builder.SanitizeLOD(StaticMesh, SplineMeshComponent, LODIndex);
}

FGLTFJsonMesh* FGLTFSplineMeshConverter::Convert(const UStaticMesh* StaticMesh, const USplineMeshComponent* SplineMeshComponent, FGLTFMaterialArray Materials, int32 LODIndex)
{
#if !WITH_EDITOR
	if (StaticMesh && !StaticMesh->bAllowCPUAccess)
	{
		Builder.LogSuggestion(FString::Printf(
			TEXT("Export of mesh %s can in runtime be speed-up by checking 'Allow CPU Access' in asset settings"),
			*StaticMesh->GetName()));
	}
#endif

	int32 MaterialCount = 0;
	if (StaticMesh)
	{
#if WITH_EDITORONLY_DATA
		if (Builder.ExportOptions->bExportSourceModel)
		{
			MaterialCount = UE::MeshParser::Private::GetNumberOfValidPolygons(StaticMesh, LODIndex);
		}
		else
#endif
		{
			MaterialCount = FGLTFMeshUtilities::GetMaterials(StaticMesh).Num();
		}
	}

	FGLTFJsonMesh* JsonMesh = Builder.AddMesh();
	JsonMesh->Primitives.AddDefaulted(MaterialCount);

	if (StaticMesh && SplineMeshComponent)
	{
		Builder.ScheduleSlowTask<FGLTFDelayedSplineMeshTask>(Builder, /*PositionBuffer,*/ MeshSectionConverter, StaticMesh, SplineMeshComponent, Materials, LODIndex, JsonMesh);
	}
	
	return JsonMesh;
}

void FGLTFLandscapeMeshConverter::Sanitize(const ULandscapeComponent*& LandscapeComponent, const UMaterialInterface*& LandscapeMaterial)
{
	FGLTFMeshUtilities::ResolveMaterials(LandscapeMaterial, LandscapeComponent);
	UGLTFMaterialExportOptions::ResolveProxy(LandscapeMaterial);
}

FGLTFJsonMesh* FGLTFLandscapeMeshConverter::Convert(const ULandscapeComponent* LandscapeComponent, const UMaterialInterface* LandscapeMaterial)
{
	const int32 MaterialCount = 1; //Sanitize will resolve it to either the LandscapeMaterial or the DefaultMaterial
	FGLTFJsonMesh* JsonMesh = Builder.AddMesh();
	JsonMesh->Primitives.AddDefaulted(MaterialCount);

	if (LandscapeComponent && LandscapeMaterial)
	{
		Builder.ScheduleSlowTask<FGLTFDelayedLandscapeTask>(Builder, *LandscapeComponent, JsonMesh, *LandscapeMaterial);
	}
	
	return JsonMesh;
}