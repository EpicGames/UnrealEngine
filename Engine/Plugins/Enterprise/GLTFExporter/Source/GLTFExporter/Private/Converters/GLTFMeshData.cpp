// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFMeshData.h"
#include "Converters/GLTFMeshUtilities.h"
#include "Converters/GLTFNameUtilities.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#if WITH_EDITOR
#include "Editor.h"
#include "Engine/MapBuildDataRegistry.h"
#include "StaticMeshComponentLODInfo.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "StaticMeshAttributes.h"
#include "IMeshMergeUtilities.h"
#include "MeshMergeModule.h"
#include "Modules/ModuleManager.h"
#endif

FGLTFMeshData::FGLTFMeshData(const UStaticMesh* StaticMesh, const UStaticMeshComponent* StaticMeshComponent, int32 LODIndex)
	: Parent(nullptr)
	, LODIndex(LODIndex)
#if WITH_EDITOR
	, LightMap(nullptr)
	, LightMapResourceCluster(nullptr)
	, LightMapTexCoord(0)
#endif
{
#if WITH_EDITOR
	const IMeshMergeUtilities& MeshMergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();
	FStaticMeshAttributes(Description).Register();
#endif

	if (StaticMeshComponent != nullptr)
	{
		Name = FGLTFNameUtilities::GetName(StaticMeshComponent);

#if WITH_EDITOR
		PrimitiveData = { StaticMeshComponent };

		//MeshMergeUtilities.RetrieveMeshDescription (for StaticMeshComponent) uses bApplyComponentTransform==true, 
		// however MeshData->Description is only used for DegenerateVertices/Triangles and UV checks, and material baking.
		MeshMergeUtilities.RetrieveMeshDescription(StaticMeshComponent, LODIndex, Description, true);

		constexpr int32 LightMapLODIndex = 0; // TODO: why is this zero?
		if (StaticMeshComponent->LODData.IsValidIndex(LightMapLODIndex))
		{
			const FStaticMeshComponentLODInfo& LODData = StaticMeshComponent->LODData[LightMapLODIndex];
			const FMeshMapBuildData* BuildData = StaticMeshComponent->GetMeshMapBuildData(LODData);
			if (BuildData != nullptr)
			{
				LightMap = BuildData->LightMap;
				LightMapResourceCluster = BuildData->ResourceCluster;
			}
		}
#endif
	}
	else
	{
		StaticMesh->GetName(Name);

#if WITH_EDITOR
		PrimitiveData = { StaticMesh };
		MeshMergeUtilities.RetrieveMeshDescription(StaticMesh, LODIndex, Description);
#endif
	}

#if WITH_EDITOR
	LightMapTexCoord = StaticMesh->GetLightMapCoordinateIndex();
 	const int32 NumTexCoords = FGLTFMeshUtilities::GetRenderData(StaticMesh, LODIndex).VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
	BakeUsingTexCoord = FMath::Min(LightMapTexCoord, NumTexCoords - 1);
	// TODO: add warning if texture coordinate for baking has overlap
#endif
}

FGLTFMeshData::FGLTFMeshData(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent, int32 LODIndex)
	: Parent(nullptr)
	, LODIndex(LODIndex)
#if WITH_EDITOR
	, LightMap(nullptr)
	, LightMapResourceCluster(nullptr)
	, LightMapTexCoord(0)
#endif
{
#if WITH_EDITOR
	const IMeshMergeUtilities& MeshMergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();
	FStaticMeshAttributes(Description).Register();
#endif

	if (SkeletalMeshComponent != nullptr)
	{
		Name = FGLTFNameUtilities::GetName(SkeletalMeshComponent);

#if WITH_EDITOR
		PrimitiveData = { SkeletalMeshComponent };
		MeshMergeUtilities.RetrieveMeshDescription(SkeletalMeshComponent, LODIndex, Description, true);
#endif
	}
	else
	{
		SkeletalMesh->GetName(Name);

#if WITH_EDITOR
		PrimitiveData = { SkeletalMesh };

		// NOTE: this is a workaround for the fact that there's no overload for FMeshMergeHelpers::RetrieveMesh
		// that accepts a USkeletalMesh, only a USkeletalMeshComponent.
		// Writing a custom utility function that would work on a "standalone" skeletal mesh is problematic
		// since we would need to implement an equivalent of USkinnedMeshComponent::GetCPUSkinnedVertices too.

		if (UWorld* World = GEditor->GetEditorWorldContext().World())
		{
			FActorSpawnParameters SpawnParams;
			SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
			SpawnParams.ObjectFlags |= RF_Transient;
			SpawnParams.bAllowDuringConstructionScript = true;

			if (AActor* TempActor = World->SpawnActor<AActor>(SpawnParams))
			{
				USkeletalMeshComponent* TempComponent = NewObject<USkeletalMeshComponent>(TempActor, TEXT(""), RF_Transient);
				TempComponent->RegisterComponent();
				TempComponent->SetSkeletalMesh(const_cast<USkeletalMesh*>(SkeletalMesh));

				MeshMergeUtilities.RetrieveMeshDescription(TempComponent, LODIndex, Description, true);

				World->DestroyActor(TempActor, false, false);
			}
		}
#endif
	}

#if WITH_EDITOR
	// TODO: don't assume last UV channel is non-overlapping
	const int32 NumTexCoords = FGLTFMeshUtilities::GetRenderData(SkeletalMesh, LODIndex).StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
	BakeUsingTexCoord = NumTexCoords - 1;
	// TODO: add warning if texture coordinate for baking has overlap
#endif
}

const FGLTFMeshData* FGLTFMeshData::GetParent() const
{
	return Parent != nullptr ? Parent : this;
}
