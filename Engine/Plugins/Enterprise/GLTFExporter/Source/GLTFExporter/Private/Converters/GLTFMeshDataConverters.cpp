// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFMeshDataConverters.h"
#include "Builders/GLTFConvertBuilder.h"

template class TGLTFMeshDataConverter<UStaticMesh, UStaticMeshComponent>;
template class TGLTFMeshDataConverter<USkeletalMesh, USkeletalMeshComponent>;

template <typename MeshType, typename MeshComponentType>
void TGLTFMeshDataConverter<MeshType, MeshComponentType>::Sanitize(const MeshType*& Mesh, const MeshComponentType*& MeshComponent, int32& LODIndex)
{
	LODIndex = Builder.SanitizeLOD(Mesh, MeshComponent, LODIndex);
}

template <typename MeshType, typename MeshComponentType>
const FGLTFMeshData* TGLTFMeshDataConverter<MeshType, MeshComponentType>::Convert(const MeshType* Mesh, const MeshComponentType* MeshComponent, int32 LODIndex)
{
	TUniquePtr<FGLTFMeshData> Output = MakeUnique<FGLTFMeshData>(Mesh, MeshComponent, LODIndex);

	if (MeshComponent != nullptr)
	{
		Output->Parent = this->GetOrAdd(Mesh, nullptr, LODIndex);
	}

	return Outputs.Add_GetRef(MoveTemp(Output)).Get();
}
