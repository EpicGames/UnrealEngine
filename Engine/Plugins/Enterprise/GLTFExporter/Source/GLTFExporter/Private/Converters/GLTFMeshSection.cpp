// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFMeshSection.h"
#include "Converters/GLTFBufferAdapter.h"
#include "Converters/GLTFMeshUtilities.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/MultiSizeIndexContainer.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Algo/MaxElement.h"

FGLTFMeshSection::FGLTFMeshSection(const UStaticMesh* StaticMesh, int32 LODIndex, const FGLTFIndexArray& SectionIndices)
	: SectionIndices(SectionIndices)
	, MaxBoneIndex(0)
{
	StaticMesh->GetName(Name);

	const FStaticMeshLODResources& RenderData = FGLTFMeshUtilities::GetRenderData(StaticMesh, LODIndex);
	const TUniquePtr<IGLTFBufferAdapter> SourceBuffer = IGLTFBufferAdapter::GetIndices(&RenderData.IndexBuffer);
	const uint8* SourceData = SourceBuffer->GetData();

	if (RenderData.IndexBuffer.Is32Bit())
	{
		Init(RenderData.Sections, reinterpret_cast<const uint32*>(SourceData));
	}
	else
	{
		Init(RenderData.Sections, reinterpret_cast<const uint16*>(SourceData));
	}
}

FGLTFMeshSection::FGLTFMeshSection(const USkeletalMesh* SkeletalMesh, int32 LODIndex, const FGLTFIndexArray& SectionIndices)
	: SectionIndices(SectionIndices)
	, MaxBoneIndex(0)
{
	SkeletalMesh->GetName(Name);

	const FSkeletalMeshLODRenderData& RenderData = FGLTFMeshUtilities::GetRenderData(SkeletalMesh, LODIndex);
	const TUniquePtr<IGLTFBufferAdapter> SourceBuffer = IGLTFBufferAdapter::GetIndices(RenderData.MultiSizeIndexContainer.GetIndexBuffer());
	const uint8* SourceData = SourceBuffer->GetData();

	if (RenderData.MultiSizeIndexContainer.GetDataTypeSize() != sizeof(uint16))
	{
		Init(RenderData.RenderSections, reinterpret_cast<const uint32*>(SourceData));
	}
	else
	{
		Init(RenderData.RenderSections, reinterpret_cast<const uint16*>(SourceData));
	}
}

template <typename IndexType, typename SectionArrayType>
void FGLTFMeshSection::Init(const SectionArrayType& Sections, const IndexType* SourceData)
{
	if (SourceData == nullptr)
	{
		// TODO: report error
		return;
	}

	uint32 TotalIndexCount = 0;
	for (const int32 SectionIndex : SectionIndices)
	{
		TotalIndexCount += Sections[SectionIndex].NumTriangles * 3;
	}

	IndexMap.Reserve(TotalIndexCount);
	IndexBuffer.AddUninitialized(TotalIndexCount);
	BoneMapLookup.Reserve(TotalIndexCount);

	TMap<uint32, uint32> IndexLookup;

	for (const int32 SectionIndex : SectionIndices)
	{
		const typename SectionArrayType::ElementType& Section = Sections[SectionIndex];
		const TArray<FBoneIndexType>& BoneMap = GetBoneMap(Section);

		const uint32 IndexOffset = GetIndexOffset(Section);
		const uint32 IndexCount = Section.NumTriangles * 3;
		const uint32 BoneMapIndex = BoneMaps.Num();

		for (uint32 Index = 0; Index < IndexCount; Index++)
		{
			const uint32 OldIndex = SourceData[IndexOffset + Index];
			uint32 NewIndex;

			if (const uint32* FoundIndex = IndexLookup.Find(OldIndex))
			{
				NewIndex = *FoundIndex;
			}
			else
			{
				NewIndex = IndexMap.Num();
				IndexLookup.Add(OldIndex, NewIndex);
				IndexMap.Add(OldIndex);

				if (BoneMap.Num() > 0)
				{
					BoneMapLookup.Add(BoneMapIndex);
				}
			}

			IndexBuffer[Index] = NewIndex;
		}

		if (BoneMap.Num() > 0)
		{
			BoneMaps.Add(BoneMap);

			if (const FBoneIndexType* MaxSectionBoneIndex = Algo::MaxElement(BoneMap))
			{
				MaxBoneIndex = FMath::Max(*MaxSectionBoneIndex, MaxBoneIndex);
			}
		}
	}
}

FString FGLTFMeshSection::ToString() const
{
	return FString::Printf(
		TEXT("%s (section%s %s)"),
		*Name,
		SectionIndices.Num() > 1 ? TEXT("s") : TEXT(""),
		*FString::JoinBy(SectionIndices, TEXT(", "), FString::FromInt)
		);
}

uint32 FGLTFMeshSection::GetIndexOffset(const FStaticMeshSection& Section)
{
	return Section.FirstIndex;
}

uint32 FGLTFMeshSection::GetIndexOffset(const FSkelMeshRenderSection& Section)
{
	return Section.BaseIndex;
}

const TArray<FBoneIndexType>& FGLTFMeshSection::GetBoneMap(const FStaticMeshSection& Section)
{
	static const TArray<FBoneIndexType> BoneMap = {};
	return BoneMap;
}

const TArray<FBoneIndexType>& FGLTFMeshSection::GetBoneMap(const FSkelMeshRenderSection& Section)
{
	return Section.BoneMap;
}
