// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFMeshUtilities.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "StaticMeshCompiler.h"
#include "SkinnedAssetCompiler.h"
#include "LandscapeComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Converters/GLTFMeshAttributesArray.h"

void FGLTFMeshUtilities::FullyLoad(const UStaticMesh* InStaticMesh)
{
	UStaticMesh* StaticMesh = const_cast<UStaticMesh*>(InStaticMesh);

#if WITH_EDITOR
	FStaticMeshCompilingManager::Get().FinishCompilation({ StaticMesh });
#endif

	StaticMesh->SetForceMipLevelsToBeResident(30.0f);
	StaticMesh->WaitForStreaming();
}

void FGLTFMeshUtilities::FullyLoad(const USkeletalMesh* InSkeletalMesh)
{
	USkeletalMesh* SkeletalMesh = const_cast<USkeletalMesh*>(InSkeletalMesh);

#if WITH_EDITOR
	FSkinnedAssetCompilingManager::Get().FinishCompilation({ SkeletalMesh });
#endif

	SkeletalMesh->SetForceMipLevelsToBeResident(30.0f);
	SkeletalMesh->WaitForStreaming();
}

const UMaterialInterface* FGLTFMeshUtilities::GetDefaultMaterial()
{
	static UMaterial* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
	return DefaultMaterial;
}

const TArray<FStaticMaterial>& FGLTFMeshUtilities::GetMaterials(const UStaticMesh* StaticMesh)
{
	return StaticMesh->GetStaticMaterials();
}

const TArray<FSkeletalMaterial>& FGLTFMeshUtilities::GetMaterials(const USkeletalMesh* SkeletalMesh)
{
	return SkeletalMesh->GetMaterials();
}

const UMaterialInterface* FGLTFMeshUtilities::GetMaterial(const UMaterialInterface* Material)
{
	return Material;
}

const UMaterialInterface* FGLTFMeshUtilities::GetMaterial(const FStaticMaterial& Material)
{
	return Material.MaterialInterface;
}

const UMaterialInterface* FGLTFMeshUtilities::GetMaterial(const FSkeletalMaterial& Material)
{
	return Material.MaterialInterface;
}

void FGLTFMeshUtilities::ResolveMaterials(TArray<const UMaterialInterface*>& Materials, const UStaticMeshComponent* StaticMeshComponent, const UStaticMesh* StaticMesh)
{
	if (StaticMeshComponent != nullptr)
	{
		ResolveMaterials(Materials, StaticMeshComponent->GetMaterials());
	}

	if (StaticMesh != nullptr)
	{
		ResolveMaterials(Materials, GetMaterials(StaticMesh));
	}

	ResolveMaterials(Materials, GetDefaultMaterial());
}

void FGLTFMeshUtilities::ResolveMaterials(TArray<const UMaterialInterface*>& Materials, const USkeletalMeshComponent* SkeletalMeshComponent, const USkeletalMesh* SkeletalMesh)
{
	if (SkeletalMeshComponent != nullptr)
	{
		ResolveMaterials(Materials, SkeletalMeshComponent->GetMaterials());
	}

	if (SkeletalMesh != nullptr)
	{
		ResolveMaterials(Materials, GetMaterials(SkeletalMesh));
	}

	ResolveMaterials(Materials, GetDefaultMaterial());
}

void FGLTFMeshUtilities::ResolveMaterials(const UMaterialInterface*& LandscapeMaterial, const ULandscapeComponent* LandscapeComponent)
{
	LandscapeMaterial = LandscapeComponent->GetLandscapeMaterial();

	if (LandscapeMaterial == nullptr)
	{
		LandscapeMaterial = GetDefaultMaterial();
	}
}

template <typename MaterialType>
void FGLTFMeshUtilities::ResolveMaterials(TArray<const UMaterialInterface*>& Materials, const TArray<MaterialType>& Defaults)
{
	const int32 Count = Defaults.Num();
	Materials.SetNumZeroed(Count);

	for (int32 Index = 0; Index < Count; ++Index)
	{
		const UMaterialInterface*& Material = Materials[Index];
		if (Material == nullptr)
		{
			Material = GetMaterial(Defaults[Index]);
		}
	}
}

void FGLTFMeshUtilities::ResolveMaterials(TArray<const UMaterialInterface*>& Materials, const UMaterialInterface* Default)
{
	for (const UMaterialInterface*& Material : Materials)
	{
		if (Material == nullptr)
		{
			Material = Default;
		}
	}
}

const FStaticMeshLODResources& FGLTFMeshUtilities::GetRenderData(const UStaticMesh* StaticMesh, int32 LODIndex)
{
	return StaticMesh->GetLODForExport(LODIndex);
}

const FSkeletalMeshLODRenderData& FGLTFMeshUtilities::GetRenderData(const USkeletalMesh* SkeletalMesh, int32 LODIndex)
{
	return SkeletalMesh->GetResourceForRendering()->LODRenderData[LODIndex];
}

FGLTFIndexArray FGLTFMeshUtilities::GetSectionIndices(const UStaticMesh* StaticMesh, int32 LODIndex, int32 MaterialIndex)
{
	if (StaticMesh == nullptr)
	{
		return {};
	}

	return GetSectionIndices(GetRenderData(StaticMesh, LODIndex), MaterialIndex);
}

FGLTFIndexArray FGLTFMeshUtilities::GetSectionIndices(const USkeletalMesh* SkeletalMesh, int32 LODIndex, int32 MaterialIndex)
{
	if (SkeletalMesh == nullptr)
	{
		return {};
	}

	return GetSectionIndices(GetRenderData(SkeletalMesh, LODIndex), MaterialIndex);
}

FGLTFIndexArray FGLTFMeshUtilities::GetSectionIndices(const FStaticMeshLODResources& RenderData, int32 MaterialIndex)
{
	const FStaticMeshSectionArray& Sections = RenderData.Sections;

	FGLTFIndexArray SectionIndices;
	SectionIndices.Reserve(Sections.Num());

	for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
	{
		if (Sections[SectionIndex].MaterialIndex == MaterialIndex)
		{
			SectionIndices.Add(SectionIndex);
		}
	}

	return SectionIndices;
}

FGLTFIndexArray FGLTFMeshUtilities::GetSectionIndices(const FSkeletalMeshLODRenderData& RenderData, int32 MaterialIndex)
{
	const TArray<FSkelMeshRenderSection>& Sections = RenderData.RenderSections;

	FGLTFIndexArray SectionIndices;
	SectionIndices.Reserve(Sections.Num());

	for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
	{
		if (Sections[SectionIndex].MaterialIndex == MaterialIndex)
		{
			SectionIndices.Add(SectionIndex);
		}
	}

	return SectionIndices;
}

int32 FGLTFMeshUtilities::GetLOD(const UStaticMesh* StaticMesh, const UStaticMeshComponent* StaticMeshComponent, int32 DefaultLOD)
{
	const int32 ForcedLOD = StaticMeshComponent != nullptr ? StaticMeshComponent->ForcedLodModel - 1 : -1;
	const int32 LOD = ForcedLOD > 0 ? ForcedLOD : FMath::Max(DefaultLOD, GetMinimumLOD(StaticMesh, StaticMeshComponent));
	return FMath::Min(LOD, GetMaximumLOD(StaticMesh));
}

int32 FGLTFMeshUtilities::GetLOD(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent, int32 DefaultLOD)
{
	const int32 ForcedLOD = SkeletalMeshComponent != nullptr ? SkeletalMeshComponent->GetForcedLOD() - 1 : -1;
	const int32 LOD = ForcedLOD > 0 ? ForcedLOD : FMath::Max(DefaultLOD, GetMinimumLOD(SkeletalMesh, SkeletalMeshComponent));
	return FMath::Min(LOD, GetMaximumLOD(SkeletalMesh));
}

int32 FGLTFMeshUtilities::GetLOD(const UStaticMesh* StaticMesh, const USplineMeshComponent* SplineMeshComponent, int32 DefaultLOD)
{
	const int32 ForcedLOD = SplineMeshComponent != nullptr ? SplineMeshComponent->ForcedLodModel - 1 : -1;
	const int32 LOD = ForcedLOD > 0 ? ForcedLOD : FMath::Max(DefaultLOD, GetMinimumLOD(StaticMesh, SplineMeshComponent));
	return FMath::Min(LOD, GetMaximumLOD(StaticMesh));
}

int32 FGLTFMeshUtilities::GetMaximumLOD(const UStaticMesh* StaticMesh)
{
	return StaticMesh != nullptr ? StaticMesh->GetNumLODs() - 1 : -1;
}

int32 FGLTFMeshUtilities::GetMaximumLOD(const USkeletalMesh* SkeletalMesh)
{
	if (SkeletalMesh != nullptr)
	{
		const FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
		if (RenderData != nullptr)
		{
			return RenderData->LODRenderData.Num() - 1;
		}
	}

	return -1;
}

int32 FGLTFMeshUtilities::GetMinimumLOD(const UStaticMesh* StaticMesh, const UStaticMeshComponent* StaticMeshComponent)
{
	if (StaticMeshComponent != nullptr && StaticMeshComponent->bOverrideMinLOD)
	{
		return StaticMeshComponent->MinLOD;
	}

	if (StaticMesh != nullptr)
	{
		return StaticMesh->GetMinLOD().Default;
	}

	return -1;
}

int32 FGLTFMeshUtilities::GetMinimumLOD(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent)
{
	if (SkeletalMeshComponent != nullptr && SkeletalMeshComponent->bOverrideMinLod)
	{
		return SkeletalMeshComponent->MinLodModel;
	}

	if (SkeletalMesh != nullptr)
	{
		return SkeletalMesh->GetMinLod().Default;
	}

	return -1;
}
