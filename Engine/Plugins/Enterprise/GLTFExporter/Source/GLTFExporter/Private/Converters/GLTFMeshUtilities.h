// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Converters/GLTFMeshAttributesArray.h"

class UMaterialInterface;
class UStaticMesh;
class UStaticMeshComponent;
class USkeletalMesh;
class USkeletalMeshComponent;
class FSkeletalMeshLODRenderData;
class ULandscapeComponent;
struct FStaticMaterial;
struct FSkeletalMaterial;
struct FStaticMeshLODResources;
class USplineMeshComponent;

struct FGLTFMeshUtilities
{
	static void FullyLoad(const UStaticMesh* InStaticMesh);
	static void FullyLoad(const USkeletalMesh* InSkeletalMesh);

	static const UMaterialInterface* GetDefaultMaterial();

	static const TArray<FStaticMaterial>& GetMaterials(const UStaticMesh* StaticMesh);
	static const TArray<FSkeletalMaterial>& GetMaterials(const USkeletalMesh* SkeletalMesh);

	static const UMaterialInterface* GetMaterial(const UMaterialInterface* Material);
	static const UMaterialInterface* GetMaterial(const FStaticMaterial& Material);
	static const UMaterialInterface* GetMaterial(const FSkeletalMaterial& Material);

	static void ResolveMaterials(TArray<const UMaterialInterface*>& Materials, const UStaticMeshComponent* StaticMeshComponent, const UStaticMesh* StaticMesh);
	static void ResolveMaterials(TArray<const UMaterialInterface*>& Materials, const USkeletalMeshComponent* SkeletalMeshComponent, const USkeletalMesh* SkeletalMesh);
	static void ResolveMaterials(const UMaterialInterface*& LandscapeMaterial, const ULandscapeComponent* LandscapeComponent);

	template <typename MaterialType>
	static void ResolveMaterials(TArray<const UMaterialInterface*>& Materials, const TArray<MaterialType>& Defaults);
	static void ResolveMaterials(TArray<const UMaterialInterface*>& Materials, const UMaterialInterface* Default);

	static const FStaticMeshLODResources& GetRenderData(const UStaticMesh* StaticMesh, int32 LODIndex);
	static const FSkeletalMeshLODRenderData& GetRenderData(const USkeletalMesh* SkeletalMesh, int32 LODIndex);

	static FGLTFIndexArray GetSectionIndices(const UStaticMesh* StaticMesh, int32 LODIndex, int32 MaterialIndex);
	static FGLTFIndexArray GetSectionIndices(const USkeletalMesh* SkeletalMesh, int32 LODIndex, int32 MaterialIndex);

	static FGLTFIndexArray GetSectionIndices(const FStaticMeshLODResources& RenderData, int32 MaterialIndex);
	static FGLTFIndexArray GetSectionIndices(const FSkeletalMeshLODRenderData& RenderData, int32 MaterialIndex);

	static int32 GetLOD(const UStaticMesh* StaticMesh, const UStaticMeshComponent* StaticMeshComponent, int32 DefaultLOD);
	static int32 GetLOD(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent, int32 DefaultLOD);
	static int32 GetLOD(const UStaticMesh* StaticMesh, const USplineMeshComponent* SplineMeshComponent, int32 DefaultLOD);

	static int32 GetMaximumLOD(const UStaticMesh* StaticMesh);
	static int32 GetMaximumLOD(const USkeletalMesh* SkeletalMesh);

	static int32 GetMinimumLOD(const UStaticMesh* StaticMesh, const UStaticMeshComponent* StaticMeshComponent);
	static int32 GetMinimumLOD(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent);
};
