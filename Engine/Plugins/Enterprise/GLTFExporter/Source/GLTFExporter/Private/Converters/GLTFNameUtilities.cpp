// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFNameUtilities.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"

FString FGLTFNameUtilities::GetName(const UEnum* Enum, int32 Value)
{
	check(Enum != nullptr);
	const FString DisplayName = Enum->GetDisplayNameTextByValue(Value).ToString();
	return DisplayName.IsEmpty() ? FString::FromInt(Value) : DisplayName;
}

FString FGLTFNameUtilities::GetName(const USceneComponent* Component)
{
	if (const AActor* Owner = Component->GetOwner())
	{
		if (Component == Owner->GetRootComponent())
		{
#if WITH_EDITOR
			return Owner->GetActorLabel();
#else
			return Owner->GetName();
#endif
		}
	}

	return Component->GetName();
}
