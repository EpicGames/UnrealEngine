// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ReflectedTypeAccessors.h"

class USceneComponent;

struct FGLTFNameUtilities
{
	template <typename EnumType, typename = typename TEnableIf<TIsEnum<EnumType>::Value>::Type>
	static FString GetName(EnumType Value)
	{
		return GetName(StaticEnum<EnumType>(), static_cast<int32>(Value));
	}

	static FString GetName(const UEnum* Enum, int32 Value);

	static FString GetName(const USceneComponent* Component);
};
