// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFNodeConverters.h"
#include "Converters/GLTFNameUtilities.h"
#include "Builders/GLTFContainerBuilder.h"
#include "Utilities/GLTFCoreUtilities.h"
#include "LevelSequenceActor.h"
#include "Camera/CameraComponent.h"
#include "Components/LightComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshSocket.h"
#include "Engine/SkeletalMeshSocket.h"
#include "GameFramework/Pawn.h"

#include "LandscapeStreamingProxy.h"
#include "LandscapeComponent.h"
#include "GLTFMeshUtilities.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/SplineMeshComponent.h"
#include "LevelInstance/LevelInstanceComponent.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "LevelInstance/LevelInstanceEditorInstanceActor.h"
#include "Engine/Level.h"

FGLTFJsonNode* FGLTFActorConverter::Convert(const AActor* Actor)
{
	if (!Actor || !Actor->IsValidLowLevel() || Actor->HasAnyFlags(RF_Transient) || Actor->IsEditorOnly() || !Builder.IsSelectedActor(Actor))
	{
		return nullptr;
	}
#if WITH_EDITOR
	if (const ALevelInstanceEditorInstanceActor* LevelInstanceEditorInstanceActor = Cast<ALevelInstanceEditorInstanceActor>(Actor))
	{
		//Editor only class, reporting as IsEditorOnly==false
		return nullptr;
	}
#endif

	if (const ALandscapeStreamingProxy* LevelInstanceEditorInstanceActor = Cast<ALandscapeStreamingProxy>(Actor))
	{
		//The actual object that it is proxied to will be iterated through as well. To avoid duplications we won't include the Proxy.
		return nullptr;
	}
	

	const USceneComponent* RootComponent = Actor->GetRootComponent();
	FGLTFJsonNode* RootNode = Builder.AddUniqueNode(RootComponent);

	// TODO: process all components since any component can be attached to any other component in runtime

	if (const ALevelSequenceActor* LevelSequenceActor = Cast<ALevelSequenceActor>(Actor))
	{
		if (Builder.ExportOptions->bExportLevelSequences)
		{
			Builder.AddUniqueAnimation(LevelSequenceActor);
		}
	}
	else
	{
		// TODO: add support for exporting brush geometry?

		// TODO: to reduce number of nodes, only export components that are of interest
		for (const UActorComponent* Component : Actor->GetComponents())
		{
			const USceneComponent* SceneComponent = Cast<USceneComponent>(Component);
			if (SceneComponent != nullptr)
			{
				Builder.AddUniqueNode(SceneComponent);
			}
		}
	}

	return RootNode;
}

FGLTFJsonNode* FGLTFComponentConverter::Convert(const USceneComponent* SceneComponent)
{
	if (!SceneComponent || SceneComponent->IsEditorOnly())
	{
		return nullptr;
	}

	const AActor* Owner = SceneComponent->GetOwner();
	if (Owner == nullptr)
	{
		// TODO: report error (invalid scene component)
		return nullptr;
	}

#if WITH_EDITOR
	if (const ALevelInstanceEditorInstanceActor* LevelInstanceEditorInstanceActor = Cast<ALevelInstanceEditorInstanceActor>(Owner))
	{
		//Editor only class, reporting as IsEditorOnly==false
		return nullptr;
	}
#endif

	if (const ALandscapeStreamingProxy* LevelInstanceEditorInstanceActor = Cast<ALandscapeStreamingProxy>(Owner))
	{
		//The actual object that it is proxied to will be iterated through as well. To avoid duplications we won't include the Proxy.
		return nullptr;
	}

	if (!Builder.IsSelectedActor(Owner))
	{
		return nullptr;
	}
	
	const bool bIsRootComponent = Owner->GetRootComponent() == SceneComponent;
	const bool bIsRootNode = bIsRootComponent && Builder.IsRootActor(Owner);
	const USceneComponent* ParentComponent = !bIsRootNode ? SceneComponent->GetAttachParent() : nullptr;

	const FName SocketName = SceneComponent->GetAttachSocketName();
	FGLTFJsonNode* ParentNode = Builder.AddUniqueNode(ParentComponent, SocketName);

	if (ParentComponent != nullptr && !SceneComponent->IsUsingAbsoluteScale())
	{
		const FVector ParentScale = ParentComponent->GetComponentScale();
		if (!ParentScale.IsUniform())
		{
			Builder.LogWarning(
				FString::Printf(TEXT("Non-uniform parent scale (%s) for component %s (in actor %s) may be represented differently in glTF"),
				*ParentScale.ToString(),
				*SceneComponent->GetName(),
				*Owner->GetName()));
		}
	}

	const FTransform3f Transform = FTransform3f(SceneComponent->GetComponentTransform());
	const FTransform3f ParentTransform = ParentComponent != nullptr ? FTransform3f(ParentComponent->GetSocketTransform(SocketName)) : FTransform3f::Identity;
	const FTransform3f RelativeTransform = bIsRootNode ? Transform : Transform.GetRelativeTransform(ParentTransform);

	FGLTFJsonNode* Node = Builder.AddNode();
	Node->Name = FGLTFNameUtilities::GetName(SceneComponent);
	Node->Translation = FGLTFCoreUtilities::ConvertPosition(RelativeTransform.GetTranslation(), Builder.ExportOptions->ExportUniformScale);
	Node->Rotation = FGLTFCoreUtilities::ConvertRotation(RelativeTransform.GetRotation());
	Node->Scale = FGLTFCoreUtilities::ConvertScale(RelativeTransform.GetScale3D());

	if (ParentNode != nullptr)
	{
		ParentNode->Children.Add(Node);
	}

	ConvertComponentSpecialization(SceneComponent, Owner, Node);
	return Node;
}

void FGLTFComponentConverter::ConvertComponentSpecialization(const USceneComponent* SceneComponent, const AActor* Owner, FGLTFJsonNode* Node)
{
	// TODO: don't export invisible components unless visibility is variable due to variant sets

	if (!Builder.ExportOptions->bExportHiddenInGame && (SceneComponent->bHiddenInGame || Owner->IsHidden()))
	{
		return;
	}

	if (Owner->IsA<ALevelSequenceActor>() || Owner->IsA<APawn>())
	{
		return;
	}
	
    if (const UInstancedStaticMeshComponent* InstancedMeshComp = Cast<UInstancedStaticMeshComponent>(SceneComponent))
	{
		const UStaticMesh* StaticMesh = InstancedMeshComp->GetStaticMesh();

		FGLTFJsonMesh* Mesh = Builder.AddUniqueMesh(StaticMesh);

		const int32 NumInstances = InstancedMeshComp->GetInstanceCount();
		for (int32 InstanceIndex = 0; InstanceIndex < NumInstances; ++InstanceIndex)
		{
			FTransform RelativeTransform;
			if (ensure(InstancedMeshComp->GetInstanceTransform(InstanceIndex, RelativeTransform, /*bWorldSpace=*/false)))
			{
				FGLTFJsonNode* InstanceNode = Builder.AddNode();
				InstanceNode->Mesh = Mesh;
				InstanceNode->Name = FString::Printf(TEXT("%s:%d"), *(InstancedMeshComp->GetName()), InstanceIndex);
				const FTransform3f Transform = FTransform3f(RelativeTransform);

				InstanceNode->Translation = FGLTFCoreUtilities::ConvertPosition(Transform.GetTranslation(), Builder.ExportOptions->ExportUniformScale);
				InstanceNode->Rotation = FGLTFCoreUtilities::ConvertRotation(Transform.GetRotation());
				InstanceNode->Scale = FGLTFCoreUtilities::ConvertScale(Transform.GetScale3D());
				Node->Children.Add(InstanceNode);
			}
		}

		if (Builder.ExportOptions->bExportLightmaps)
		{
			Node->LightMap = Builder.AddUniqueLightMap(InstancedMeshComp);
		}
	}
	else if (const USplineMeshComponent* SplineMeshComponent = Cast<USplineMeshComponent>(SceneComponent))
	{
		FGLTFJsonNode* SplineNode = Builder.AddNode();
		SplineNode->Name = SplineMeshComponent->GetName();
		SplineNode->Mesh = Builder.AddUniqueMesh(SplineMeshComponent);

		const FTransform3f Transform;
		SplineNode->Translation = FGLTFCoreUtilities::ConvertPosition(Transform.GetTranslation(), Builder.ExportOptions->ExportUniformScale);
		SplineNode->Rotation = FGLTFCoreUtilities::ConvertRotation(Transform.GetRotation());
		SplineNode->Scale = FGLTFCoreUtilities::ConvertScale(Transform.GetScale3D());

		Node->Children.Add(SplineNode);
	}
	else if (const ULandscapeComponent* LandscapeComponent = Cast<ULandscapeComponent>(SceneComponent))
	{
		FGLTFJsonNode* LandscapeNode = Builder.AddNode();
		LandscapeNode->Name = LandscapeComponent->GetName();
		LandscapeNode->Mesh = Builder.AddUniqueMesh(LandscapeComponent);
		
		const FTransform3f Transform;
		LandscapeNode->Translation = FGLTFCoreUtilities::ConvertPosition(Transform.GetTranslation(), Builder.ExportOptions->ExportUniformScale);
		LandscapeNode->Rotation = FGLTFCoreUtilities::ConvertRotation(Transform.GetRotation());
		LandscapeNode->Scale = FGLTFCoreUtilities::ConvertScale(Transform.GetScale3D());
		
		Node->Children.Add(LandscapeNode);
	}
	else if (const ULevelInstanceComponent* LevelInstanceComponent = Cast<ULevelInstanceComponent>(SceneComponent))
	{
		const ALevelInstance* LevelInstance = Cast<ALevelInstance>(Owner);

		TArray<TObjectPtr<AActor>> Actors;
		if (const ULevel* LoadedLevel = LevelInstance->GetLoadedLevel())
		{
			Actors = LoadedLevel->Actors;
		}

		for (const AActor* LevelActor : Actors)
		{
			FGLTFJsonNode* LevelActorNode = Builder.AddUniqueNode(LevelActor);

			if (LevelActorNode == nullptr)
			{
				continue;
			}
			const USceneComponent* LevelActorComponent = LevelActor->GetRootComponent();
			const USceneComponent* ParentComponent = SceneComponent;

			const FName SocketName = LevelActorComponent->GetAttachSocketName();
			const FTransform3f Transform = FTransform3f(LevelActorComponent->GetComponentTransform());
			const FTransform3f ParentTransform = ParentComponent != nullptr ? FTransform3f(ParentComponent->GetSocketTransform(SocketName)) : FTransform3f::Identity;
			const FTransform3f RelativeTransform = Transform.GetRelativeTransform(ParentTransform);

			LevelActorNode->Translation = FGLTFCoreUtilities::ConvertPosition(RelativeTransform.GetTranslation(), Builder.ExportOptions->ExportUniformScale);
			LevelActorNode->Rotation = FGLTFCoreUtilities::ConvertRotation(RelativeTransform.GetRotation());
			LevelActorNode->Scale = FGLTFCoreUtilities::ConvertScale(RelativeTransform.GetScale3D());

			if (LevelActorNode != nullptr)
			{
				Node->Children.Add(LevelActorNode);
			}
		}
	}
	else if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(SceneComponent))
	{
		Node->Mesh = Builder.AddUniqueMesh(StaticMeshComponent);

		if (Builder.ExportOptions->bExportLightmaps)
		{
			Node->LightMap = Builder.AddUniqueLightMap(StaticMeshComponent);
		}
	}
	else if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(SceneComponent))
	{
		Node->Mesh = Builder.AddUniqueMesh(SkeletalMeshComponent);

		if (Builder.ExportOptions->bExportVertexSkinWeights)
		{
			Node->Skin = Builder.AddUniqueSkin(Node, SkeletalMeshComponent);
			if (Node->Skin != nullptr)
			{
				if (Builder.ExportOptions->bExportAnimationSequences)
				{
					Builder.AddUniqueAnimation(Node, SkeletalMeshComponent);
				}
			}
		}
	}
	else if (const UCameraComponent* CameraComponent = Cast<UCameraComponent>(SceneComponent))
	{
		if (Builder.ExportOptions->bExportCameras)
		{
			// TODO: conversion of camera direction should be done in separate converter
			FGLTFJsonNode* CameraNode = Builder.AddNode();
			CameraNode->Name = FGLTFNameUtilities::GetName(CameraComponent);
			CameraNode->Rotation = FGLTFCoreUtilities::GetLocalCameraRotation();
			CameraNode->Camera = Builder.AddUniqueCamera(CameraComponent);
			Node->Children.Add(CameraNode);
		}
	}
	else if (const ULightComponent* LightComponent = Cast<ULightComponent>(SceneComponent))
	{
		if (Builder.ExportOptions->bExportLights)
		{
			// TODO: conversion of light direction should be done in separate converter
			FGLTFJsonNode* LightNode = Builder.AddNode();
			LightNode->Name = TEXT("LightNode_") + FGLTFNameUtilities::GetName(LightComponent);
			LightNode->Rotation = FGLTFCoreUtilities::GetLocalLightRotation();

			LightNode->Light = Builder.AddUniqueLight(LightComponent);

			if (LightComponent->IESTexture)
			{
				LightNode->LightIESInstance = Builder.AddUniqueLightIESInstance(LightComponent);
			}

			Node->Children.Add(LightNode);
		}
	}
}

FGLTFJsonNode* FGLTFComponentSocketConverter::Convert(const USceneComponent* SceneComponent, FName SocketName)
{
	FGLTFJsonNode* JsonNode = Builder.AddUniqueNode(SceneComponent);

	if (SocketName != NAME_None)
	{
		if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(SceneComponent))
		{
			const UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
			return Builder.AddUniqueNode(JsonNode, StaticMesh, SocketName);
		}

		if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(SceneComponent))
		{
			// TODO: add support for SocketOverrideLookup?
			const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
			return Builder.AddUniqueNode(JsonNode, SkeletalMesh, SocketName);
		}

		// TODO: add support for more socket types

		Builder.LogWarning(
			FString::Printf(TEXT("Can't export socket %s because it belongs to an unsupported mesh component %s"),
			*SocketName.ToString(),
			*SceneComponent->GetName()));
	}

	return JsonNode;
}

FGLTFJsonNode* FGLTFStaticSocketConverter::Convert(FGLTFJsonNode* RootNode, const UStaticMesh* StaticMesh, FName SocketName)
{
	const UStaticMeshSocket* Socket = StaticMesh->FindSocket(SocketName);
	if (Socket == nullptr)
	{
		// TODO: report error
		return nullptr;
	}

	FGLTFJsonNode* Node = Builder.AddNode();
	Node->Name = SocketName.ToString();

	// TODO: add warning check for non-uniform scaling
	Node->Translation = FGLTFCoreUtilities::ConvertPosition(FVector3f(Socket->RelativeLocation), Builder.ExportOptions->ExportUniformScale);
	Node->Rotation = FGLTFCoreUtilities::ConvertRotation(FRotator3f(Socket->RelativeRotation).Quaternion());
	Node->Scale = FGLTFCoreUtilities::ConvertScale(FVector3f(Socket->RelativeScale));

	RootNode->Children.Add(Node);
	return Node;
}

FGLTFJsonNode* FGLTFSkeletalSocketConverter::Convert(FGLTFJsonNode* RootNode, const USkeletalMesh* SkeletalMesh, FName SocketName)
{
	const FReferenceSkeleton& ReferenceSkeleton = SkeletalMesh->GetRefSkeleton();

	const USkeletalMeshSocket* Socket = SkeletalMesh->FindSocket(SocketName);
	if (Socket != nullptr)
	{
		FGLTFJsonNode* Node = Builder.AddNode();
		Node->Name = SocketName.ToString();

		// TODO: add warning check for non-uniform scaling
		Node->Translation = FGLTFCoreUtilities::ConvertPosition(FVector3f(Socket->RelativeLocation), Builder.ExportOptions->ExportUniformScale);
		Node->Rotation = FGLTFCoreUtilities::ConvertRotation(FRotator3f(Socket->RelativeRotation).Quaternion());
		Node->Scale = FGLTFCoreUtilities::ConvertScale(FVector3f(Socket->RelativeScale));

		const int32 ParentBone = ReferenceSkeleton.FindBoneIndex(Socket->BoneName);
		FGLTFJsonNode* ParentNode = ParentBone != INDEX_NONE ? Builder.AddUniqueNode(RootNode, SkeletalMesh, ParentBone) : RootNode;

		ParentNode->Children.Add(Node);
		return Node;
	}

	const int32 BoneIndex = ReferenceSkeleton.FindBoneIndex(SocketName);
	if (BoneIndex != INDEX_NONE)
	{
		return Builder.AddUniqueNode(RootNode, SkeletalMesh, BoneIndex);
	}

	// TODO: report error
	return nullptr;
}

FGLTFJsonNode* FGLTFSkeletalBoneConverter::Convert(FGLTFJsonNode* RootNode, const USkeletalMesh* SkeletalMesh, int32 BoneIndex)
{
	const FReferenceSkeleton& ReferenceSkeleton = SkeletalMesh->GetRefSkeleton();

	// TODO: add support for LeaderPoseComponent?

	const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton.GetRefBoneInfo();
	if (!BoneInfos.IsValidIndex(BoneIndex))
	{
		// TODO: report error
		return nullptr;
	}

	const FMeshBoneInfo& BoneInfo = BoneInfos[BoneIndex];

	FGLTFJsonNode* Node = Builder.AddNode();
	Node->Name = BoneInfo.Name.ToString();

	const TArray<FTransform>& BonePoses = ReferenceSkeleton.GetRefBonePose();
	if (BonePoses.IsValidIndex(BoneIndex))
	{
		// TODO: add warning check for non-uniform scaling
		const FTransform3f BonePose = FTransform3f(BonePoses[BoneIndex]);
		Node->Translation = FGLTFCoreUtilities::ConvertPosition(BonePose.GetTranslation(), Builder.ExportOptions->ExportUniformScale);
		Node->Rotation = FGLTFCoreUtilities::ConvertRotation(BonePose.GetRotation());
		Node->Scale = FGLTFCoreUtilities::ConvertScale(BonePose.GetScale3D());
	}
	else
	{
		// TODO: report error
	}

	FGLTFJsonNode* ParentNode = BoneInfo.ParentIndex != INDEX_NONE ? Builder.AddUniqueNode(RootNode, SkeletalMesh, BoneInfo.ParentIndex) : RootNode;
	ParentNode->Children.Add(Node);
	return Node;
}
