// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	NormalMapPreview.h: Implementation for previewing normal maps.
==============================================================================*/

#include "GLTFNormalMapPreview.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "SimpleElementShaders.h"
#include "ShaderParameterUtils.h"
#include "PipelineStateCache.h"
#include "RHIStaticStates.h"
#include "DataDrivenShaderPlatformInfo.h"

/*------------------------------------------------------------------------------
	Batched element shaders for previewing normal maps.
------------------------------------------------------------------------------*/

/**
 * Simple pixel shader that reconstructs a normal for the purposes of visualization.
 */
class FGLTFSimpleElementNormalMapPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FGLTFSimpleElementNormalMapPS,Global);
public:

	/** Should the shader be cached? Always. */
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsPCPlatform(Parameters.Platform);
	}

	/** Default constructor. */
	FGLTFSimpleElementNormalMapPS() {}

	/**
	 * Initialization constructor.
	 * @param Initializer - Shader initialization container.
	 */
	FGLTFSimpleElementNormalMapPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		Texture.Bind(Initializer.ParameterMap,TEXT("NormalMapTexture"));
		TextureSampler.Bind(Initializer.ParameterMap,TEXT("NormalMapTextureSampler"));
	}

	/**
	 * Set shader parameters.
	 * @param NormalMapTexture - The normal map texture to sample.
	 */
	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* NormalMapTexture)
	{
		SetTextureParameter(BatchedParameters, Texture, TextureSampler, NormalMapTexture);
	}

private:
	/** The texture to sample. */
	LAYOUT_FIELD(FShaderResourceParameter, Texture);
	LAYOUT_FIELD(FShaderResourceParameter, TextureSampler);
};
IMPLEMENT_SHADER_TYPE(,FGLTFSimpleElementNormalMapPS,TEXT("/Engine/Private/SimpleElementNormalMapPixelShader.usf"),TEXT("Main"),SF_Pixel);

/** Binds vertex and pixel shaders for this element */
void FGLTFNormalMapPreview::BindShaders(
	FRHICommandList& RHICmdList,
	FGraphicsPipelineStateInitializer& GraphicsPSOInit,
	ERHIFeatureLevel::Type InFeatureLevel,
	const FMatrix& InTransform,
	const float InGamma,
	const FMatrix& ColorWeights,
	const FTexture* Texture)
{
	TShaderMapRef<FSimpleElementVS> VertexShader(GetGlobalShaderMap(InFeatureLevel));
	TShaderMapRef<FGLTFSimpleElementNormalMapPS> PixelShader(GetGlobalShaderMap(InFeatureLevel));

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GSimpleElementVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;

	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0, EApplyRendertargetOption::CheckApply);

	SetShaderParametersLegacyVS(RHICmdList, VertexShader, InTransform);
	SetShaderParametersLegacyPS(RHICmdList, PixelShader, Texture);
}

