// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	NormalMapPreview.h: Definitions for previewing normal maps.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "BatchedElements.h"

/**
 * Batched element parameters for previewing normal maps.
 */
class FGLTFNormalMapPreview : public FBatchedElementParameters
{
	/** Binds vertex and pixel shaders for this element */
	virtual void BindShaders(
		FRHICommandList& RHICmdList,
		FGraphicsPipelineStateInitializer& GraphicsPSOInit,
		ERHIFeatureLevel::Type InFeatureLevel,
		const FMatrix& InTransform,
		const float InGamma,
		const FMatrix& ColorWeights,
		const FTexture* Texture) override;
};
