// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "MaterialCompiler.h"
#include "Materials/MaterialExpressionCustom.h"

class FGLTFProxyMaterialCompiler : public FProxyMaterialCompiler
{
public:

	bool bUsesActorPosition = false;
	bool bUsesObjectPosition = false;
	bool bUsesObjectOrientation = false;
	bool bUsesObjectRadius = false;
	bool bUsesObjectBounds = false;
	bool bUsesObjectLocalBounds = false;
	bool bUsesPreSkinnedLocalBounds = false;
	bool bUsesCustomPrimitiveData = false;
	bool bUsesTransformVector = false;
	bool bUsesTransformPosition = false;

	bool UsesPrimitiveData() const
	{
		return bUsesActorPosition
			|| bUsesObjectPosition
			|| bUsesObjectOrientation
			|| bUsesObjectRadius
			|| bUsesObjectBounds
			|| bUsesObjectLocalBounds
			|| bUsesPreSkinnedLocalBounds
			|| bUsesCustomPrimitiveData
			|| bUsesTransformVector
			|| bUsesTransformPosition;
	}

	using FProxyMaterialCompiler::FProxyMaterialCompiler;

	virtual int32 ActorWorldPosition(EPositionOrigin OriginType) override
	{
		bUsesActorPosition = true;
		return Compiler->ActorWorldPosition(OriginType);
	}

	virtual int32 ObjectWorldPosition(EPositionOrigin OriginType) override
	{
		bUsesObjectPosition = true;
		return Compiler->ObjectWorldPosition(OriginType);
	}

	virtual int32 ObjectOrientation() override
	{
		bUsesObjectOrientation = true;
		return Compiler->ObjectOrientation();
	}

	virtual int32 ObjectRadius() override
	{
		bUsesObjectRadius = true;
		return Compiler->ObjectRadius();
	}

	virtual int32 ObjectBounds() override
	{
		bUsesObjectBounds = true;
		return Compiler->ObjectBounds();
	}

	virtual int32 ObjectLocalBounds(int32 OutputIndex) override
	{
		bUsesObjectLocalBounds = true;
		return Compiler->ObjectLocalBounds(OutputIndex);
	}

	// NOTE: also need to check old in-engine material function ObjectLocalBounds that bypasses the above override since it uses custom expressions 
	virtual int32 CustomExpression(class UMaterialExpressionCustom* Custom, int32 OutputIndex, TArray<int32>& CompiledInputs) override
	{
		if (Custom->Code.Contains(TEXT("GetPrimitiveData(Parameters).LocalObjectBounds")))
		{
			bUsesObjectLocalBounds = true;
		}

		return Compiler->CustomExpression(Custom, OutputIndex, CompiledInputs);
	}

	virtual int32 PreSkinnedLocalBounds(int32 OutputIndex) override
	{
		bUsesPreSkinnedLocalBounds = true;
		return Compiler->PreSkinnedLocalBounds(OutputIndex);
	}

	virtual int32 CustomPrimitiveData(int32 OutputIndex, EMaterialValueType Type) override
	{
		bUsesCustomPrimitiveData = true;
		return Compiler->CustomPrimitiveData(OutputIndex, Type);
	}

	virtual int32 TransformVector(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, FTransformParameters& Parameters, int32 A) override
	{
		if (SourceCoordBasis != DestCoordBasis) bUsesTransformVector = true; // TODO: optimize later by ignoring some irrelevant basis (like MCB_View and MCB_Camera)
		return Compiler->TransformVector(SourceCoordBasis, DestCoordBasis, Parameters, A);
	}

	virtual int32 TransformPosition(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, FTransformParameters& Parameters, int32 A) override
	{
		if (SourceCoordBasis != DestCoordBasis) bUsesTransformPosition = true; // TODO: optimize later by ignoring some irrelevant basis (like MCB_View and MCB_Camera)
		return Compiler->TransformPosition(SourceCoordBasis, DestCoordBasis, Parameters, A);
	}

	virtual int32 GIReplace(int32 Direct, int32 StaticIndirect, int32 DynamicIndirect) override
	{
		return Direct; // ignore all non-default branches in this runtime switch
	}

	virtual int32 ShadowReplace(int32 Default, int32 Shadow) override
	{
		return Default; // ignore all non-default branches in this runtime switch
	}

	virtual int32 RayTracingQualitySwitchReplace(int32 Normal, int32 RayTraced) override
	{
		return Normal; // ignore all non-default branches in this runtime switch
	}

	virtual int32 ReflectionCapturePassSwitch(int32 Default, int32 Reflection) override
	{
		return Default; // ignore all non-default branches in this runtime switch
	}

	virtual int32 ReflectionAboutCustomWorldNormal(int32 CustomWorldNormal, int32 bNormalizeCustomWorldNormal) override
	{
		return Compiler->ReflectionAboutCustomWorldNormal(CustomWorldNormal, bNormalizeCustomWorldNormal);
	}

	virtual int32 ParticleRelativeTime() override
	{
		return Compiler->ParticleRelativeTime();
	}

	virtual int32 ParticleMotionBlurFade() override
	{
		return Compiler->ParticleMotionBlurFade();
	}

	virtual int32 ParticleRandom() override
	{
		return Compiler->ParticleRandom();
	}

	virtual int32 ParticleDirection() override
	{
		return Compiler->ParticleDirection();
	}

	virtual int32 ParticleSpeed() override
	{
		return Compiler->ParticleSpeed();
	}

	virtual int32 ParticleSize() override
	{
		return Compiler->ParticleSize();
	}

	virtual int32 ParticleSpriteRotation() override
	{
		return Compiler->ParticleSpriteRotation();
	}

	virtual int32 VertexInterpolator(uint32 InterpolatorIndex) override
	{
		return Compiler->VertexInterpolator(InterpolatorIndex);
	}

	virtual int32 MaterialBakingWorldPosition() override
	{
		return Compiler->MaterialBakingWorldPosition();
	}

	virtual EMaterialCompilerType GetCompilerType() const override
	{
		return EMaterialCompilerType::MaterialProxy;
	}
};

#endif
