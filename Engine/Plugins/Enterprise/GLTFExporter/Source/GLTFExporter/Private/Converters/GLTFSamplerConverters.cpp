// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFSamplerConverters.h"
#include "Builders/GLTFContainerBuilder.h"
#include "Utilities/GLTFCoreUtilities.h"
#include "Converters/GLTFTextureUtilities.h"

void FGLTFSamplerConverter::Sanitize(TextureAddress& AddressX, TextureAddress& AddressY, TextureFilter& Filter, TextureGroup& LODGroup)
{
	if (Filter == TF_Default)
	{
		Filter = FGLTFTextureUtilities::GetDefaultFilter(LODGroup);
	}

	LODGroup = TEXTUREGROUP_MAX; // Ignore it, since Filter should cover any missing information
}

FGLTFJsonSampler* FGLTFSamplerConverter::Convert(TextureAddress AddressX, TextureAddress AddressY, TextureFilter Filter, TextureGroup LODGroup)
{
	FGLTFJsonSampler* JsonSampler = Builder.AddSampler();
	JsonSampler->MinFilter = FGLTFCoreUtilities::ConvertMinFilter(Filter);
	JsonSampler->MagFilter = FGLTFCoreUtilities::ConvertMagFilter(Filter);
	JsonSampler->WrapS = FGLTFCoreUtilities::ConvertWrap(AddressX);
	JsonSampler->WrapT = FGLTFCoreUtilities::ConvertWrap(AddressY);
	return JsonSampler;
}
