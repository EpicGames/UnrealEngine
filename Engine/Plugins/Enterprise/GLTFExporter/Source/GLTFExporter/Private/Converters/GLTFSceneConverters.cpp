// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFSceneConverters.h"
#include "Builders/GLTFContainerBuilder.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "LevelVariantSetsActor.h"
#include "LevelVariantSets.h"
#include "VariantSet.h"

FGLTFJsonScene* FGLTFSceneConverter::Convert(const UWorld* World)
{
	FGLTFJsonScene* Scene = Builder.AddScene();
	World->GetName(Scene->Name);

	const TArray<ULevel*>& Levels = World->GetLevels();
	if (Levels.Num() > 0)
	{
		for (const ULevel* Level : Levels)
		{
			if (Level == nullptr || (Level->IsInstancedLevel() && !Level->IsCurrentLevel()))
			{
				//LevelInstances are handled in FGLTFComponentConverter::ConvertComponentSpecialization in the ULevelInstanceComponent handling branch.
				continue;
			}

			// TODO: add support for exporting Level->Model?

			TArray<AActor*> LevelActors(Level->Actors); // iterate using copy, in case new temporary actors are added during conversion

			if (Builder.ExportOptions->ExportMaterialVariants != EGLTFMaterialVariantMode::None)
			{
				for (const AActor* Actor : LevelActors)
				{
					// TODO: should a LevelVariantSet be exported even if not selected for export?
					if (const ALevelVariantSetsActor *LevelVariantSetsActor = Cast<ALevelVariantSetsActor>(Actor))
					{
						const ULevelVariantSets* LevelVariantSets = const_cast<ALevelVariantSetsActor*>(LevelVariantSetsActor)->GetLevelVariantSets(true);

						if (LevelVariantSets != nullptr)
						{
							for (const UVariantSet* VariantSet: LevelVariantSets->GetVariantSets())
							{
								for (const UVariant* Variant: VariantSet->GetVariants())
								{
									Builder.AddUniqueMaterialVariant(Variant);
								}
							}
						}
					}
				}
			}

			for (const AActor* Actor : LevelActors)
			{
				FGLTFJsonNode* Node = Builder.AddUniqueNode(Actor);
				if (Node != nullptr && Builder.IsRootActor(Actor))
				{
					// TODO: to avoid having to add irrelevant actors/components let GLTFComponentConverter decide and add root nodes to Scene->
					// This change may require node converters to support cyclic calls.
					Scene->Nodes.Add(Node);
				}
			}

			if (Builder.ExportOptions->bMakeSkinnedMeshesRoot)
			{
				MakeSkinnedMeshesRoot(Scene);
			}
		}
	}
	else
	{
		Builder.LogWarning(
			FString::Printf(TEXT("World %s has no levels. Please make sure the world has been fully initialized"),
			*World->GetName()));
	}

	return Scene;
}

void FGLTFSceneConverter::MakeSkinnedMeshesRoot(FGLTFJsonScene* Scene)
{
	TArray<FGLTFJsonNode*> OriginalRootNodes = Scene->Nodes;

	for (FGLTFJsonNode* RootNode : OriginalRootNodes)
	{
		MakeSkinnedMeshesRoot(RootNode, true, Scene);
	}
}

void FGLTFSceneConverter::MakeSkinnedMeshesRoot(FGLTFJsonNode* Node, bool bIsRootNode, FGLTFJsonScene* Scene)
{
	if (Node->Mesh != nullptr && Node->Skin != nullptr && (!bIsRootNode || *Node != FGLTFJsonTransform::Identity))
	{
		// The glTF spec states "the transform of the skinned mesh node MUST be ignored" (only joint transforms are applied).
		// To comply with this requirement we can move the skinned mesh to a unique root node that doesn't have any transform.
		// The end result will be the same, since final joint transforms are unaffected and still applied to the skinned mesh.

		FGLTFJsonNode* SkinnedMeshNode = Builder.AddNode();
		SkinnedMeshNode->Mesh = Node->Mesh;
		SkinnedMeshNode->Skin = Node->Skin;

		Node->Mesh = nullptr;
		Node->Skin = nullptr;
		Scene->Nodes.Add(SkinnedMeshNode);
	}

	for (FGLTFJsonNode* ChildNode : Node->Children)
	{
		MakeSkinnedMeshesRoot(ChildNode, false, Scene);
	}
}
