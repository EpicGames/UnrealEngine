// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	NormalMapPreview.h: Implementation for previewing normal maps.
==============================================================================*/

#include "GLTFSimpleTexture2DPreview.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "RHIStaticStates.h"
#include "SimpleElementShaders.h"
#include "ShaderParameterUtils.h"
#include "PipelineStateCache.h"
#include "DataDrivenShaderPlatformInfo.h"

class FGLTFSimpleTexture2DPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FGLTFSimpleTexture2DPS,Global);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsPCPlatform(Parameters.Platform);
	}

	FGLTFSimpleTexture2DPS() {}

	/**
	 * Initialization constructor.
	 * @param Initializer - Shader initialization container.
	 */
	FGLTFSimpleTexture2DPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		Texture.Bind(Initializer.ParameterMap,TEXT("Texture"));
		TextureSampler.Bind(Initializer.ParameterMap,TEXT("TextureSampler"));
	}

	/**
	 * Set shader parameters.
	 * @param SampleTexture - The texture to sample.
	 */
	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* SampleTexture)
	{
		SetTextureParameter(BatchedParameters, Texture, TextureSampler, SampleTexture);
	}

private:
	/** The texture to sample. */
	LAYOUT_FIELD(FShaderResourceParameter, Texture);
	LAYOUT_FIELD(FShaderResourceParameter, TextureSampler);
};
IMPLEMENT_SHADER_TYPE(,FGLTFSimpleTexture2DPS,TEXT("/Plugin/GLTFExporter/Private/SimpleTexture2DPS.usf"),TEXT("Main"),SF_Pixel);

/** Binds vertex and pixel shaders for this element */
void FGLTFSimpleTexture2DPreview::BindShaders(
	FRHICommandList& RHICmdList,
	FGraphicsPipelineStateInitializer& GraphicsPSOInit,
	ERHIFeatureLevel::Type InFeatureLevel,
	const FMatrix& InTransform,
	const float InGamma,
	const FMatrix& ColorWeights,
	const FTexture* Texture)
{
	TShaderMapRef<FSimpleElementVS> VertexShader(GetGlobalShaderMap(InFeatureLevel));
	TShaderMapRef<FGLTFSimpleTexture2DPS> PixelShader(GetGlobalShaderMap(InFeatureLevel));

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GSimpleElementVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;

	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0, EApplyRendertargetOption::CheckApply);

	SetShaderParametersLegacyVS(RHICmdList, VertexShader, InTransform);
	SetShaderParametersLegacyPS(RHICmdList, PixelShader, Texture);
}
