// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BatchedElements.h"

class FGLTFSimpleTexture2DPreview : public FBatchedElementParameters
{
	/** Binds vertex and pixel shaders for this element */
	virtual void BindShaders(
		FRHICommandList& RHICmdList,
		FGraphicsPipelineStateInitializer& GraphicsPSOInit,
		ERHIFeatureLevel::Type InFeatureLevel,
		const FMatrix& InTransform,
		const float InGamma,
		const FMatrix& ColorWeights,
		const FTexture* Texture) override;
};
