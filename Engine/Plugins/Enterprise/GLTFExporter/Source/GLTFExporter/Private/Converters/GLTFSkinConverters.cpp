// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFSkinConverters.h"
#include "Utilities/GLTFCoreUtilities.h"
#include "Converters/GLTFBoneUtilities.h"
#include "Builders/GLTFConvertBuilder.h"
#include "Engine/SkeletalMesh.h"

FGLTFJsonSkin* FGLTFSkinConverter::Convert(FGLTFJsonNode* RootNode, const USkeletalMesh* SkeletalMesh)
{
	const FReferenceSkeleton& ReferenceSkeleton = SkeletalMesh->GetRefSkeleton();

	const int32 BoneCount = ReferenceSkeleton.GetNum();
	if (BoneCount == 0)
	{
		// TODO: report warning
		return nullptr;
	}

	FGLTFJsonSkin* JsonSkin = Builder.AddSkin();
	JsonSkin->Name = SkeletalMesh->GetName();
	JsonSkin->Skeleton = RootNode;
	JsonSkin->Joints.AddUninitialized(BoneCount);

	for (int32 BoneIndex = 0; BoneIndex < BoneCount; ++BoneIndex)
	{
		JsonSkin->Joints[BoneIndex] = Builder.AddUniqueNode(RootNode, SkeletalMesh, BoneIndex);
	}

	TArray<FGLTFMatrix4> InverseBindMatrices;
	InverseBindMatrices.AddUninitialized(BoneCount);

	for (int32 BoneIndex = 0; BoneIndex < BoneCount; ++BoneIndex)
	{
		const FTransform3f InverseBindTransform = FTransform3f(FGLTFBoneUtilities::GetBindTransform(ReferenceSkeleton, BoneIndex).Inverse());
		InverseBindMatrices[BoneIndex] = FGLTFCoreUtilities::ConvertTransform(InverseBindTransform, Builder.ExportOptions->ExportUniformScale);
	}

	FGLTFJsonAccessor* JsonBindMatricesAccessor = Builder.AddAccessor();
	JsonBindMatricesAccessor->BufferView = Builder.AddBufferView(InverseBindMatrices);
	JsonBindMatricesAccessor->ComponentType = EGLTFJsonComponentType::Float;
	JsonBindMatricesAccessor->Count = BoneCount;
	JsonBindMatricesAccessor->Type = EGLTFJsonAccessorType::Mat4;

	JsonSkin->InverseBindMatrices = JsonBindMatricesAccessor;
	return JsonSkin;
}
