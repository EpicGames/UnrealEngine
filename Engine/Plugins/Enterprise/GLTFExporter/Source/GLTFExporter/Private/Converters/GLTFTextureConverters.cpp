// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFTextureConverters.h"
#include "Tasks/GLTFDelayedTextureTasks.h"

FGLTFJsonTexture* FGLTFTexture2DConverter::Convert(const UTexture2D* Texture2D, bool bToSRGB, TextureAddress WrapS, TextureAddress WrapT)
{
	if (Builder.ExportOptions->TextureImageFormat == EGLTFTextureImageFormat::None)
	{
		return nullptr;
	}

	FGLTFJsonTexture* JsonTexture = Builder.AddTexture();
	Builder.ScheduleSlowTask<FGLTFDelayedTexture2DTask>(Builder, Texture2D, bToSRGB, JsonTexture, WrapS, WrapT);
	return JsonTexture;
}

FGLTFJsonTexture* FGLTFTextureRenderTarget2DConverter::Convert(const UTextureRenderTarget2D* RenderTarget2D, bool bToSRGB)
{
	if (Builder.ExportOptions->TextureImageFormat == EGLTFTextureImageFormat::None)
	{
		return nullptr;
	}

	FGLTFJsonTexture* JsonTexture = Builder.AddTexture();
	Builder.ScheduleSlowTask<FGLTFDelayedTextureRenderTarget2DTask>(Builder, RenderTarget2D, bToSRGB, JsonTexture);
	return JsonTexture;
}

FGLTFJsonTexture* FGLTFTextureLightMapConverter::Convert(const ULightMapTexture2D* LightMap)
{
	if (Builder.ExportOptions->TextureImageFormat == EGLTFTextureImageFormat::None)
	{
		return nullptr;
	}

#if WITH_EDITOR
	FGLTFJsonTexture* JsonTexture = Builder.AddTexture();
	Builder.ScheduleSlowTask<FGLTFDelayedTextureLightMapTask>(Builder, LightMap, JsonTexture);
	return JsonTexture;
#else
	return nullptr;
#endif
}
