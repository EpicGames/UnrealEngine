// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFTextureUtilities.h"
#include "Converters/GLTFNormalMapPreview.h"
#include "Converters/GLTFSimpleTexture2DPreview.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "TextureCompiler.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureCube.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTargetCube.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "RenderingThread.h"
#include "TextureResource.h"

bool FGLTFTextureUtilities::IsAlphaless(EPixelFormat PixelFormat)
{
	switch (PixelFormat)
	{
		case PF_ATC_RGB:
		case PF_BC4:
		case PF_BC5:
		case PF_DXT1:
		case PF_ETC1:
		case PF_ETC2_RGB:
		case PF_FloatR11G11B10:
		case PF_FloatRGB:
		case PF_R5G6B5_UNORM:
			// TODO: add more pixel formats that don't support alpha, but beware of formats like PF_G8 (that still seem to return alpha in some cases)
			return true;
		default:
			return false;
	}
}

void FGLTFTextureUtilities::FullyLoad(const UTexture* InTexture)
{
	UTexture* Texture = const_cast<UTexture*>(InTexture);

#if WITH_EDITOR
	FTextureCompilingManager::Get().FinishCompilation({ Texture });
#endif

	Texture->SetForceMipLevelsToBeResident(30.0f);
	Texture->WaitForStreaming();
}

bool FGLTFTextureUtilities::Is2D(const UTexture* Texture)
{
	return Texture->IsA<UTexture2D>() || Texture->IsA<UTextureRenderTarget2D>();
}

bool FGLTFTextureUtilities::IsCubemap(const UTexture* Texture)
{
	return Texture->IsA<UTextureCube>() || Texture->IsA<UTextureRenderTargetCube>();
}

bool FGLTFTextureUtilities::IsHDR(const UTexture* Texture)
{
	switch (Texture->CompressionSettings)
	{
		case TC_HDR:
		case TC_HDR_Compressed:
		case TC_HalfFloat:
			return true;
		default:
			return false;
	}
}

TextureFilter FGLTFTextureUtilities::GetDefaultFilter(TextureGroup LODGroup)
{
	const UTextureLODSettings* TextureLODSettings = UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings();
	const ETextureSamplerFilter Filter = TextureLODSettings->GetTextureLODGroup(LODGroup).Filter;

	switch (Filter)
	{
		case ETextureSamplerFilter::Point:             return TF_Nearest;
		case ETextureSamplerFilter::Bilinear:          return TF_Bilinear;
		case ETextureSamplerFilter::Trilinear:         return TF_Trilinear;
		case ETextureSamplerFilter::AnisotropicPoint:  return TF_Trilinear; // A lot of engine code doesn't result in nearest
		case ETextureSamplerFilter::AnisotropicLinear: return TF_Trilinear;
		default:                                       return TF_MAX;
	}
}

int32 FGLTFTextureUtilities::GetMipBias(const UTexture* Texture)
{
	if (const UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
	{
		return Texture2D->GetNumMips() - Texture2D->GetNumMipsAllowed(true);
	}

	return Texture->GetCachedLODBias();
}

FIntPoint FGLTFTextureUtilities::GetInGameSize(const UTexture* Texture)
{
	const int32 Width = FMath::CeilToInt(Texture->GetSurfaceWidth());
	const int32 Height = FMath::CeilToInt(Texture->GetSurfaceHeight());

	const int32 MipBias = GetMipBias(Texture);

	const int32 InGameWidth = FMath::Max(Width >> MipBias, 1);
	const int32 InGameHeight = FMath::Max(Height >> MipBias, 1);

	return { InGameWidth, InGameHeight };
}

UTextureRenderTarget2D* FGLTFTextureUtilities::CreateRenderTarget(const FIntPoint& Size, bool bHDR)
{
	// TODO: instead of PF_FloatRGBA (i.e. RTF_RGBA16f) use PF_A32B32G32R32F (i.e. RTF_RGBA32f) to avoid accuracy loss
	const EPixelFormat PixelFormat = bHDR ? PF_FloatRGBA : PF_B8G8R8A8;

	// NOTE: both bForceLinearGamma and TargetGamma=2.2 seem necessary for exported images to match their source data.
	// It's not entirely clear why gamma must be 2.2 (instead of 0.0) and why bInForceLinearGamma must also be true.
	constexpr bool bForceLinearGamma = true;
	constexpr float TargetGamma = 2.2f;

	UTextureRenderTarget2D* RenderTarget = NewObject<UTextureRenderTarget2D>();
	RenderTarget->InitCustomFormat(Size.X, Size.Y, PixelFormat, bForceLinearGamma);

	RenderTarget->TargetGamma = TargetGamma;
	return RenderTarget;
}

bool FGLTFTextureUtilities::DrawTexture(UTextureRenderTarget2D* OutTarget, const UTexture2D* InSource)
{
	FRenderTarget* RenderTarget = OutTarget->GameThread_GetRenderTargetResource();
	if (RenderTarget == nullptr)
	{
		return false;
	}

	TRefCountPtr<FBatchedElementParameters> BatchedElementParameters;

	if (InSource->IsNormalMap())
	{
		BatchedElementParameters = new FGLTFNormalMapPreview();
	}
	else if (IsHDR(InSource))
	{
		// NOTE: Simple preview parameters are used to prevent any modifications
		// such as gamma-correction from being applied during rendering.
		BatchedElementParameters = new FGLTFSimpleTexture2DPreview();
	}

	FCanvas Canvas(RenderTarget, nullptr, FGameTime::CreateDilated(0.0f, 0.0f, 0.0f, 0.0f), GMaxRHIFeatureLevel);
	FCanvasTileItem TileItem(FVector2D::ZeroVector, InSource->GetResource(), FLinearColor::White);

	TileItem.BatchedElementParameters = BatchedElementParameters;
	TileItem.Draw(&Canvas);

	Canvas.Flush_GameThread();
	FlushRenderingCommands();
	Canvas.SetRenderTarget_GameThread(nullptr);
	FlushRenderingCommands();

	return true;
}

bool FGLTFTextureUtilities::ReadPixels(const UTextureRenderTarget2D* InRenderTarget, TArray<FColor>& OutPixels)
{
	FTextureRenderTarget2DResource* Resource = static_cast<FTextureRenderTarget2DResource*>(const_cast<UTextureRenderTarget2D*>(InRenderTarget)->GetResource());
	if (Resource == nullptr)
	{
		return false;
	}

	FReadSurfaceDataFlags ReadSurfaceDataFlags(RCM_UNorm, CubeFace_MAX);
	ReadSurfaceDataFlags.SetLinearToGamma(false);
	return Resource->ReadPixels(OutPixels, ReadSurfaceDataFlags);
}

void FGLTFTextureUtilities::FlipGreenChannel(TArray<FColor>& Pixels)
{
	for (FColor& Pixel: Pixels)
	{
		Pixel.G = 255 - Pixel.G;
	}
}

void FGLTFTextureUtilities::TransformColorSpace(TArray<FColor>& Pixels, bool bFromSRGB, bool bToSRGB)
{
	if (bFromSRGB == bToSRGB)
	{
		return;
	}

	if (bToSRGB)
	{
		for (FColor& Pixel: Pixels)
		{
			Pixel = Pixel.ReinterpretAsLinear().ToFColor(true);
		}
	}
	else
	{
		for (FColor& Pixel: Pixels)
		{
			Pixel = FLinearColor(Pixel).ToFColor(false);
		}
	}
}
