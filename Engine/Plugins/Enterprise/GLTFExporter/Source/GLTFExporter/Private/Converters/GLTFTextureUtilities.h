// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelFormat.h"
#include "Engine/TextureDefines.h"

class UTexture;
class UTexture2D;
class UTextureRenderTarget2D;

struct FGLTFTextureUtilities
{
	static bool IsAlphaless(EPixelFormat PixelFormat);

	static void FullyLoad(const UTexture* InTexture);

	static bool Is2D(const UTexture* Texture);
	static bool IsCubemap(const UTexture* Texture);
	static bool IsHDR(const UTexture* Texture);

	static TextureFilter GetDefaultFilter(TextureGroup Group);

	static int32 GetMipBias(const UTexture* Texture);

	static FIntPoint GetInGameSize(const UTexture* Texture);

	static UTextureRenderTarget2D* CreateRenderTarget(const FIntPoint& Size, bool bIsHDR);

	static bool DrawTexture(UTextureRenderTarget2D* OutTarget, const UTexture2D* InSource);

	static bool ReadPixels(const UTextureRenderTarget2D* InRenderTarget, TArray<FColor>& OutPixels);

	static void FlipGreenChannel(TArray<FColor>& Pixels);
	static void TransformColorSpace(TArray<FColor>& Pixels, bool bFromSRGB, bool bToSRGB);
};
