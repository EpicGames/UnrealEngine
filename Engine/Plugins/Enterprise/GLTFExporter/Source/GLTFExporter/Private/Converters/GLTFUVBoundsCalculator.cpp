// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFUVBoundsCalculator.h"
#if WITH_EDITOR
#include "StaticMeshAttributes.h"
#include "MeshDescription.h"
#endif

FBox2f FGLTFUVBoundsCalculator::Convert(const FMeshDescription* Description, FGLTFIndexArray SectionIndices, int32 TexCoord)
{
	FBox2f UVBounds(ForceInit);

#if WITH_EDITOR
	if (Description != nullptr)
	{
		const TVertexInstanceAttributesConstRef<FVector2f> UVs =
			Description->VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);

		for (const int32 SectionIndex : SectionIndices)
		{
			for (const FPolygonID PolygonID : Description->GetPolygonGroupPolygonIDs(FPolygonGroupID(SectionIndex)))
			{
				for (const FTriangleID TriangleID : Description->GetPolygonTriangles(PolygonID))
				{
					for (const FVertexInstanceID TriangleVertexInstanceID : Description->GetTriangleVertexInstances(TriangleID))
					{
						const FVector2f UV = UVs.Get(TriangleVertexInstanceID, TexCoord);
						UVBounds += UV;
					}
				}
			}
		}
	}
#endif

	return UVBounds;
}
