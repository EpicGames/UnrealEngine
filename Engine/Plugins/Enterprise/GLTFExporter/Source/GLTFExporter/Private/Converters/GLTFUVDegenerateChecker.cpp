// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFUVDegenerateChecker.h"
#if WITH_EDITOR
#include "StaticMeshAttributes.h"
#include "MeshDescription.h"
#endif

float FGLTFUVDegenerateChecker::Convert(const FMeshDescription* Description, FGLTFIndexArray SectionIndices, int32 TexCoord)
{
#if WITH_EDITOR
	if (Description != nullptr)
	{
		const TVertexAttributesConstRef<FVector3f> Positions =
			Description->VertexAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position);
		const TVertexInstanceAttributesConstRef<FVector2f> UVs =
			Description->VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);

		int32 TriangleCount = 0;
		int32 DegenerateCount = 0;

		for (const int32 SectionIndex : SectionIndices)
		{
			for (const FPolygonID PolygonID : Description->GetPolygonGroupPolygonIDs(FPolygonGroupID(SectionIndex)))
			{
				for (const FTriangleID TriangleID : Description->GetPolygonTriangles(PolygonID))
				{
					const TArrayView<const FVertexID> TriangleVertexIDs = Description->GetTriangleVertices(TriangleID);
					const TArrayView<const FVertexInstanceID> TriangleVertexInstanceIDs = Description->GetTriangleVertexInstances(TriangleID);

					TStaticArray<FVector3f, 3> TrianglePositions;
					TStaticArray<FVector2f, 3> TriangleUVs;

					for (int32 Index = 0; Index < 3; Index++)
					{
						TrianglePositions[Index] = Positions.Get(TriangleVertexIDs[Index]);
						TriangleUVs[Index] = UVs.Get(TriangleVertexInstanceIDs[Index], TexCoord);
					}

					if (!IsDegenerateTriangle(TrianglePositions))
					{
						TriangleCount++;

						if (IsDegenerateTriangle(TriangleUVs))
						{
							DegenerateCount++;
						}
					}
				}
			}
		}

		if (TriangleCount == 0)
		{
			return -1;
		}

		if (TriangleCount == DegenerateCount)
		{
			return 1;
		}

		return static_cast<float>(DegenerateCount) / static_cast<float>(TriangleCount);
	}
#endif

	return -1;
}

bool FGLTFUVDegenerateChecker::IsDegenerateTriangle(const TStaticArray<FVector2f, 3>& Points)
{
	const FVector2f AB = Points[1] - Points[0];
	const FVector2f AC = Points[2] - Points[0];
	const float DoubleArea = FMath::Abs(AB ^ AC);
	return DoubleArea < 2 * SMALL_NUMBER;
}

bool FGLTFUVDegenerateChecker::IsDegenerateTriangle(const TStaticArray<FVector3f, 3>& Points)
{
	const FVector3f AB = Points[1] - Points[0];
	const FVector3f AC = Points[2] - Points[0];
	const float DoubleAreaSquared = (AB ^ AC).SizeSquared();
	return DoubleAreaSquared < 2 * SMALL_NUMBER * SMALL_NUMBER;
}
