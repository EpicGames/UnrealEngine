// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFUVOverlapChecker.h"
#include "Materials/MaterialInterface.h"
#if WITH_EDITOR
#include "StaticMeshAttributes.h"
#include "MeshDescription.h"
#include "Modules/ModuleManager.h"
#include "IMaterialBakingModule.h"
#include "MaterialBakingStructures.h"
#else
#include "UObject/UObjectGlobals.h"
#endif

float FGLTFUVOverlapChecker::Convert(const FMeshDescription* Description, FGLTFIndexArray SectionIndices, int32 TexCoord)
{
#if WITH_EDITOR
	if (Description != nullptr)
	{
		// TODO: investigate if the fixed size is high enough to properly calculate overlap
		const FIntPoint TextureSize(512, 512);
		const FMaterialPropertyEx Property = MP_Opacity;

		FMeshData MeshSet;
		MeshSet.TextureCoordinateBox = { { 0.0f, 0.0f }, { 1.0f, 1.0f } };
		MeshSet.TextureCoordinateIndex = TexCoord;
		MeshSet.MeshDescription = const_cast<FMeshDescription*>(Description);
		MeshSet.MaterialIndices = SectionIndices; // NOTE: MaterialIndices is actually section indices

		FMaterialDataEx MatSet;
		MatSet.Material = GetMaterial();
		MatSet.PropertySizes.Add(Property, TextureSize);
		MatSet.BlendMode = MatSet.Material->GetBlendMode();
		MatSet.BackgroundColor = FColor::Black;
		MatSet.bPerformBorderSmear = false;

		TArray<FMeshData*> MeshSettings;
		TArray<FMaterialDataEx*> MatSettings;
		MeshSettings.Add(&MeshSet);
		MatSettings.Add(&MatSet);

		TArray<FBakeOutputEx> BakeOutputs;
		IMaterialBakingModule& Module = FModuleManager::Get().LoadModuleChecked<IMaterialBakingModule>("MaterialBaking");

		Module.BakeMaterials(MatSettings, MeshSettings, BakeOutputs);

		const FBakeOutputEx& BakeOutput = BakeOutputs[0];
		const TArray<FColor>& BakedPixels = BakeOutput.PropertyData.FindChecked(Property);

		if (BakedPixels.Num() <= 0)
		{
			return -1;
		}

		int32 TotalCount = 0;
		int32 OverlapCount = 0;

		for (const FColor& Pixel: BakedPixels)
		{
			const bool bIsBackground = Pixel.G < 64;
			if (bIsBackground)
			{
				continue;
			}

			TotalCount++;

			const bool bIsOverlapping = Pixel.G > 192;
			if (bIsOverlapping)
			{
				OverlapCount++;
			}
		}

		if (TotalCount == 0)
		{
			return -1;
		}

		if (TotalCount == OverlapCount)
		{
			return 1;
		}

		return static_cast<float>(OverlapCount) / static_cast<float>(TotalCount);
	}
#endif

	return -1;
}

UMaterialInterface* FGLTFUVOverlapChecker::GetMaterial()
{
	static UMaterialInterface* Material = LoadObject<UMaterialInterface>(nullptr, TEXT("/GLTFExporter/Materials/M_UVOverlapChecker.M_UVOverlapChecker"));
	check(Material);
	return Material;
}
