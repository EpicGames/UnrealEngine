// Copyright Epic Games, Inc. All Rights Reserved.

#include "Converters/GLTFVariantUtilities.h"
#include "Builders/GLTFConvertBuilder.h"
#include "LevelVariantSets.h"
#include "VariantSet.h"
#include "Variant.h"
#include "VariantObjectBinding.h"
#include "PropertyValue.h"

bool FGLTFVariantUtilities::TryGetPropertyValue(UPropertyValue* Property, void* OutData, uint32 OutSize)
{
	if (Property == nullptr || !Property->HasRecordedData())
	{
		return false;
	}

	const TArray<uint8>& RecordedData = Property->GetRecordedData();
	check(OutSize == RecordedData.Num());

	FMemory::Memcpy(OutData, RecordedData.GetData(), OutSize);
	return true;
}

FString FGLTFVariantUtilities::GetLogContext(const UPropertyValue* Property)
{
	const UVariantObjectBinding* Parent = Property->GetParent();
	return GetLogContext(Parent) + TEXT("/") + Property->GetFullDisplayString();
}

FString FGLTFVariantUtilities::GetLogContext(const UVariantObjectBinding* Binding)
{
	const UVariant* Parent = const_cast<UVariantObjectBinding*>(Binding)->GetParent();
	return GetLogContext(Parent) + TEXT("/") + Binding->GetDisplayText().ToString();
}

FString FGLTFVariantUtilities::GetLogContext(const UVariant* Variant)
{
	const UVariantSet* Parent = const_cast<UVariant*>(Variant)->GetParent();
	return GetLogContext(Parent) + TEXT("/") + Variant->GetDisplayText().ToString();
}

FString FGLTFVariantUtilities::GetLogContext(const UVariantSet* VariantSet)
{
	const ULevelVariantSets* Parent = const_cast<UVariantSet*>(VariantSet)->GetParent();
	return GetLogContext(Parent) + TEXT("/") + VariantSet->GetDisplayText().ToString();
}

FString FGLTFVariantUtilities::GetLogContext(const ULevelVariantSets* LevelVariantSets)
{
	return LevelVariantSets->GetName();
}

FGLTFJsonMaterial* FGLTFVariantUtilities::AddUniqueMaterial(FGLTFConvertBuilder& Builder, const UMaterialInterface* Material, const UMeshComponent* MeshComponent, int32 MaterialIndex)
{
	if (Builder.ExportOptions->ExportMaterialVariants == EGLTFMaterialVariantMode::UseMeshData)
	{
		if (Builder.ExportOptions->BakeMaterialInputs == EGLTFMaterialBakeMode::UseMeshData)
		{
			return Builder.AddUniqueMaterial(Material, MeshComponent, -1, MaterialIndex);
		}

		// TODO: report warning (about materials won't be export using mesh data because BakeMaterialInputs is not set to UseMeshData)
	}

	return Builder.AddUniqueMaterial(Material);
}
