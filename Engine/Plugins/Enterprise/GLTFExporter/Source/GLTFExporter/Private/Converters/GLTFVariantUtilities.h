// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"

class FGLTFConvertBuilder;
struct FCapturedPropSegment;
class ULevelVariantSets;
class UMaterialInterface;
class UMeshComponent;
class UVariantSet;
class UVariant;
class UVariantObjectBinding;
class UPropertyValue;

struct FGLTFVariantUtilities
{
	template <typename T>
	static bool TryGetPropertyValue(UPropertyValue* Property, T& OutValue)
	{
		return TryGetPropertyValue(Property, &OutValue, sizeof(T));
	}

	static bool TryGetPropertyValue(UPropertyValue* Property, void* OutData, uint32 OutSize);

	static FString GetLogContext(const UPropertyValue* Property);
	static FString GetLogContext(const UVariantObjectBinding* Binding);
	static FString GetLogContext(const UVariant* Variant);
	static FString GetLogContext(const UVariantSet* VariantSet);
	static FString GetLogContext(const ULevelVariantSets* LevelVariantSets);

	static FGLTFJsonMaterial* AddUniqueMaterial(FGLTFConvertBuilder& Builder, const UMaterialInterface* Material, const UMeshComponent* MeshComponent, int32 MaterialIndex);
};
