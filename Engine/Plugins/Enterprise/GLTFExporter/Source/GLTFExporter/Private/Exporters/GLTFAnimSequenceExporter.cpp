// Copyright Epic Games, Inc. All Rights Reserved.

#include "Exporters/GLTFAnimSequenceExporter.h"
#include "Exporters/GLTFExporterUtilities.h"
#include "Builders/GLTFContainerBuilder.h"
#include "Animation/AnimSequence.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GLTFAnimSequenceExporter)

UGLTFAnimSequenceExporter::UGLTFAnimSequenceExporter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UAnimSequence::StaticClass();
}

bool UGLTFAnimSequenceExporter::AddObject(FGLTFContainerBuilder& Builder, const UObject* Object)
{
	const UAnimSequence* AnimSequence = CastChecked<UAnimSequence>(Object);

	if (!Builder.ExportOptions->bExportAnimationSequences)
	{
		Builder.LogError(
			FString::Printf(TEXT("Failed to export animation sequence %s because animation sequences are disabled by export options"),
			*AnimSequence->GetName()));
		return false;
	}

	if (!Builder.ExportOptions->bExportVertexSkinWeights)
	{
		Builder.LogError(
			FString::Printf(TEXT("Failed to export animation sequence %s because vertex skin weights are disabled by export options"),
			*AnimSequence->GetName()));
		return false;
	}

	const USkeletalMesh* SkeletalMesh = FGLTFExporterUtilities::GetPreviewMesh(AnimSequence);
	if (SkeletalMesh == nullptr)
	{
		Builder.LogError(
			FString::Printf(TEXT("Failed to export animation sequence %s because of missing preview mesh"),
			*AnimSequence->GetName()));
		return false;
	}

	FGLTFJsonNode* Node = Builder.AddNode();

	if (Builder.ExportOptions->bExportPreviewMesh)
	{
		Node->Mesh = Builder.AddUniqueMesh(SkeletalMesh);
		if (Node->Mesh == nullptr)
		{
			Builder.LogError(
				FString::Printf(TEXT("Failed to export skeletal mesh %s for animation sequence %s"),
				*SkeletalMesh->GetName(),
				*AnimSequence->GetName()));
			return false;
		}
	}

	Node->Skin = Builder.AddUniqueSkin(Node, SkeletalMesh);
	if (Node->Skin == nullptr)
	{
		Builder.LogError(
			FString::Printf(TEXT("Failed to export bones in skeletal mesh %s for animation sequence %s"),
			*SkeletalMesh->GetName(),
			*AnimSequence->GetName()));
		return false;
	}

	FGLTFJsonAnimation* Animation = Builder.AddUniqueAnimation(Node, SkeletalMesh, AnimSequence);
	if (Animation == nullptr)
	{
		Builder.LogError(
			FString::Printf(TEXT("Failed to export animation sequence %s"),
			*AnimSequence->GetName()));
		return false;
	}

	FGLTFJsonScene* Scene = Builder.AddScene();
	Scene->Nodes.Add(Node);

	Builder.DefaultScene = Scene;
	return true;
}
