// Copyright Epic Games, Inc. All Rights Reserved.

#include "Exporters/GLTFExporter.h"
#include "Engine/Engine.h"
#include "Exporters/GLTFExporterUtilities.h"
#include "Exporters/GLTFExporterAnalytics.h"
#include "Misc/Paths.h"
#include "Options/GLTFExportOptions.h"
#include "UI/GLTFExportOptionsWindow.h"
#include "Builders/GLTFContainerBuilder.h"
#include "UObject/GCObjectScopeGuard.h"
#include "AssetExportTask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GLTFExporter)

#if WITH_EDITOR
#include "Editor.h"
#endif

UGLTFExporter::UGLTFExporter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = nullptr;
	bText = false;
	PreferredFormatIndex = 0;

#if GLTF_EXPORT_ENABLE
	FormatExtension.Add(TEXT("gltf"));
	FormatDescription.Add(TEXT("GL Transmission Format"));

	FormatExtension.Add(TEXT("glb"));
	FormatDescription.Add(TEXT("GL Transmission Format (Binary)"));
#endif
}

bool UGLTFExporter::ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Archive, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags)
{
	const UGLTFExportOptions* Options = GetExportOptions();
	if (Options == nullptr)
	{
		// User cancelled the export
		return false;
	}

	const uint64 StartTime = FPlatformTime::Cycles64();

	FGCObjectScopeGuard OptionsGuard(Options);

	TSet<AActor*> SelectedActors;
	if (bSelectedOnly)
	{
		FGLTFExporterUtilities::GetSelectedActors(SelectedActors);
	}

	// TODO: add support for UAssetExportTask::IgnoreObjectList?

	const FString FilePath = GetFilePath();
	const FString FileName = FPaths::GetCleanFilename(FilePath);
	const FString DirPath = FPaths::GetPath(FilePath);

	FGLTFContainerBuilder Builder(FileName, Options, SelectedActors);
	Builder.ClearLog();

	bool bSuccess = AddObject(Builder, Object);
	if (bSuccess)
	{
		Builder.ProcessSlowTasks(Warn);
		Builder.WriteInternalArchive(Archive);
		bSuccess = Builder.WriteExternalFiles(DirPath);
	}

	FGLTFExporterAnalytics(Object, Builder, true, IsAutomated(), bSuccess, StartTime).Send();

	// TODO: should we copy messages to UAssetExportTask::Errors?

	if (!FApp::IsUnattended() && Builder.HasLoggedMessages())
	{
		Builder.OpenLog();
	}

	return bSuccess;
}

bool UGLTFExporter::ExportToGLTF(UObject* Object, const FString& FilePath, const UGLTFExportOptions* Options, const TSet<AActor*>& SelectedActors, FGLTFExportMessages& OutMessages)
{
	if (Object == nullptr)
	{
		Object = GEngine->GetCurrentPlayWorld();
#if WITH_EDITOR
		if (Object == nullptr)
		{
			Object = GEditor->GetEditorWorldContext().World();
		}
#endif

		if (Object == nullptr)
		{
			OutMessages.Errors.Add(TEXT("No object to export"));
			return false;
		}
	}

	UGLTFExporter* Exporter = Cast<UGLTFExporter>(FindExporter(Object, TEXT("gltf")));
	if (Exporter == nullptr)
	{
		OutMessages.Errors.Add(FString::Printf(TEXT("Couldn't find exporter for object of type %s"), *Object->GetClass()->GetName()));
		return false;
	}

	const uint64 StartTime = FPlatformTime::Cycles64();

	const FString FileName = FPaths::GetCleanFilename(FilePath);
	const FString DirPath = FPaths::GetPath(FilePath);

	FGLTFContainerBuilder Builder(FileName, Options, SelectedActors);
	bool bSuccess = Exporter->AddObject(Builder, Object);

	OutMessages.Suggestions = Builder.GetLoggedSuggestions();
	OutMessages.Warnings = Builder.GetLoggedWarnings();
	OutMessages.Errors = Builder.GetLoggedErrors();

	if (bSuccess)
	{
		Builder.ProcessSlowTasks();
		bSuccess = Builder.WriteAllFiles(DirPath);
	}

	FGLTFExporterAnalytics(Object, Builder, false, FApp::IsUnattended(), bSuccess, StartTime).Send();

	return bSuccess;
}

bool UGLTFExporter::ExportToGLTF(UObject* Object, const FString& FilePath, const UGLTFExportOptions* Options, const TSet<AActor*>& SelectedActors)
{
	FGLTFExportMessages Messages;
	return ExportToGLTF(Object, FilePath, Options, SelectedActors, Messages);
}

bool UGLTFExporter::AddObject(FGLTFContainerBuilder& Builder, const UObject* Object)
{
	return false;
}

const UGLTFExportOptions* UGLTFExporter::GetExportOptions()
{
	UGLTFExportOptions* Options = nullptr;

	if (ExportTask != nullptr)
	{
		Options = Cast<UGLTFExportOptions>(ExportTask->Options);
	}

	if (Options == nullptr)
	{
		Options = NewObject<UGLTFExportOptions>();
	}

#if WITH_EDITOR
	if (GetShowExportOption() && !IsAutomated())
	{
		bool bExportAll = GetBatchMode();
		bool bOperationCanceled = false;

		FGCObjectScopeGuard OptionsGuard(Options);
		SGLTFExportOptionsWindow::ShowDialog(Options, CurrentFilename, GetBatchMode(), bOperationCanceled, bExportAll);

		if (bOperationCanceled)
		{
			SetCancelBatch(GetBatchMode());
			return nullptr;
		}

		SetShowExportOption(!bExportAll);
		Options->SaveConfig();
	}
#endif

	return Options;
}

FString UGLTFExporter::GetFilePath() const
{
	return ExportTask != nullptr ? ExportTask->Filename : CurrentFilename;
}

bool UGLTFExporter::IsAutomated() const
{
	// NOTE: FApp::IsUnattended() already checks GIsAutomationTesting
	return FApp::IsUnattended() || (ExportTask != nullptr && ExportTask->bAutomated);
}
