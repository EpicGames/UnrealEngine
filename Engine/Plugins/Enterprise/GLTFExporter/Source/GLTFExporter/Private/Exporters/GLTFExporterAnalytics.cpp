// Copyright Epic Games, Inc. All Rights Reserved.

#include "Exporters/GLTFExporterAnalytics.h"
#include "Builders/GLTFConvertBuilder.h"
#include "GLTFExporterModule.h"
#include "EngineAnalytics.h"
#include "Engine/World.h"

FGLTFExporterAnalytics::FGLTFExporterAnalytics(const UObject* Object, const FGLTFConvertBuilder& InBuilder, bool bInitiatedByTask, bool bExportAutomated, bool bExportSuccessful, uint64 StartTime)
	: FGLTFExporterAnalytics(Object, InBuilder, bInitiatedByTask, bExportAutomated, bExportSuccessful, StartTime, FPlatformTime::Cycles64())
{
}

FGLTFExporterAnalytics::FGLTFExporterAnalytics(const UObject* Object, const FGLTFConvertBuilder& InBuilder, bool bInitiatedByTask, bool bExportAutomated, bool bExportSuccessful, uint64 StartTime, uint64 EndTime)
	: AssetType(Object != nullptr ? Object->GetClass() : nullptr)
	, ExportOptions(InBuilder.ExportOptions)
	, bExportAsGLB(InBuilder.bIsGLB)
	, bSelectedOnly(!InBuilder.SelectedActors.IsEmpty())
	, bInitiatedByTask(bInitiatedByTask)
	, bExportAutomated(bExportAutomated)
	, bExportSuccessful(bExportSuccessful)
	, ExportDuration(FPlatformTime::ToSeconds64(EndTime - StartTime))
	, Builder(InBuilder)
{
}

void FGLTFExporterAnalytics::Send() const
{
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}

	TArray<FAnalyticsEventAttribute> EventAttributes;
	EventAttributes.Emplace(TEXT("AssetType"), AssetType != nullptr ? AssetType->GetName() : "null");
	if (AssetType == UWorld::StaticClass()) EventAttributes.Emplace(TEXT("bSelectedOnly"), bSelectedOnly);
	EventAttributes.Emplace(TEXT("bExportAsGLB"), bExportAsGLB);
	EventAttributes.Emplace(TEXT("bInitiatedByTask"), bInitiatedByTask);
	EventAttributes.Emplace(TEXT("bExportAutomated"), bExportAutomated);
	EventAttributes.Emplace(TEXT("ExportDuration"), ExportDuration);

	GetAttributesFromStruct(ExportOptions->GetClass(), ExportOptions, TEXT("ExportOptions"), EventAttributes);

	EventAttributes.Emplace(TEXT("Platform"), FPlatformProperties::IniPlatformName());
	EventAttributes.Emplace(TEXT("EngineMode"), FPlatformMisc::GetEngineMode());

	EventAttributes.Append(Builder.GenerateAnalytics());

	const FString EventName = bExportSuccessful ? TEXT("GLTFExporter.Export") : TEXT("GLTFExporter.ExportFailure");
	FEngineAnalytics::GetProvider().RecordEvent(EventName, EventAttributes);
}

void FGLTFExporterAnalytics::GetAttributesFromStruct(const UStruct* Struct, const void* ContainerPtr, const FString& AttributeName, TArray<FAnalyticsEventAttribute>& OutAttributes)
{
	for (const FProperty* Property = Struct->PropertyLink; Property != nullptr; Property = Property->PropertyLinkNext)
	{
		if (Property->HasAnyPropertyFlags(CPF_Deprecated | CPF_Transient))
		{
			continue; // Ignore deprecated and transient property
		}

#if WITH_EDITORONLY_DATA
		if (Property->HasMetaData("InlineEditConditionToggle"))
		{
			continue; // Ignore inlined toggle property
		}

		FString EditCondition = Property->GetMetaData("EditCondition");
		if (!EditCondition.IsEmpty())
		{
			if (const FBoolProperty* EditProperty = CastField<FBoolProperty>(Struct->FindPropertyByName(*EditCondition)))
			{
				const void* ValuePtr = EditProperty->ContainerPtrToValuePtr<void>(ContainerPtr);
				if (!EditProperty->GetPropertyValue(ValuePtr))
				{
					continue; // Ignore property if (simple) edit condition is not met
				}
			}
		}
#endif

		const void* ValuePtr = Property->ContainerPtrToValuePtr<void>(ContainerPtr);
		GetAttributesFromProperty(Property, ValuePtr, AttributeName + TEXT(".") + Property->GetAuthoredName(), OutAttributes);
	}
}

void FGLTFExporterAnalytics::GetAttributesFromProperty(const FProperty* Property, const void* ValuePtr, const FString& AttributeName, TArray<FAnalyticsEventAttribute>& OutAttributes)
{
	if (Property->ArrayDim != 1)
	{
		// If we get this warning, it means a new export option has been added that GLTFExporterAnalytics doesn't yet know how to serialize
		UE_LOGF(LogGLTFExporter, Warning, "GLTFExporterAnalytics doesn't support attribute %ls with array dimension %d", *AttributeName, Property->ArrayDim);
		return;
	}

	if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
	{
		OutAttributes.Emplace(AttributeName, BoolProperty->GetPropertyValue(ValuePtr));
	}
	else if (const FStrProperty* StringProperty = CastField<FStrProperty>(Property))
	{
		OutAttributes.Emplace(AttributeName, StringProperty->GetPropertyValue(ValuePtr));
	}
	else if (Property->IsA<FEnumProperty>() || Property->IsA<FNumericProperty>())
	{
		FString ValueString;
		Property->ExportTextItem_Direct(ValueString, ValuePtr, NULL, NULL, PPF_None);
		OutAttributes.Emplace(AttributeName, ValueString);
	}
	else if (const FMapProperty* MapProperty = CastField<FMapProperty>(Property))
	{
		FScriptMapHelper Helper(MapProperty, ValuePtr);
		OutAttributes.Emplace(AttributeName, Helper.Num() > 0);

		for (FScriptMapHelper::FIterator It(Helper); It; ++It)
		{
			FString KeyString;
			MapProperty->KeyProp->ExportTextItem_Direct(KeyString, Helper.GetKeyPtr(It), nullptr, nullptr, PPF_None);

			GetAttributesFromProperty(MapProperty->ValueProp, Helper.GetValuePtr(It), AttributeName + TEXT(".") + KeyString, OutAttributes);
		}
	}
	else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		GetAttributesFromStruct(StructProperty->Struct, ValuePtr, AttributeName, OutAttributes);
	}
	else
	{
		// If we get this warning, it means a new export option has been added that GLTFExporterAnalytics doesn't yet know how to serialize
		UE_LOGF(LogGLTFExporter, Warning, "GLTFExporterAnalytics doesn't support attribute %ls with property class %ls", *AttributeName, *Property->GetClass()->GetName());
	}
}
