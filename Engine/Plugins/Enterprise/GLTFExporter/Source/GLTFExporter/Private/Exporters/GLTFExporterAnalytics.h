// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UGLTFExportOptions;
class FGLTFConvertBuilder;
struct FAnalyticsEventAttribute;

struct FGLTFExporterAnalytics
{
	const UClass* AssetType;
	const UGLTFExportOptions* ExportOptions;
	bool bExportAsGLB;
	bool bSelectedOnly;
	bool bInitiatedByTask;
	bool bExportAutomated;
	bool bExportSuccessful;
	double ExportDuration;

	FGLTFExporterAnalytics(const UObject* Object, const FGLTFConvertBuilder& InBuilder, bool bInitiatedByTask, bool bExportAutomated, bool bExportSuccessful, uint64 StartTime);
	FGLTFExporterAnalytics(const UObject* Object, const FGLTFConvertBuilder& InBuilder, bool bInitiatedByTask, bool bExportAutomated, bool bExportSuccessful, uint64 StartTime, uint64 EndTime);

	void Send() const;

private:
	const FGLTFConvertBuilder& Builder;

	static void GetAttributesFromStruct(const UStruct* Struct, const void* ContainerPtr, const FString& AttributeName, TArray<FAnalyticsEventAttribute>& OutAttributes);
	static void GetAttributesFromProperty(const FProperty* Property, const void* ValuePtr, const FString& AttributeName, TArray<FAnalyticsEventAttribute>& OutAttributes);
};
