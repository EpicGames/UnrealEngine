// Copyright Epic Games, Inc. All Rights Reserved.

#include "Exporters/GLTFExporterUtilities.h"
#include "AssetRegistry/AssetData.h"
#include "Materials/MaterialInstance.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/Selection.h"
#include "Engine/World.h"
#if WITH_EDITOR
#include "Editor.h"
#endif

void FGLTFExporterUtilities::GetSelectedActors(TSet<AActor*>& OutSelectedActors)
{
#if WITH_EDITOR
	USelection* Selection = GEditor->GetSelectedActors();
	for (FSelectionIterator SelectedObject(*Selection); SelectedObject; ++SelectedObject)
	{
		if (AActor* SelectedActor = Cast<AActor>(*SelectedObject))
		{
			OutSelectedActors.Add(SelectedActor);
		}
	}
#endif
}

const UStaticMesh* FGLTFExporterUtilities::GetPreviewMesh(const UMaterialInterface* Material)
{
#if WITH_EDITORONLY_DATA
	do
	{
		const UStaticMesh* PreviewMesh = Cast<UStaticMesh>(Material->PreviewMesh.TryLoad());
		if (PreviewMesh != nullptr)
		{
			return PreviewMesh;
		}

		const UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Material);
		Material = MaterialInstance != nullptr ? MaterialInstance->Parent.Get() : nullptr;
	} while (Material != nullptr);
#endif

	static const UStaticMesh* DefaultPreviewMesh = GetSphereMesh();
	return DefaultPreviewMesh;
}

const UStaticMesh* FGLTFExporterUtilities::GetSphereMesh()
{
	const UStaticMesh* SphereMesh = LoadObject<UStaticMesh>(nullptr, TEXT("/Engine/EditorMeshes/EditorSphere.EditorSphere"));
	if (SphereMesh == nullptr)
	{
		SphereMesh = LoadObject<UStaticMesh>(nullptr, TEXT("/Engine/BasicShapes/Sphere.Sphere"));
		if (SphereMesh == nullptr)
		{
			SphereMesh = LoadObject<UStaticMesh>(nullptr, TEXT("/Engine/EngineMeshes/Sphere.Sphere"));
		}
	}

	return SphereMesh;
}

const USkeletalMesh* FGLTFExporterUtilities::GetPreviewMesh(const UAnimSequence* AnimSequence)
{
	const USkeletalMesh* PreviewMesh = AnimSequence->GetPreviewMesh();
	if (PreviewMesh == nullptr)
	{
		const USkeleton* Skeleton = AnimSequence->GetSkeleton();
		if (Skeleton != nullptr)
		{
			PreviewMesh = Skeleton->GetPreviewMesh();
			if (PreviewMesh == nullptr)
			{
				PreviewMesh = FindCompatibleMesh(Skeleton);
			}
		}
	}

	return PreviewMesh;
}

const USkeletalMesh* FGLTFExporterUtilities::FindCompatibleMesh(const USkeleton *Skeleton)
{
	const FName SkeletonMemberName = USkeletalMesh::GetSkeletonMemberName();

	FARFilter Filter;
	Filter.ClassPaths.Add(USkeletalMesh::StaticClass()->GetClassPathName());
	Filter.TagsAndValues.Add(SkeletonMemberName, FAssetData(Skeleton).GetExportTextName());

	TArray<FAssetData> FilteredAssets;
	FAssetRegistryModule::GetRegistry().GetAssets(Filter, FilteredAssets);

	for (const FAssetData& Asset : FilteredAssets)
	{
		const USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Asset.GetAsset());
		if (SkeletalMesh != nullptr)
		{
			return SkeletalMesh;
		}
	}

	return nullptr;
}

TArray<UWorld*> FGLTFExporterUtilities::GetAssociatedWorlds(const UObject* Object)
{
	TArray<UWorld*> Worlds;
	TArray<FAssetIdentifier> Dependencies;

	const FName OuterPathName = *Object->GetOutermost()->GetPathName();
	FAssetRegistryModule::GetRegistry().GetDependencies(OuterPathName, Dependencies);

	for (FAssetIdentifier& Dependency : Dependencies)
	{
		FString PackageName = Dependency.PackageName.ToString();
		UWorld* World = LoadObject<UWorld>(nullptr, *PackageName, nullptr, LOAD_NoWarn);
		if (World != nullptr)
		{
			Worlds.AddUnique(World);
		}
	}

	return Worlds;
}
