// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class AActor;
class USkeletalMesh;
class UStaticMesh;
class UMaterialInterface;
class UAnimSequence;
class USkeleton;

struct FGLTFExporterUtilities
{
	static void GetSelectedActors(TSet<AActor*>& OutSelectedActors);

	static const UStaticMesh* GetPreviewMesh(const UMaterialInterface* Material);
	static const UStaticMesh* GetSphereMesh();

	static const USkeletalMesh* GetPreviewMesh(const UAnimSequence* AnimSequence);
	static const USkeletalMesh* FindCompatibleMesh(const USkeleton* Skeleton);

	static TArray<UWorld*> GetAssociatedWorlds(const UObject* Object);
};
