// Copyright Epic Games, Inc. All Rights Reserved.

#include "Exporters/GLTFLevelExporter.h"
#include "Builders/GLTFContainerBuilder.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GLTFLevelExporter)

UGLTFLevelExporter::UGLTFLevelExporter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UWorld::StaticClass();
}

bool UGLTFLevelExporter::AddObject(FGLTFContainerBuilder& Builder, const UObject* Object)
{
	const UWorld* World = CastChecked<UWorld>(Object);

	FGLTFJsonScene* Scene = Builder.AddUniqueScene(World);
	if (Scene == nullptr)
	{
		Builder.LogError(FString::Printf(TEXT("Failed to export level %s"), *World->GetName()));
		return false;
	}

	Builder.DefaultScene = Scene;
	return true;
}
