// Copyright Epic Games, Inc. All Rights Reserved.

#include "Exporters/GLTFLevelSequenceExporter.h"
#include "Exporters/GLTFExporterUtilities.h"
#include "Builders/GLTFContainerBuilder.h"
#include "LevelSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GLTFLevelSequenceExporter)

UGLTFLevelSequenceExporter::UGLTFLevelSequenceExporter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = ULevelSequence::StaticClass();
}

bool UGLTFLevelSequenceExporter::AddObject(FGLTFContainerBuilder& Builder, const UObject* Object)
{
	const ULevelSequence* LevelSequence = CastChecked<ULevelSequence>(Object);

	if (!Builder.ExportOptions->bExportLevelSequences)
	{
		Builder.LogError(
			FString::Printf(TEXT("Failed to export level sequence %s because level sequences are disabled by export options"),
			*LevelSequence->GetName()));
		return false;
	}

	TArray<UWorld*> Worlds = FGLTFExporterUtilities::GetAssociatedWorlds(LevelSequence);
	if (Worlds.Num() == 0)
	{
		Builder.LogError(
			FString::Printf(TEXT("Failed to export level sequence %s because no associated level"),
			*LevelSequence->GetName()));
		return false;
	}

	if (Worlds.Num() > 1)
	{
		Builder.LogError(
			FString::Printf(TEXT("Failed to export level sequence %s because more than one associated level"),
			*LevelSequence->GetName()));
		return false;
	}

	const UWorld* World = Worlds[0];
	FGLTFJsonScene* Scene = Builder.AddUniqueScene(World);
	if (Scene == nullptr)
	{
		Builder.LogError(
			FString::Printf(TEXT("Failed to export level %s for level sequence %s"),
			*World->GetName(),
			*LevelSequence->GetName()));
		return false;
	}

	FGLTFJsonAnimation* Animation = Builder.AddUniqueAnimation(World->PersistentLevel, LevelSequence);
	if (Animation == nullptr)
	{
		Builder.LogError(
			FString::Printf(TEXT("Failed to export level sequence %s"),
			*LevelSequence->GetName()));
		return false;
	}

	Builder.DefaultScene = Scene;
	return true;
}
