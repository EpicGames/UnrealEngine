// Copyright Epic Games, Inc. All Rights Reserved.

#include "Exporters/GLTFLevelVariantSetsExporter.h"
#include "Exporters/GLTFExporterUtilities.h"
#include "Builders/GLTFContainerBuilder.h"
#include "Engine/World.h"
#include "LevelVariantSets.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GLTFLevelVariantSetsExporter)

UGLTFLevelVariantSetsExporter::UGLTFLevelVariantSetsExporter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = ULevelVariantSets::StaticClass();
}

bool UGLTFLevelVariantSetsExporter::AddObject(FGLTFContainerBuilder& Builder, const UObject* Object)
{
	const ULevelVariantSets* LevelVariantSets = CastChecked<ULevelVariantSets>(Object);

	if (Builder.ExportOptions->ExportMaterialVariants == EGLTFMaterialVariantMode::None)
	{
		Builder.LogError(
			FString::Printf(TEXT("Failed to export level variant sets %s because material variants are disabled by export options"),
			*LevelVariantSets->GetName()));
		return false;
	}

	TArray<UWorld*> Worlds = FGLTFExporterUtilities::GetAssociatedWorlds(LevelVariantSets);
	if (Worlds.Num() == 0)
	{
		Builder.LogError(
			FString::Printf(TEXT("Failed to export level variant sets %s because no associated level"),
			*LevelVariantSets->GetName()));
		return false;
	}

	if (Worlds.Num() > 1)
	{
		Builder.LogError(
			FString::Printf(TEXT("Failed to export level variant sets %s because more than one associated level"),
			*LevelVariantSets->GetName()));
		return false;
	}

	const UWorld* World = Worlds[0];
	FGLTFJsonScene* Scene = Builder.AddUniqueScene(World);
	if (Scene == nullptr)
	{
		Builder.LogError(
			FString::Printf(TEXT("Failed to export level %s for level variant sets %s"),
			*World->GetName(),
			*LevelVariantSets->GetName()));
		return false;
	}

	if (Builder.GetRoot().MaterialVariants.Num() < 1)
	{
		Builder.LogError(
			FString::Printf(TEXT("Failed to export any supported variant from level variant sets %s"),
			*LevelVariantSets->GetName()));
		return false;
	}

	Builder.DefaultScene = Scene;
	return true;
}
