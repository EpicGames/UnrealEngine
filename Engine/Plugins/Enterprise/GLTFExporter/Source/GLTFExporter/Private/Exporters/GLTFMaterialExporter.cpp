// Copyright Epic Games, Inc. All Rights Reserved.

#include "Exporters/GLTFMaterialExporter.h"
#include "Exporters/GLTFExporterUtilities.h"
#include "Builders/GLTFContainerBuilder.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"
#include "UObject/ConstructorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GLTFMaterialExporter)

UGLTFMaterialExporter::UGLTFMaterialExporter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMaterialInterface::StaticClass();
}

bool UGLTFMaterialExporter::AddObject(FGLTFContainerBuilder& Builder, const UObject* Object)
{
	const UMaterialInterface* Material = CastChecked<UMaterialInterface>(Object);

	if (Builder.ExportOptions->bExportPreviewMesh)
	{
		const UStaticMesh* PreviewMesh = FGLTFExporterUtilities::GetPreviewMesh(Material);
		if (PreviewMesh != nullptr)
		{
			FGLTFJsonMesh* Mesh = Builder.AddUniqueMesh(PreviewMesh, { Material });
			if (Mesh == nullptr)
			{
				Builder.LogError(
					FString::Printf(TEXT("Failed to export preview mesh %s for material %s"),
					*Material->GetName(),
					*PreviewMesh->GetName()));
				return false;
			}

			FGLTFJsonNode* Node = Builder.AddNode();
			Node->Mesh = Mesh;

			FGLTFJsonScene* Scene = Builder.AddScene();
			Scene->Nodes.Add(Node);

			Builder.DefaultScene = Scene;
		}
		else
		{
			Builder.LogError(
				FString::Printf(TEXT("Failed to export material %s because of missing preview mesh"),
				*Material->GetName()));
			return false;
		}
	}
	else
	{
		Builder.AddUniqueMaterial(Material);
	}

	return true;
}
