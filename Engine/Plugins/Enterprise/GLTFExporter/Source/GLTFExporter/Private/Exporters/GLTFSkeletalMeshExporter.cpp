// Copyright Epic Games, Inc. All Rights Reserved.

#include "Exporters/GLTFSkeletalMeshExporter.h"
#include "Builders/GLTFContainerBuilder.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GLTFSkeletalMeshExporter)

UGLTFSkeletalMeshExporter::UGLTFSkeletalMeshExporter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USkeletalMesh::StaticClass();
}

bool UGLTFSkeletalMeshExporter::AddObject(FGLTFContainerBuilder& Builder, const UObject* Object)
{
	const USkeletalMesh* SkeletalMesh = CastChecked<USkeletalMesh>(Object);

	FGLTFJsonMesh* Mesh = Builder.AddUniqueMesh(SkeletalMesh);
	if (Mesh == nullptr)
	{
		Builder.LogError(FString::Printf(TEXT("Failed to export skeletal mesh %s"), *SkeletalMesh->GetName()));
		return false;
	}

	FGLTFJsonNode* Node = Builder.AddNode();
	Node->Mesh = Mesh;

	if (Builder.ExportOptions->bExportVertexSkinWeights)
	{
		Node->Skin = Builder.AddUniqueSkin(Node, SkeletalMesh);
		if (Node->Skin == nullptr)
		{
			Builder.LogError(FString::Printf(TEXT("Failed to export bones in skeletal mesh %s"), *SkeletalMesh->GetName()));
			return false;
		}
	}

	FGLTFJsonScene* Scene = Builder.AddScene();
	Scene->Nodes.Add(Node);

	Builder.DefaultScene = Scene;
	return true;
}
