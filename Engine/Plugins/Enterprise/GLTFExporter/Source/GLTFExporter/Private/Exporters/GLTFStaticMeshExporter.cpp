// Copyright Epic Games, Inc. All Rights Reserved.

#include "Exporters/GLTFStaticMeshExporter.h"
#include "Builders/GLTFContainerBuilder.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GLTFStaticMeshExporter)

UGLTFStaticMeshExporter::UGLTFStaticMeshExporter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UStaticMesh::StaticClass();
}

bool UGLTFStaticMeshExporter::AddObject(FGLTFContainerBuilder& Builder, const UObject* Object)
{
	const UStaticMesh* StaticMesh = CastChecked<UStaticMesh>(Object);

	FGLTFJsonMesh* Mesh = Builder.AddUniqueMesh(StaticMesh);
	if (Mesh == nullptr)
	{
		Builder.LogError(FString::Printf(TEXT("Failed to export static mesh %s"), *StaticMesh->GetName()));
		return false;
	}

	FGLTFJsonNode* Node = Builder.AddNode();
	Node->Mesh = Mesh;

	FGLTFJsonScene* Scene = Builder.AddScene();
	Scene->Nodes.Add(Node);

	Builder.DefaultScene = Scene;
	return true;
}
