// Copyright Epic Games, Inc. All Rights Reserved.

#include "GLTFExporterModule.h"
#include "Utilities/GLTFProxyMaterialUtilities.h"
#include "UI/GLTFMaterialBakeSizeCustomization.h"
#include "Actions/GLTFProxyAssetActions.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "ShaderCore.h"

DEFINE_LOG_CATEGORY(LogGLTFExporter);

class FGLTFExporterModule final : public IGLTFExporterModule
{
public:

	virtual void StartupModule() override
	{
		// TODO: shaders should be moved into its own module
		const FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("GLTFExporter"))->GetBaseDir(), TEXT("Shaders"));
		AddShaderSourceDirectoryMapping(TEXT("/Plugin/GLTFExporter"), PluginShaderDir);

#if WITH_EDITOR
		// TODO: UI and editor-only functions should be moved (as much as possible) into their own module
		FCoreDelegates::GetOnPostEngineInit().AddStatic(&FGLTFMaterialBakeSizeCustomization::Register);
		FCoreDelegates::GetOnPostEngineInit().AddStatic(&FGLTFProxyAssetActions::AddMenuEntry);
#endif
	}

	virtual void ShutdownModule() override
	{
#if WITH_EDITOR
		FGLTFMaterialBakeSizeCustomization::Unregister();
		FGLTFProxyAssetActions::RemoveMenuEntry();
#endif
	}
};

IMPLEMENT_MODULE(FGLTFExporterModule, GLTFExporter);
