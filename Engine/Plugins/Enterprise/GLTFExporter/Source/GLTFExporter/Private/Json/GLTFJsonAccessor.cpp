// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonAccessor.h"
#include "Json/GLTFJsonBufferView.h"

void FGLTFJsonAccessor::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!Name.IsEmpty())
	{
		Writer.Write(TEXT("name"), Name);
	}

	Writer.Write(TEXT("bufferView"), BufferView);

	if (ByteOffset != 0)
	{
		Writer.Write(TEXT("byteOffset"), ByteOffset);
	}

	Writer.Write(TEXT("count"), Count);
	Writer.Write(TEXT("type"), Type);
	Writer.Write(TEXT("componentType"), ComponentType);

	if (bNormalized)
	{
		Writer.Write(TEXT("normalized"), bNormalized);
	}

	if (MinMaxLength > 0)
	{
		Writer.Write(TEXT("min"), Min, MinMaxLength);
		Writer.Write(TEXT("max"), Max, MinMaxLength);
	}
}
