// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonAnimation.h"
#include "Json/GLTFJsonNode.h"
#include "Json/GLTFJsonAccessor.h"

void FGLTFJsonAnimationChannelTarget::WriteObject(IGLTFJsonWriter& Writer) const
{
	Writer.Write(TEXT("node"), Node);
	Writer.Write(TEXT("path"), Path);
}

void FGLTFJsonAnimationChannel::WriteObject(IGLTFJsonWriter& Writer) const
{
	Writer.Write(TEXT("sampler"), Sampler);

	Writer.Write(TEXT("target"), Target);
}

void FGLTFJsonAnimationSampler::WriteObject(IGLTFJsonWriter& Writer) const
{
	Writer.Write(TEXT("input"), Input);
	Writer.Write(TEXT("output"), Output);

	if (Interpolation != EGLTFJsonInterpolation::Linear)
	{
		Writer.Write(TEXT("interpolation"), Interpolation);
	}
}

void FGLTFJsonAnimation::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!Name.IsEmpty())
	{
		Writer.Write(TEXT("name"), Name);
	}

	Writer.Write(TEXT("channels"), Channels);
	Writer.Write(TEXT("samplers"), Samplers);
}
