// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonArray.h"
#include "Json/GLTFJsonWriter.h"

void IGLTFJsonArray::WriteValue(IGLTFJsonWriter& Writer) const
{
	Writer.StartArray();
	WriteArray(Writer);
	Writer.EndArray();
}
