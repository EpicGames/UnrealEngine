// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonAsset.h"

void FGLTFJsonAsset::WriteObject(IGLTFJsonWriter& Writer) const
{
	Writer.Write(TEXT("version"), Version);

	if (!Generator.IsEmpty())
	{
		Writer.Write(TEXT("generator"), Generator);
	}

	if (!Copyright.IsEmpty())
	{
		Writer.Write(TEXT("copyright"), Copyright);
	}
}
