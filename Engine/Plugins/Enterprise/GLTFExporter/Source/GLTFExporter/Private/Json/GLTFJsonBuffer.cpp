// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonBuffer.h"

void FGLTFJsonBuffer::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!Name.IsEmpty())
	{
		Writer.Write(TEXT("name"), Name);
	}

	if (!URI.IsEmpty())
	{
		Writer.Write(TEXT("uri"), URI);
	}

	Writer.Write(TEXT("byteLength"), ByteLength);
}
