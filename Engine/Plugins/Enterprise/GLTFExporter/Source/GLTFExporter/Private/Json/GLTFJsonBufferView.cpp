// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonBufferView.h"
#include "Json/GLTFJsonBuffer.h"

void FGLTFJsonBufferView::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!Name.IsEmpty())
	{
		Writer.Write(TEXT("name"), Name);
	}

	Writer.Write(TEXT("buffer"), Buffer);
	Writer.Write(TEXT("byteLength"), ByteLength);

	if (ByteOffset != 0)
	{
		Writer.Write(TEXT("byteOffset"), ByteOffset);
	}

	if (ByteStride != 0)
	{
		Writer.Write(TEXT("byteStride"), ByteStride);
	}

	if (Target != EGLTFJsonBufferTarget::None)
	{
		Writer.Write(TEXT("target"), Target);
	}
}
