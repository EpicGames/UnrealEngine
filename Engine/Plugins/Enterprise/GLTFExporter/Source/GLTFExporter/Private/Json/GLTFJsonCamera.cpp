// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonCamera.h"

void FGLTFJsonOrthographic::WriteObject(IGLTFJsonWriter& Writer) const
{
	Writer.Write(TEXT("xmag"), XMag);
	Writer.Write(TEXT("ymag"), YMag);
	Writer.Write(TEXT("zfar"), ZFar);
	Writer.Write(TEXT("znear"), ZNear);
}

void FGLTFJsonPerspective::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!FMath::IsNearlyEqual(AspectRatio, 0, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("aspectRatio"), AspectRatio);
	}

	Writer.Write(TEXT("yfov"), YFov);

	if (!FMath::IsNearlyEqual(ZFar, 0, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("zfar"), ZFar);
	}

	Writer.Write(TEXT("znear"), ZNear);
}

void FGLTFJsonCamera::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!Name.IsEmpty())
	{
		Writer.Write(TEXT("name"), Name);
	}

	Writer.Write(TEXT("type"), Type);

	switch (Type)
	{
		case EGLTFJsonCameraType::Orthographic:
			Writer.Write(TEXT("orthographic"), Orthographic);
			break;

		case EGLTFJsonCameraType::Perspective:
			Writer.Write(TEXT("perspective"), Perspective);
			break;

		default:
			break;
	}
}
