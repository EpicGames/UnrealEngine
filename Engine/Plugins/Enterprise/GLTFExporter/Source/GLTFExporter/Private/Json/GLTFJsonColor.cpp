// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonColor.h"

const FGLTFJsonColor3 FGLTFJsonColor3::Black({ 0, 0, 0 });
const FGLTFJsonColor3 FGLTFJsonColor3::White({ 1, 1, 1 });

const FGLTFJsonColor4 FGLTFJsonColor4::Black({ 0, 0, 0, 1 });
const FGLTFJsonColor4 FGLTFJsonColor4::White({ 1, 1, 1, 1 });
