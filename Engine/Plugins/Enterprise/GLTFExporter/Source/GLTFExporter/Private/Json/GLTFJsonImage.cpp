// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonImage.h"
#include "Json/GLTFJsonBufferView.h"

void FGLTFJsonImage::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!Name.IsEmpty())
	{
		Writer.Write(TEXT("name"), Name);
	}

	if (!URI.IsEmpty())
	{
		Writer.Write(TEXT("uri"), URI);
	}

	if (MimeType != EGLTFJsonMimeType::None)
	{
		Writer.Write(TEXT("mimeType"), MimeType);
	}

	if (BufferView != nullptr)
	{
		Writer.Write(TEXT("bufferView"), BufferView);
	}
}
