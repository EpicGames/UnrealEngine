// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonLight.h"
#include "Json/GLTFJsonBufferView.h"

void FGLTFJsonSpotLight::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!FMath::IsNearlyEqual(InnerConeAngle, 0, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("innerConeAngle"), InnerConeAngle);
	}

	if (!FMath::IsNearlyEqual(OuterConeAngle, HALF_PI, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("outerConeAngle"), OuterConeAngle);
	}
}

void FGLTFJsonLight::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!Name.IsEmpty())
	{
		Writer.Write(TEXT("name"), Name);
	}

	Writer.Write(TEXT("type"), Type);

	if (!Color.IsNearlyEqual(FGLTFJsonColor3::White, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("color"), Color);
	}

	if (!FMath::IsNearlyEqual(Intensity, 1, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("intensity"), Intensity);
	}

	if (Type == EGLTFJsonLightType::Point || Type == EGLTFJsonLightType::Spot)
	{
		if (!FMath::IsNearlyEqual(Range, 0, Writer.DefaultTolerance))
		{
			Writer.Write(TEXT("range"), Range);
		}

		if (Type == EGLTFJsonLightType::Spot)
		{
			Writer.Write(TEXT("spot"), Spot);
		}
	}
}

void FGLTFJsonLightIES::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!Name.IsEmpty())
	{
		Writer.Write(TEXT("name"), Name);
	}

	if (!URI.IsEmpty())
	{
		Writer.Write(TEXT("uri"), URI);
	}

	if (BufferView && !MimeType.IsEmpty())
	{
		Writer.Write(TEXT("mimeType"), MimeType);
	}

	if (BufferView != nullptr)
	{
		Writer.Write(TEXT("bufferView"), BufferView);
	}
}

void FGLTFJsonLightIESInstance::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!FMath::IsNearlyEqual(Multiplier, 1, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("multiplier"), Multiplier);
	}

	Writer.Write(TEXT("light"), LightIES);
}