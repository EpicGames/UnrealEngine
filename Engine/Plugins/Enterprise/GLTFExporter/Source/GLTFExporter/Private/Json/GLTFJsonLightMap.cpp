// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonLightMap.h"

void FGLTFJsonLightMap::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!Name.IsEmpty())
	{
		Writer.Write(TEXT("name"), Name);
	}

	if (Texture.Index != nullptr)
	{
		Writer.Write(TEXT("texture"), Texture);
	}

	Writer.Write(TEXT("lightmapScale"), LightMapScale);
	Writer.Write(TEXT("lightmapAdd"), LightMapAdd);
	Writer.Write(TEXT("coordinateScaleBias"), CoordinateScaleBias);
}
