// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonMaterial.h"
#include "Json/GLTFJsonTexture.h"

void FGLTFJsonTextureInfo::WriteObject(IGLTFJsonWriter& Writer) const
{
	Writer.Write(TEXT("index"), Index);

	if (TexCoord != 0)
	{
		Writer.Write(TEXT("texCoord"), TexCoord);
	}

	if (!Transform.IsNearlyDefault(Writer.DefaultTolerance))
	{
		Writer.StartExtensions();
		Writer.Write(EGLTFJsonExtension::KHR_TextureTransform, Transform);
		Writer.EndExtensions();
	}
}

void FGLTFJsonNormalTextureInfo::WriteObject(IGLTFJsonWriter& Writer) const
{
	Writer.Write(TEXT("index"), Index);

	if (TexCoord != 0)
	{
		Writer.Write(TEXT("texCoord"), TexCoord);
	}

	if (!FMath::IsNearlyEqual(Scale, 1, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("scale"), Scale);
	}

	if (!Transform.IsNearlyDefault(Writer.DefaultTolerance))
	{
		Writer.StartExtensions();
		Writer.Write(EGLTFJsonExtension::KHR_TextureTransform, Transform);
		Writer.EndExtensions();
	}
}

void FGLTFJsonOcclusionTextureInfo::WriteObject(IGLTFJsonWriter& Writer) const
{
	Writer.Write(TEXT("index"), Index);

	if (TexCoord != 0)
	{
		Writer.Write(TEXT("texCoord"), TexCoord);
	}

	if (!FMath::IsNearlyEqual(Strength, 1, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("strength"), Strength);
	}

	if (!Transform.IsNearlyDefault(Writer.DefaultTolerance))
	{
		Writer.StartExtensions();
		Writer.Write(EGLTFJsonExtension::KHR_TextureTransform, Transform);
		Writer.EndExtensions();
	}
}

void FGLTFJsonPBRMetallicRoughness::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!BaseColorFactor.IsNearlyEqual(FGLTFJsonColor4::White, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("baseColorFactor"), BaseColorFactor);
	}

	if (BaseColorTexture.Index != nullptr)
	{
		Writer.Write(TEXT("baseColorTexture"), BaseColorTexture);
	}

	if (!FMath::IsNearlyEqual(MetallicFactor, 1, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("metallicFactor"), MetallicFactor);
	}

	if (!FMath::IsNearlyEqual(RoughnessFactor, 1, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("roughnessFactor"), RoughnessFactor);
	}

	if (MetallicRoughnessTexture.Index != nullptr)
	{
		Writer.Write(TEXT("metallicRoughnessTexture"), MetallicRoughnessTexture);
	}
}

void FGLTFJsonClearCoatExtension::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!FMath::IsNearlyEqual(ClearCoatFactor, 0, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("clearcoatFactor"), ClearCoatFactor);
	}

	if (ClearCoatTexture.Index != nullptr)
	{
		Writer.Write(TEXT("clearcoatTexture"), ClearCoatTexture);
	}

	if (!FMath::IsNearlyEqual(ClearCoatRoughnessFactor, 0, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("clearcoatRoughnessFactor"), ClearCoatRoughnessFactor);
	}

	if (ClearCoatRoughnessTexture.Index != nullptr)
	{
		Writer.Write(TEXT("clearcoatRoughnessTexture"), ClearCoatRoughnessTexture);
	}

	if (ClearCoatNormalTexture.Index != nullptr)
	{
		Writer.Write(TEXT("clearcoatNormalTexture"), ClearCoatNormalTexture);
	}
}

void FGLTFJsonSpecularExtension::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!FMath::IsNearlyEqual(Factor, 1, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("specularFactor"), Factor);
	}

	if (Texture.Index != nullptr)
	{
		Writer.Write(TEXT("specularTexture"), Texture);
	}
}

void FGLTFJsonIORExtension::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!FMath::IsNearlyEqual(Value, 1.5, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("ior"), Value);
	}
}

void FGLTFJsonSheenExtension::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!ColorFactor.IsNearlyEqual(FGLTFJsonColor3::Black))
	{
		Writer.Write(TEXT("sheenColorFactor"), ColorFactor);
	}

	if (ColorTexture.Index != nullptr)
	{
		Writer.Write(TEXT("sheenColorTexture"), ColorTexture);
	}

	if (!FMath::IsNearlyEqual(RoughnessFactor, 0.f))
	{
		Writer.Write(TEXT("sheenRoughnessFactor"), RoughnessFactor);
	}

	if (RoughnessTexture.Index != nullptr)
	{
		Writer.Write(TEXT("sheenRoughnessTexture"), RoughnessTexture);
	}
}

void FGLTFJsonTransmissionExtension::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!FMath::IsNearlyEqual(Factor, 0.f))
	{
		Writer.Write(TEXT("transmissionFactor"), Factor);
	}

	if (Texture.Index != nullptr)
	{
		Writer.Write(TEXT("transmissionTexture"), Texture);
	}
}

void FGLTFJsonSpecularGlossinessExtension::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!DiffuseFactor.IsNearlyEqual(FGLTFJsonColor4::White))
	{
		Writer.Write(TEXT("diffuseFactor"), DiffuseFactor);
	}

	if (DiffuseTexture.Index != nullptr)
	{
		Writer.Write(TEXT("diffuseTexture"), DiffuseTexture);
	}

	if (!SpecularFactor.IsNearlyEqual(FGLTFJsonColor3::White))
	{
		Writer.Write(TEXT("specularFactor"), SpecularFactor);
	}

	if (!FMath::IsNearlyEqual(GlossinessFactor, 1.0f))
	{
		Writer.Write(TEXT("glossinessFactor"), GlossinessFactor);
	}

	if (SpecularGlossinessTexture.Index != nullptr)
	{
		Writer.Write(TEXT("specularGlossinessTexture"), SpecularGlossinessTexture);
	}
}

void FGLTFJsonIridescenceExtension::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!FMath::IsNearlyEqual(IridescenceFactor, 0.f))
	{
		Writer.Write(TEXT("iridescenceFactor"), IridescenceFactor);
	}

	if (IridescenceTexture.Index != nullptr)
	{
		Writer.Write(TEXT("iridescenceTexture"), IridescenceTexture);
	}

	if (!FMath::IsNearlyEqual(IridescenceIOR, 1.3f))
	{
		Writer.Write(TEXT("iridescenceIor"), IridescenceIOR);
	}


	if (!FMath::IsNearlyEqual(IridescenceThicknessMinimum, 100.f))
	{
		Writer.Write(TEXT("iridescenceThicknessMinimum"), IridescenceThicknessMinimum);
	}
	
	if (!FMath::IsNearlyEqual(IridescenceThicknessMaximum, 100.f))
	{
		Writer.Write(TEXT("iridescenceThicknessMaximum"), IridescenceThicknessMaximum);
	}

	if (IridescenceThicknessTexture.Index != nullptr)
	{
		Writer.Write(TEXT("iridescenceThicknessTexture"), IridescenceThicknessTexture);
	}
}

void FGLTFJsonAnisotropyExtension::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!FMath::IsNearlyEqual(AnisotropyStrength, 0.f))
	{
		Writer.Write(TEXT("anisotropyStrength"), AnisotropyStrength);
	}

	if (!FMath::IsNearlyEqual(AnisotropyRotation, 0.f))
	{
		Writer.Write(TEXT("anisotropyRotation"), AnisotropyRotation);
	}

	if (AnisotropyTexture.Index != nullptr)
	{
		Writer.Write(TEXT("anisotropyTexture"), AnisotropyTexture);
	}
}

void FGLTFJsonMaterial::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!Name.IsEmpty())
	{
		Writer.Write(TEXT("name"), Name);
	}

	if (ShadingModel != EGLTFJsonShadingModel::None &&
		ShadingModel != EGLTFJsonShadingModel::SpecularGlossiness)
	{
		Writer.Write(TEXT("pbrMetallicRoughness"), PBRMetallicRoughness);
	}

	if (NormalTexture.Index != nullptr)
	{
		Writer.Write(TEXT("normalTexture"), NormalTexture);
	}

	if (OcclusionTexture.Index != nullptr)
	{
		Writer.Write(TEXT("occlusionTexture"), OcclusionTexture);
	}

	if (EmissiveTexture.Index != nullptr)
	{
		Writer.Write(TEXT("emissiveTexture"), EmissiveTexture);
	}

	if (!EmissiveFactor.IsNearlyEqual(FGLTFJsonColor3::Black, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("emissiveFactor"), EmissiveFactor);
	}

	if (AlphaMode != EGLTFJsonAlphaMode::Opaque)
	{
		Writer.Write(TEXT("alphaMode"), AlphaMode);

		if (AlphaMode == EGLTFJsonAlphaMode::Mask && !FMath::IsNearlyEqual(AlphaCutoff, 0.5f, Writer.DefaultTolerance))
		{
			Writer.Write(TEXT("alphaCutoff"), AlphaCutoff);
		}
	}

	if (DoubleSided)
	{
		Writer.Write(TEXT("doubleSided"), DoubleSided);
	}

	const bool HasEmissiveStrength = !FMath::IsNearlyEqual(EmissiveStrength, 1.0f, Writer.DefaultTolerance);
	if (ShadingModel == EGLTFJsonShadingModel::Unlit ||
		ShadingModel == EGLTFJsonShadingModel::ClearCoat ||
		(ShadingModel == EGLTFJsonShadingModel::Transmission && Transmission.HasValue()) ||
		(ShadingModel == EGLTFJsonShadingModel::SpecularGlossiness && PBRSpecularGlossiness.HasValue()) ||
		HasEmissiveStrength || 
		Specular.HasValue() || 
		IOR.HasValue() ||
		Sheen.HasValue() ||
		Iridescence.HasValue() ||
		Anisotropy.HasValue()
		)
	{
		Writer.StartExtensions();

		if (ShadingModel != EGLTFJsonShadingModel::Unlit && ShadingModel != EGLTFJsonShadingModel::SpecularGlossiness && Specular.HasValue())
		{
			Writer.Write(EGLTFJsonExtension::KHR_MaterialsSpecular, Specular);
		}

		if (ShadingModel != EGLTFJsonShadingModel::Unlit && ShadingModel != EGLTFJsonShadingModel::SpecularGlossiness && IOR.HasValue())
		{
			Writer.Write(EGLTFJsonExtension::KHR_MaterialsIOR, IOR);
		}

		if (ShadingModel != EGLTFJsonShadingModel::Unlit && ShadingModel != EGLTFJsonShadingModel::SpecularGlossiness && Sheen.HasValue())
		{
			Writer.Write(EGLTFJsonExtension::KHR_MaterialsSheen, Sheen);
		}

		if (ShadingModel != EGLTFJsonShadingModel::Unlit && ShadingModel != EGLTFJsonShadingModel::SpecularGlossiness && Iridescence.HasValue())
		{
			Writer.Write(EGLTFJsonExtension::KHR_MaterialsIridescence, Iridescence);
		}

		if (ShadingModel != EGLTFJsonShadingModel::Unlit && ShadingModel != EGLTFJsonShadingModel::SpecularGlossiness && Anisotropy.HasValue())
		{
			Writer.Write(EGLTFJsonExtension::KHR_MaterialsAnisotropy, Anisotropy);
		}

		if (ShadingModel == EGLTFJsonShadingModel::Unlit)
		{
			// Write empty object
			Writer.StartExtension(EGLTFJsonExtension::KHR_MaterialsUnlit);
			Writer.EndExtension();
		}
		else if (ShadingModel == EGLTFJsonShadingModel::ClearCoat)
		{
			Writer.Write(EGLTFJsonExtension::KHR_MaterialsClearCoat, ClearCoat);
		}
		else if (ShadingModel == EGLTFJsonShadingModel::Transmission && Transmission.HasValue())
		{
			Writer.Write(EGLTFJsonExtension::KHR_MaterialsTransmission, Transmission);
		}
		else if (ShadingModel == EGLTFJsonShadingModel::SpecularGlossiness && PBRSpecularGlossiness.HasValue())
		{
			Writer.Write(EGLTFJsonExtension::KHR_MaterialsSpecularGlossiness, PBRSpecularGlossiness);
		}

		if (ShadingModel != EGLTFJsonShadingModel::SpecularGlossiness &&
			ShadingModel != EGLTFJsonShadingModel::Unlit && 
			Anisotropy.HasValue())
		{

		}

		if (HasEmissiveStrength)
		{
			Writer.StartExtension(EGLTFJsonExtension::KHR_MaterialsEmissiveStrength);
			Writer.Write(TEXT("emissiveStrength"), EmissiveStrength);
			Writer.EndExtension();
		}

		Writer.EndExtensions();
	}
}
