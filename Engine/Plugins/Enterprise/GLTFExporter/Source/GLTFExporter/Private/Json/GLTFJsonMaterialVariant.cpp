// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonMaterialVariant.h"
#include "Json/GLTFJsonMaterial.h"

void FGLTFJsonMaterialVariantMapping::WriteObject(IGLTFJsonWriter& Writer) const
{
	Writer.Write(TEXT("material"), Material);
	Writer.Write(TEXT("variants"), Variants);
}

void FGLTFJsonMaterialVariant::WriteObject(IGLTFJsonWriter& Writer) const
{
	Writer.Write(TEXT("name"), Name);
}
