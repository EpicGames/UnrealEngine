// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonMatrix.h"

const FGLTFJsonMatrix2 FGLTFJsonMatrix2::Identity
({
	1, 0,
	0, 1
});

const FGLTFJsonMatrix3 FGLTFJsonMatrix3::Identity
({
	1, 0, 0,
	0, 1, 0,
	0, 0, 1
});

const FGLTFJsonMatrix4 FGLTFJsonMatrix4::Identity
({
	1, 0, 0, 0,
	0, 1, 0, 0,
	0, 0, 1, 0,
	0, 0, 0, 1
});
