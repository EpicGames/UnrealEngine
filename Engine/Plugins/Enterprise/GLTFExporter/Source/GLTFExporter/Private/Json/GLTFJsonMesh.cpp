// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonMesh.h"
#include "Json/GLTFJsonAccessor.h"
#include "Json/GLTFJsonMaterial.h"

void FGLTFJsonTarget::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (Position != nullptr) Writer.Write(TEXT("POSITION"), Position);
	if (Normal != nullptr) Writer.Write(TEXT("NORMAL"), Normal);
}

bool FGLTFJsonTarget::HasValue() const
{
	return (Position && Position->Count)
		|| (Normal && Normal->Count);
}


void FGLTFJsonAttributes::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (Position != nullptr) Writer.Write(TEXT("POSITION"), Position);
	if (Color0 != nullptr) Writer.Write(TEXT("COLOR_0"), Color0);
	if (Normal != nullptr) Writer.Write(TEXT("NORMAL"), Normal);
	if (Tangent != nullptr) Writer.Write(TEXT("TANGENT"), Tangent);

	for (int32 Index = 0; Index < TexCoords.Num(); ++Index)
	{
		const FGLTFJsonAccessor* TexCoord = TexCoords[Index];
		if (TexCoord != nullptr) Writer.Write(TEXT("TEXCOORD_") + FString::FromInt(Index), TexCoord);
	}

	for (int32 Index = 0; Index < Joints.Num(); ++Index)
	{
		const FGLTFJsonAccessor* Joint = Joints[Index];
		if (Joint != nullptr) Writer.Write(TEXT("JOINTS_") + FString::FromInt(Index), Joint);
	}

	for (int32 Index = 0; Index < Weights.Num(); ++Index)
	{
		const FGLTFJsonAccessor* Weight = Weights[Index];
		if (Weight != nullptr) Writer.Write(TEXT("WEIGHTS_") + FString::FromInt(Index), Weight);
	}
}

bool FGLTFJsonAttributes::HasValue() const
{
	return (Position && Position->Count)
		|| (Color0 && Color0->Count)
		|| (Normal && Normal->Count)
		|| (Tangent && Tangent->Count)
		|| TexCoords.Num()
		|| Joints.Num()
		|| Weights.Num();
}

void FGLTFJsonPrimitive::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!(Attributes.HasValue() && Indices != nullptr))
	{
		return;
	}

	if (Attributes.HasValue())
	{
		Writer.Write(TEXT("attributes"), Attributes);
	}

	if (Indices != nullptr)
	{
		Writer.Write(TEXT("indices"), Indices);
	}

	if (Material != nullptr)
	{
		Writer.Write(TEXT("material"), Material);
	}

	if (Mode != EGLTFJsonPrimitiveMode::Triangles)
	{
		Writer.Write(TEXT("mode"), Mode);
	}

	if (MaterialVariantMappings.Num() > 0)
	{
		Writer.StartExtensions();

		Writer.StartExtension(EGLTFJsonExtension::KHR_MaterialsVariants);
		Writer.Write(TEXT("mappings"), MaterialVariantMappings);
		Writer.EndExtension();

		Writer.EndExtensions();
	}

	if (Targets.Num() > 0)
	{
		Writer.Write(TEXT("targets"), Targets);
	}
}

bool FGLTFJsonPrimitive::HasValue() const
{
	return (Attributes.HasValue() && Indices != nullptr);
}

void FGLTFJsonMesh::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!Name.IsEmpty())
	{
		Writer.Write(TEXT("name"), Name);
	}

	Writer.SetIdentifier(TEXT("primitives"));
	Writer.StartArray();
	for (const FGLTFJsonPrimitive& Primitive : Primitives)
	{
		if (Primitive.HasValue())
		{
			Writer.Write(Primitive);
		}
	}
	Writer.EndArray();

	if (TargetNames.Num())
	{
		Writer.StartObject(TEXT("extras"));
		Writer.StartArray(TEXT("targetNames"));
		for (const FString& TargetName : TargetNames)
		{
			Writer.Write(TargetName);
		}
		Writer.EndArray();
		Writer.EndObject();
	}
}

bool FGLTFJsonMesh::HasValue() const
{
	for (const FGLTFJsonPrimitive& Primitive : Primitives)
	{
		if (Primitive.HasValue())
		{
			return true;
		}
	}
	return false;
}