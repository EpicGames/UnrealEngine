// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonNode.h"
#include "Json/GLTFJsonCamera.h"
#include "Json/GLTFJsonSkin.h"
#include "Json/GLTFJsonMesh.h"
#include "Json/GLTFJsonLight.h"
#include "Json/GLTFJsonLightMap.h"

void FGLTFJsonNode::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!Name.IsEmpty())
	{
		Writer.Write(TEXT("name"), Name);
	}

	FGLTFJsonTransform::WriteValue(Writer);

	if (Camera != nullptr)
	{
		Writer.Write(TEXT("camera"), Camera);
	}

	if (Skin != nullptr)
	{
		Writer.Write(TEXT("skin"), Skin);
	}

	if (Mesh != nullptr)
	{
		Writer.Write(TEXT("mesh"), Mesh);
	}

	if (Light != nullptr
		|| (LightIESInstance != nullptr && LightIESInstance->HasValue())
		|| LightMap != nullptr)
	{
		Writer.StartExtensions();

		if (Light != nullptr)
		{
			Writer.StartExtension(EGLTFJsonExtension::KHR_LightsPunctual);
			Writer.Write(TEXT("light"), Light);
			Writer.EndExtension();
		}

		if (LightMap != nullptr)
		{
			Writer.StartExtension(EGLTFJsonExtension::EPIC_LightmapTextures);
			Writer.Write(TEXT("lightmap"), LightMap);
			Writer.EndExtension();
		}

		
		if ((LightIESInstance != nullptr && LightIESInstance->HasValue())) {
			Writer.StartExtension(EGLTFJsonExtension::EXT_LightsIES);
			LightIESInstance->WriteObject(Writer);
			Writer.EndExtension();
		}

		Writer.EndExtensions();
	}

	if (Children.Num() > 0)
	{
		Writer.Write(TEXT("children"), Children);
	}
}
