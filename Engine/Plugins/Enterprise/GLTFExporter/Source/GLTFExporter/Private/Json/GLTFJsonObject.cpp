// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonObject.h"
#include "Json/GLTFJsonWriter.h"

void IGLTFJsonObject::WriteValue(IGLTFJsonWriter& Writer) const
{
	Writer.StartObject();
	WriteObject(Writer);
	Writer.EndObject();
}
