// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonQuaternion.h"

const FGLTFQuaternion FGLTFJsonQuaternion::Identity = { 0, 0, 0, 1 };
