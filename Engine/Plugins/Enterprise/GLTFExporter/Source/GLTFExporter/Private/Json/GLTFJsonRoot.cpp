// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonRoot.h"

void FGLTFJsonRoot::WriteObject(IGLTFJsonWriter& Writer) const
{
	Writer.Write(TEXT("asset"), Asset);

	if (DefaultScene != nullptr)
	{
		Writer.Write(TEXT("scene"), DefaultScene);
	}

	if (Accessors.Num() > 0) Writer.Write(TEXT("accessors"), Accessors);
	if (Animations.Num() > 0) Writer.Write(TEXT("animations"), Animations);
	if (Buffers.Num() > 0) Writer.Write(TEXT("buffers"), Buffers);
	if (BufferViews.Num() > 0) Writer.Write(TEXT("bufferViews"), BufferViews);
	if (Cameras.Num() > 0) Writer.Write(TEXT("cameras"), Cameras);
	if (Images.Num() > 0) Writer.Write(TEXT("images"), Images);
	if (Materials.Num() > 0) Writer.Write(TEXT("materials"), Materials);
	if (Meshes.Num() > 0) Writer.Write(TEXT("meshes"), Meshes);
	if (Nodes.Num() > 0) Writer.Write(TEXT("nodes"), Nodes);
	if (Samplers.Num() > 0) Writer.Write(TEXT("samplers"), Samplers);
	if (Scenes.Num() > 0) Writer.Write(TEXT("scenes"), Scenes);
	if (Skins.Num() > 0) Writer.Write(TEXT("skins"), Skins);
	if (Textures.Num() > 0) Writer.Write(TEXT("textures"), Textures);

	if (MaterialVariants.Num() > 0
		|| Lights.Num() > 0 
		|| LightMaps.Num() > 0
		|| LightIESs.Num() > 0)
	{
		Writer.StartExtensions();

		if (MaterialVariants.Num() > 0)
		{
			Writer.StartExtension(EGLTFJsonExtension::KHR_MaterialsVariants);
			Writer.Write(TEXT("variants"), MaterialVariants);
			Writer.EndExtension();
		}

		if (Lights.Num() > 0)
		{
			Writer.StartExtension(EGLTFJsonExtension::KHR_LightsPunctual);
			Writer.Write(TEXT("lights"), Lights);
			Writer.EndExtension();
		}

		if (LightMaps.Num() > 0)
		{
			Writer.StartExtension(EGLTFJsonExtension::EPIC_LightmapTextures);
			Writer.Write(TEXT("lightmaps"), LightMaps);
			Writer.EndExtension();
		}

		if (LightIESs.Num() > 0)
		{
			Writer.StartExtension(EGLTFJsonExtension::EXT_LightsIES);
			Writer.Write(TEXT("lights"), LightIESs);
			Writer.EndExtension();
		}

		Writer.EndExtensions();
	}

	if (Extensions.Used.Num() > 0)
	{
		Writer.Write(TEXT("extensionsUsed"), Extensions.Used);
	}

	if (Extensions.Required.Num() > 0)
	{
		Writer.Write(TEXT("extensionsRequired"), Extensions.Required);
	}
}

void FGLTFJsonRoot::WriteJson(FArchive& Archive, bool bPrettyJson, float DefaultTolerance)
{
	const TSharedRef<IGLTFJsonWriter> Writer = IGLTFJsonWriter::Create(Archive, bPrettyJson, Extensions);
	Writer->DefaultTolerance = DefaultTolerance;
	Writer->Write(*this);
	Writer->Close();
}
