// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonSampler.h"

void FGLTFJsonSampler::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!Name.IsEmpty())
	{
		Writer.Write(TEXT("name"), Name);
	}

	if (MinFilter != EGLTFJsonTextureFilter::None)
	{
		Writer.Write(TEXT("minFilter"), MinFilter);
	}

	if (MagFilter != EGLTFJsonTextureFilter::None)
	{
		Writer.Write(TEXT("magFilter"), MagFilter);
	}

	if (WrapS != EGLTFJsonTextureWrap::Repeat)
	{
		Writer.Write(TEXT("wrapS"), WrapS);
	}

	if (WrapT != EGLTFJsonTextureWrap::Repeat)
	{
		Writer.Write(TEXT("wrapT"), WrapT);
	}
}
