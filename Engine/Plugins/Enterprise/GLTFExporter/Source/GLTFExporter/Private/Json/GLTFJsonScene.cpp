// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonScene.h"
#include "Json/GLTFJsonNode.h"

void FGLTFJsonScene::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!Name.IsEmpty())
	{
		Writer.Write(TEXT("name"), Name);
	}

	if (Nodes.Num() > 0)
	{
		Writer.Write(TEXT("nodes"), Nodes);
	}
}
