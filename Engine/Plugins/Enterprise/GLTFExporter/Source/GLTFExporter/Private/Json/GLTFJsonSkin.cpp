// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonSkin.h"
#include "Json/GLTFJsonAccessor.h"
#include "Json/GLTFJsonNode.h"

void FGLTFJsonSkin::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!Name.IsEmpty())
	{
		Writer.Write(TEXT("name"), Name);
	}

	if (InverseBindMatrices != nullptr)
	{
		Writer.Write(TEXT("inverseBindMatrices"), InverseBindMatrices);
	}

	if (Skeleton != nullptr)
	{
		Writer.Write(TEXT("skeleton"), Skeleton);
	}

	if (Joints.Num() > 0)
	{
		Writer.Write(TEXT("joints"), Joints);
	}
}
