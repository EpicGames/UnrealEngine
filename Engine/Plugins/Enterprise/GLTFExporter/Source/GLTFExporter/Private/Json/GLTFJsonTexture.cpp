// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonTexture.h"
#include "Json/GLTFJsonSampler.h"
#include "Json/GLTFJsonImage.h"

void FGLTFJsonTexture::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!Name.IsEmpty())
	{
		Writer.Write(TEXT("name"), Name);
	}

	if (Sampler != nullptr)
	{
		Writer.Write(TEXT("sampler"), Sampler);
	}

	if (Source != nullptr)
	{
		Writer.Write(TEXT("source"), Source);
	}
}
