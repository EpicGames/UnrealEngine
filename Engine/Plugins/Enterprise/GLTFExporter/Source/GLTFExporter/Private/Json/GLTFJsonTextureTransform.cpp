// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonTextureTransform.h"

void FGLTFJsonTextureTransform::WriteObject(IGLTFJsonWriter& Writer) const
{
	if (!Offset.IsNearlyEqual(FGLTFJsonVector2::Zero, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("offset"), Offset);
	}

	if (!Scale.IsNearlyEqual(FGLTFJsonVector2::One, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("scale"), Scale);
	}

	if (!FMath::IsNearlyEqual(Rotation, 0, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("rotation"), Rotation);
	}
}

bool FGLTFJsonTextureTransform::IsNearlyEqual(const FGLTFJsonTextureTransform& Other, float Tolerance) const
{
	return Offset.IsNearlyEqual(Other.Offset, Tolerance)
		&& Scale.IsNearlyEqual(Other.Scale, Tolerance)
		&& FMath::IsNearlyEqual(Rotation, Other.Rotation, Tolerance);
}

bool FGLTFJsonTextureTransform::IsExactlyEqual(const FGLTFJsonTextureTransform& Other) const
{
	return Offset.X == Other.Offset.X && Offset.Y == Other.Offset.Y
		&& Scale.X == Other.Scale.X && Scale.Y == Other.Scale.Y
		&& Rotation == Other.Rotation;
}

bool FGLTFJsonTextureTransform::IsNearlyDefault(float Tolerance) const
{
	return IsNearlyEqual({}, Tolerance);
}

bool FGLTFJsonTextureTransform::IsExactlyDefault() const
{
	return IsExactlyEqual({});
}
