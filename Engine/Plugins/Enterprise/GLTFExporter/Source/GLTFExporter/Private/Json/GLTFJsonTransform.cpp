// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonTransform.h"

const FGLTFJsonTransform FGLTFJsonTransform::Identity;

void FGLTFJsonTransform::WriteValue(IGLTFJsonWriter& Writer) const
{
	if (!Translation.IsNearlyEqual(FGLTFJsonVector3::Zero, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("translation"), Translation);
	}

	if (!Rotation.IsNearlyEqual(FGLTFJsonQuaternion::Identity, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("rotation"), Rotation);
	}

	if (!Scale.IsNearlyEqual(FGLTFJsonVector3::One, Writer.DefaultTolerance))
	{
		Writer.Write(TEXT("scale"), Scale);
	}
}
