// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonVector.h"

const FGLTFJsonVector2 FGLTFJsonVector2::Zero({ 0, 0 });
const FGLTFJsonVector2 FGLTFJsonVector2::One ({ 1, 1 });

const FGLTFJsonVector3 FGLTFJsonVector3::Zero({ 0, 0, 0 });
const FGLTFJsonVector3 FGLTFJsonVector3::One ({ 1, 1, 1 });

const FGLTFJsonVector4 FGLTFJsonVector4::Zero({ 0, 0, 0, 0 });
const FGLTFJsonVector4 FGLTFJsonVector4::One ({ 1, 1, 1, 1 });
