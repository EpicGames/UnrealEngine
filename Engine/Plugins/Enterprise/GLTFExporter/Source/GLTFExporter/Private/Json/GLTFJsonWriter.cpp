// Copyright Epic Games, Inc. All Rights Reserved.

#include "Json/GLTFJsonWriter.h"
#include "Serialization/JsonSerializer.h"

template <class CharType, class PrintPolicy>
class TGLTFJsonWriterImpl final : public IGLTFJsonWriter
{
public:

	TGLTFJsonWriterImpl(FArchive& Archive, FGLTFJsonExtensions& Extensions)
		: IGLTFJsonWriter(Extensions)
		, JsonWriter(TJsonWriterFactory<CharType, PrintPolicy>::Create(&Archive))
	{
	}

	virtual void Close() override
	{
		JsonWriter->Close();
	}

	virtual void Write(bool Boolean) override
	{
		if (CurrentIdentifier.IsEmpty())
		{
			JsonWriter->WriteValue(Boolean);
		}
		else
		{
			JsonWriter->WriteValue(CurrentIdentifier, Boolean);
			CurrentIdentifier.Empty();
		}
	}

	virtual void Write(int32 Number) override
	{
		if (CurrentIdentifier.IsEmpty())
		{
			JsonWriter->WriteValue(Number);
		}
		else
		{
			JsonWriter->WriteValue(CurrentIdentifier, Number);
			CurrentIdentifier.Empty();
		}
	}

	virtual void Write(int64 Number) override
	{
		if (CurrentIdentifier.IsEmpty())
		{
			JsonWriter->WriteValue(Number);
		}
		else
		{
			JsonWriter->WriteValue(CurrentIdentifier, Number);
			CurrentIdentifier.Empty();
		}
	}

	virtual void Write(float Number) override
	{
		if (CurrentIdentifier.IsEmpty())
		{
			JsonWriter->WriteValue(Number);
		}
		else
		{
			JsonWriter->WriteValue(CurrentIdentifier, Number);
			CurrentIdentifier.Empty();
		}
	}

	virtual void Write(const FString& String) override
	{
		if (CurrentIdentifier.IsEmpty())
		{
			JsonWriter->WriteValue(String);
		}
		else
		{
			JsonWriter->WriteValue(CurrentIdentifier, String);
			CurrentIdentifier.Empty();
		}
	}

	virtual void Write(TYPE_OF_NULLPTR) override
	{
		if (CurrentIdentifier.IsEmpty())
		{
			JsonWriter->WriteValue(nullptr);
		}
		else
		{
			JsonWriter->WriteValue(CurrentIdentifier, nullptr);
			CurrentIdentifier.Empty();
		}
	}

	virtual void SetIdentifier(const FString& Identifier) override
	{
		this->CurrentIdentifier = Identifier;
	}

	virtual void StartObject() override
	{
		if (CurrentIdentifier.IsEmpty())
		{
			JsonWriter->WriteObjectStart();
		}
		else
		{
			JsonWriter->WriteObjectStart(CurrentIdentifier);
			CurrentIdentifier.Empty();
		}
	}

	virtual void EndObject() override
	{
		JsonWriter->WriteObjectEnd();
	}

	virtual void StartArray() override
	{
		if (CurrentIdentifier.IsEmpty())
		{
			JsonWriter->WriteArrayStart();
		}
		else
		{
			JsonWriter->WriteArrayStart(CurrentIdentifier);
			CurrentIdentifier.Empty();
		}
	}

	virtual void EndArray() override
	{
		JsonWriter->WriteArrayEnd();
	}

private:

	FString CurrentIdentifier;
	TSharedRef<TJsonWriter<CharType, PrintPolicy>> JsonWriter;
};

struct TGLTFJsonPrintPolicyUTF8
{
	static void WriteChar(FArchive* Stream, UTF8CHAR Char)
	{
		Stream->Serialize(&Char, sizeof(Char));
	}

	static void WriteChar(FArchive* Stream, TCHAR Char)
	{
		ANSICHAR Buffer[8] = { 0 };
		const int32 Length = FTCHARToUTF8_Convert::Utf8FromCodepoint(Char, Buffer, UE_ARRAY_COUNT(Buffer));
		Stream->Serialize(Buffer, Length);
	}

	template <typename CharType>
	static void WriteStringView(FArchive* Stream, TStringView<CharType> String)
	{
		const CharType* CharPtr = String.GetData();

		for (int32 CharIndex = 0; CharIndex < String.Len(); ++CharIndex, ++CharPtr)
		{
			WriteChar(Stream, *CharPtr);
		}
	}

	static void WriteString(FArchive* Stream, FStringView String)
	{
		WriteStringView(Stream, String);
	}

	static void WriteString(FArchive* Stream, FUtf8StringView String)
	{
		WriteStringView(Stream, String);
	}

	static void WriteFloat(FArchive* Stream, float Number)
	{
		WriteString(Stream, FString::Printf(TEXT("%.9g"), Number));
	}

	static void WriteDouble(FArchive* Stream, double Number)
	{
		WriteString(Stream, FString::Printf(TEXT("%.17g"), Number));
	}
};

struct TGLTFCondensedJsonPrintPolicyUTF8 : TGLTFJsonPrintPolicyUTF8
{
	static void WriteLineTerminator(FArchive* Stream) {}
	static void WriteTabs(FArchive* Stream, int32 Count) {}
	static void WriteSpace(FArchive* Stream) {}
};

struct TGLTFPrettyJsonPrintPolicyUTF8 : TGLTFJsonPrintPolicyUTF8
{
	static void WriteLineTerminator(FArchive* Stream)
	{
		WriteChar(Stream, static_cast<UTF8CHAR>('\n')); // don't use system dependent line terminator
	}

	static void WriteTabs( FArchive* Stream, int32 Count )
	{
		for (int32 i = 0; i < Count; ++i)
		{
			WriteChar(Stream, static_cast<UTF8CHAR>('\t'));
		}
	}

	static void WriteSpace( FArchive* Stream )
	{
		WriteChar(Stream, static_cast<UTF8CHAR>(' '));
	}
};

TSharedRef<IGLTFJsonWriter> IGLTFJsonWriter::Create(FArchive& Archive, bool bPrettyJson, FGLTFJsonExtensions& Extensions)
{
	return MakeShareable(bPrettyJson
		? static_cast<IGLTFJsonWriter*>(new TGLTFJsonWriterImpl<UTF8CHAR, TGLTFPrettyJsonPrintPolicyUTF8>(Archive, Extensions))
		: static_cast<IGLTFJsonWriter*>(new TGLTFJsonWriterImpl<UTF8CHAR, TGLTFCondensedJsonPrintPolicyUTF8>(Archive, Extensions))
	);
}
