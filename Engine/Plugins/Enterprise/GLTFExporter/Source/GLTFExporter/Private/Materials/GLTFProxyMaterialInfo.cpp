// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/GLTFProxyMaterialInfo.h"

const FGLTFProxyMaterialTextureParameterInfo FGLTFProxyMaterialInfo::BaseColor = { TEXT("Base Color") };
const TGLTFProxyMaterialParameterInfo<FLinearColor> FGLTFProxyMaterialInfo::BaseColorFactor = { TEXT("Base Color Factor") };

const FGLTFProxyMaterialTextureParameterInfo FGLTFProxyMaterialInfo::Emissive = { TEXT("Emissive") };
const TGLTFProxyMaterialParameterInfo<FLinearColor> FGLTFProxyMaterialInfo::EmissiveFactor = { TEXT("Emissive Factor") };
const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::EmissiveStrength = { TEXT("Emissive Strength") };

const FGLTFProxyMaterialTextureParameterInfo FGLTFProxyMaterialInfo::MetallicRoughness = { TEXT("Metallic Roughness") };
const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::MetallicFactor = { TEXT("Metallic Factor") };
const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::RoughnessFactor = { TEXT("Roughness Factor") };

const FGLTFProxyMaterialTextureParameterInfo FGLTFProxyMaterialInfo::Normal = { TEXT("Normal") };
const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::NormalScale = { TEXT("Normal Scale") };

const FGLTFProxyMaterialTextureParameterInfo FGLTFProxyMaterialInfo::Occlusion = { TEXT("Occlusion") };
const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::OcclusionStrength = { TEXT("Occlusion Strength") };

const FGLTFProxyMaterialTextureParameterInfo FGLTFProxyMaterialInfo::ClearCoat = { TEXT("Clear Coat") };
const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::ClearCoatFactor = { TEXT("Clear Coat Factor") };

const FGLTFProxyMaterialTextureParameterInfo FGLTFProxyMaterialInfo::ClearCoatRoughness = { TEXT("Clear Coat Roughness") };
const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::ClearCoatRoughnessFactor = { TEXT("Clear Coat Roughness Factor") };

const FGLTFProxyMaterialTextureParameterInfo FGLTFProxyMaterialInfo::ClearCoatNormal = { TEXT("Clear Coat Normal") };
const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::ClearCoatNormalScale = { TEXT("Clear Coat Normal Scale") };

const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::SpecularFactor = { TEXT("Specular Factor") };
const FGLTFProxyMaterialTextureParameterInfo FGLTFProxyMaterialInfo::SpecularTexture = { TEXT("Specular") };

const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::IOR = { TEXT("IOR") };

const TGLTFProxyMaterialParameterInfo<FLinearColor> FGLTFProxyMaterialInfo::SheenColorFactor = { TEXT("Sheen Color Factor") };
const FGLTFProxyMaterialTextureParameterInfo FGLTFProxyMaterialInfo::SheenColorTexture = { TEXT("Sheen Color") };
const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::SheenRoughnessFactor = { TEXT("Sheen Roughness Factor") };
const FGLTFProxyMaterialTextureParameterInfo FGLTFProxyMaterialInfo::SheenRoughnessTexture = { TEXT("Sheen Roughness") };

const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::TransmissionFactor = { TEXT("Transmission Factor") };
const FGLTFProxyMaterialTextureParameterInfo FGLTFProxyMaterialInfo::TransmissionTexture = { TEXT("Transmission") };

const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::IridescenceFactor = { TEXT("Iridescence Factor") };
const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::IridescenceIOR = { TEXT("Iridescence IOR") };
const FGLTFProxyMaterialTextureParameterInfo FGLTFProxyMaterialInfo::IridescenceTexture = { TEXT("Iridescence") };
const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::IridescenceThicknessMinimum = { TEXT("IridescenceThickness Minimum") };
const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::IridescenceThicknessMaximum = { TEXT("IridescenceThickness Maximum") };
const FGLTFProxyMaterialTextureParameterInfo FGLTFProxyMaterialInfo::IridescenceThicknessTexture = { TEXT("IridescenceThickness") };

const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::AnisotropyStrength = { TEXT("Anisotropy Strength") };
const TGLTFProxyMaterialParameterInfo<float> FGLTFProxyMaterialInfo::AnisotropyRotation = { TEXT("Anisotropy Rotation") };
const FGLTFProxyMaterialTextureParameterInfo FGLTFProxyMaterialInfo::AnisotropyTexture = { TEXT("Anisotropy") };
