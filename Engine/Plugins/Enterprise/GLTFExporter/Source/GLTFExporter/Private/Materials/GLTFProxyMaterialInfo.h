// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/GLTFProxyMaterialParameterInfo.h"

#define UE_API GLTFEXPORTER_API

class FGLTFProxyMaterialInfo
{
public:

	static UE_API const FGLTFProxyMaterialTextureParameterInfo BaseColor;
	static UE_API const TGLTFProxyMaterialParameterInfo<FLinearColor> BaseColorFactor;

	static UE_API const FGLTFProxyMaterialTextureParameterInfo Emissive;
	static UE_API const TGLTFProxyMaterialParameterInfo<FLinearColor> EmissiveFactor;
	static UE_API const TGLTFProxyMaterialParameterInfo<float> EmissiveStrength;

	static UE_API const FGLTFProxyMaterialTextureParameterInfo MetallicRoughness;
	static UE_API const TGLTFProxyMaterialParameterInfo<float> MetallicFactor;
	static UE_API const TGLTFProxyMaterialParameterInfo<float> RoughnessFactor;

	static UE_API const FGLTFProxyMaterialTextureParameterInfo Normal;
	static UE_API const TGLTFProxyMaterialParameterInfo<float> NormalScale;

	static UE_API const FGLTFProxyMaterialTextureParameterInfo Occlusion;
	static UE_API const TGLTFProxyMaterialParameterInfo<float> OcclusionStrength;

	static UE_API const FGLTFProxyMaterialTextureParameterInfo ClearCoat;
	static UE_API const TGLTFProxyMaterialParameterInfo<float> ClearCoatFactor;

	static UE_API const FGLTFProxyMaterialTextureParameterInfo ClearCoatRoughness;
	static UE_API const TGLTFProxyMaterialParameterInfo<float> ClearCoatRoughnessFactor;

	static UE_API const FGLTFProxyMaterialTextureParameterInfo ClearCoatNormal;
	static UE_API const TGLTFProxyMaterialParameterInfo<float> ClearCoatNormalScale;
	
	static UE_API const TGLTFProxyMaterialParameterInfo<float> SpecularFactor;
	static UE_API const FGLTFProxyMaterialTextureParameterInfo SpecularTexture; //Only using Alpha Channel

	static UE_API const TGLTFProxyMaterialParameterInfo<float> IOR;

	static UE_API const TGLTFProxyMaterialParameterInfo<FLinearColor> SheenColorFactor;
	static UE_API const FGLTFProxyMaterialTextureParameterInfo        SheenColorTexture; //RGB
	static UE_API const TGLTFProxyMaterialParameterInfo<float>        SheenRoughnessFactor;
	static UE_API const FGLTFProxyMaterialTextureParameterInfo        SheenRoughnessTexture; //A

	static UE_API const TGLTFProxyMaterialParameterInfo<float> TransmissionFactor;
	static UE_API const FGLTFProxyMaterialTextureParameterInfo TransmissionTexture; //Only using Red Channel

	static UE_API const TGLTFProxyMaterialParameterInfo<float> IridescenceFactor;
	static UE_API const TGLTFProxyMaterialParameterInfo<float> IridescenceIOR;
	static UE_API const FGLTFProxyMaterialTextureParameterInfo IridescenceTexture;
	static UE_API const TGLTFProxyMaterialParameterInfo<float> IridescenceThicknessMinimum;
	static UE_API const TGLTFProxyMaterialParameterInfo<float> IridescenceThicknessMaximum;
	static UE_API const FGLTFProxyMaterialTextureParameterInfo IridescenceThicknessTexture;

	static UE_API const TGLTFProxyMaterialParameterInfo<float> AnisotropyStrength;
	static UE_API const TGLTFProxyMaterialParameterInfo<float> AnisotropyRotation;
	static UE_API const FGLTFProxyMaterialTextureParameterInfo AnisotropyTexture;
};

#undef UE_API
