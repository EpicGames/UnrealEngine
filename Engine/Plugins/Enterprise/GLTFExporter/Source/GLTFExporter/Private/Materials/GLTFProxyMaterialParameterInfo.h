// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Utilities/GLTFProxyMaterialUtilities.h"

class UMaterialInstanceConstant;
class UMaterialInterface;
class UTexture;

template <typename ParameterType>
class TGLTFProxyMaterialParameterInfo
{
public:

	TGLTFProxyMaterialParameterInfo(const FString& ParameterName)
		: ParameterInfo(*ParameterName)
	{
	}

	bool Get(const UMaterialInterface* Material, ParameterType& OutValue, bool NonDefaultOnly = false) const
	{
		return FGLTFProxyMaterialUtilities::GetParameterValue(Material, ParameterInfo, OutValue, NonDefaultOnly);
	}

#if WITH_EDITOR
	void Set(UMaterialInstanceConstant* Material, const ParameterType& Value, bool NonDefaultOnly = false) const
	{
		FGLTFProxyMaterialUtilities::SetParameterValue(Material, ParameterInfo, Value, NonDefaultOnly);
	}
#endif

	FString ToString() const
	{
		return ParameterInfo.Name.ToString();
	}

	bool operator==(const TGLTFProxyMaterialParameterInfo<ParameterType>& Other) const
	{
		return ParameterInfo.Name == Other.ParameterInfo.Name
			&& ParameterInfo.Index == Other.ParameterInfo.Index
			&& ParameterInfo.Association == Other.ParameterInfo.Association;
	}

	bool operator!=(const TGLTFProxyMaterialParameterInfo<ParameterType>& Other) const
	{
		return !(*this == Other);
	}

private:

	FHashedMaterialParameterInfo ParameterInfo;
};

class FGLTFProxyMaterialTextureParameterInfo
{
public:

	FGLTFProxyMaterialTextureParameterInfo(const FString& ParameterName)
		: Texture(ParameterName + TEXT(" Texture"))
		, UVIndex(ParameterName + TEXT(" UV Index"))
		, UVOffset(ParameterName + TEXT(" UV Offset"))
		, UVScale(ParameterName + TEXT(" UV Scale"))
		, UVRotation(ParameterName + TEXT(" UV Rotation"))
		, TilingMethod(ParameterName + TEXT(" Tiling Method"))
	{
	}

	bool operator==(const FGLTFProxyMaterialTextureParameterInfo& Other) const
	{
		return Texture == Other.Texture;
	}

	bool operator!=(const FGLTFProxyMaterialTextureParameterInfo& Other) const
	{
		return !(*this == Other);
	}

	TGLTFProxyMaterialParameterInfo<UTexture*> Texture;
	TGLTFProxyMaterialParameterInfo<float> UVIndex;
	TGLTFProxyMaterialParameterInfo<FLinearColor> UVOffset;
	TGLTFProxyMaterialParameterInfo<FLinearColor> UVScale;
	TGLTFProxyMaterialParameterInfo<float> UVRotation;
	TGLTFProxyMaterialParameterInfo<FLinearColor> TilingMethod;
};
