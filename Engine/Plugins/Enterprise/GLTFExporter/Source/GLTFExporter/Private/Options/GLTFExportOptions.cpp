// Copyright Epic Games, Inc. All Rights Reserved.

#include "Options/GLTFExportOptions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GLTFExportOptions)

UGLTFExportOptions::UGLTFExportOptions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ResetToDefault();
}

void UGLTFExportOptions::ResetToDefault()
{
	ExportUniformScale = 0.01f;
	bExportPreviewMesh = true;
	bSkipNearDefaultValues = true;
	bIncludeCopyrightNotice = false;
	bExportProxyMaterials = true;
	bUseImporterMaterialMapping = true;
	bExportUnlitMaterials = true;
	bExportClearCoatMaterials = true;
	bExportClothMaterials = true;
	bExportThinTranslucentMaterials = false;
	bExportEmissiveStrength = true;
	bExportSpecularGlossinessMaterials = true;
	BakeMaterialInputs = EGLTFMaterialBakeMode::UseMeshData;
	DefaultMaterialBakeSize = FGLTFMaterialBakeSize::Default;
	DefaultMaterialBakeFilter = TF_Trilinear;
	DefaultMaterialBakeTiling = TA_Wrap;
	DefaultLevelOfDetail = 0;
	bExportSourceModel = false;
	bExportVertexColors = false;
	bExportVertexSkinWeights = true;
	bMakeSkinnedMeshesRoot = true;
	bUseMeshQuantization = false;
	bExportMorphTargets = true;
	bExportLevelSequences = true;
	bExportAnimationSequences = true;
	TextureImageFormat = EGLTFTextureImageFormat::PNG;
	TextureImageQuality = 0;
	bExportTextureTransforms = true;
	bExportLightmaps = false;
	bAdjustNormalmaps = true;
	bExportHiddenInGame = false;
	bExportLights = true;
	bExportCameras = true;
	ExportMaterialVariants = EGLTFMaterialVariantMode::UseMeshData;
}
