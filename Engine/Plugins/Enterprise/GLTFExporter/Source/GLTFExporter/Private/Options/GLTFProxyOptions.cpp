// Copyright Epic Games, Inc. All Rights Reserved.

#include "Options/GLTFProxyOptions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GLTFProxyOptions)

UGLTFProxyOptions::UGLTFProxyOptions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ResetToDefault();
}

void UGLTFProxyOptions::ResetToDefault()
{
	bBakeMaterialInputs = true;
	bUseThinTranslucentShadingModel = false;
	DefaultMaterialBakeSize = FGLTFMaterialBakeSize::Default;
	DefaultMaterialBakeFilter = TF_Trilinear;
	DefaultMaterialBakeTiling = TA_Wrap;
}
