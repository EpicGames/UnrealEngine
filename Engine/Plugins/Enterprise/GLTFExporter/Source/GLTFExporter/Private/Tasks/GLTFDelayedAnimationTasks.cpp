// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/GLTFDelayedAnimationTasks.h"
#include "Builders/GLTFContainerBuilder.h"
#include "Utilities/GLTFCoreUtilities.h"
#include "Converters/GLTFBoneUtilities.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/Level.h"
#include "LevelSequence.h"
#include "LevelSequenceActor.h"
#include "LevelSequencePlayer.h"
#include "MovieSceneTimeHelpers.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Channels/MovieSceneChannelProxy.h"

using namespace UE;

void FGLTFDelayedAnimSequenceTask::Process()
{
	// TODO: bone transforms should be absolute (not relative) according to gltf spec

	JsonAnimation->Name = AnimSequence->GetName() + TEXT("_") + FString::FromInt(JsonAnimation->Index); // Ensure unique name due to limitation in certain gltf viewers

	TArray<float> Timestamps;
	FGLTFBoneUtilities::GetFrameTimestamps(AnimSequence, Timestamps);
	const int32 FrameCount = Timestamps.Num();

	// TODO: add animation data accessor converters to reuse track information

	FGLTFJsonAccessor* JsonInputAccessor = Builder.AddAccessor();
	JsonInputAccessor->BufferView = Builder.AddBufferView(Timestamps);
	JsonInputAccessor->ComponentType = EGLTFJsonComponentType::Float;
	JsonInputAccessor->Type = EGLTFJsonAccessorType::Scalar;
	JsonInputAccessor->Count = FrameCount;
	JsonInputAccessor->MinMaxLength = 1;
	JsonInputAccessor->Min[0] = 0;
	JsonInputAccessor->Max[0] = Timestamps[FrameCount - 1];

	EGLTFJsonInterpolation Interpolation = FGLTFCoreUtilities::ConvertInterpolation(AnimSequence->Interpolation);
	if (Interpolation == EGLTFJsonInterpolation::None)
	{
		Interpolation = EGLTFJsonInterpolation::Linear;
		// TODO: report warning (about unknown interpolation exported as linear)
	}

	TArray<FBoneIndexType> BoneIndices;
	FGLTFBoneUtilities::GetBoneIndices(SkeletalMesh->GetRefSkeleton(), BoneIndices);

	TArray<TArray<FTransform>> FrameTransforms;
	FGLTFBoneUtilities::GetBoneTransformsByFrame(SkeletalMesh, AnimSequence, BoneIndices, Timestamps, FrameTransforms);

	for (const FBoneIndexType BoneIndex : BoneIndices)
	{
		FGLTFJsonNode* Node = Builder.AddUniqueNode(RootNode, SkeletalMesh, BoneIndex);

		// TODO: detect if a bone has the same transforms across multiple frames (at least if its the same across all frames) and optimize

		{
			TArray<FGLTFVector3> Translations;
			Translations.AddUninitialized(FrameCount);

			for (int32 Frame = 0; Frame < FrameCount; ++Frame)
			{
				const FVector3f KeyPosition = FVector3f(FrameTransforms[Frame][BoneIndex].GetTranslation());
				Translations[Frame] = FGLTFCoreUtilities::ConvertPosition(KeyPosition, Builder.ExportOptions->ExportUniformScale);
			}

			FGLTFJsonAccessor* JsonOutputAccessor = Builder.AddAccessor();
			JsonOutputAccessor->BufferView = Builder.AddBufferView(Translations);
			JsonOutputAccessor->ComponentType = EGLTFJsonComponentType::Float;
			JsonOutputAccessor->Count = Translations.Num();
			JsonOutputAccessor->Type = EGLTFJsonAccessorType::Vec3;

			FGLTFJsonAnimationSampler* JsonSampler = JsonAnimation->Samplers.Add();
			JsonSampler->Input = JsonInputAccessor;
			JsonSampler->Output = JsonOutputAccessor;
			JsonSampler->Interpolation = Interpolation;

			FGLTFJsonAnimationChannel JsonChannel;
			JsonChannel.Sampler = JsonSampler;
			JsonChannel.Target.Path = EGLTFJsonTargetPath::Translation;
			JsonChannel.Target.Node = Node;
			JsonAnimation->Channels.Add(JsonChannel);
		}

		{
			TArray<FGLTFQuaternion> Rotations;
			Rotations.AddUninitialized(FrameCount);

			for (int32 Frame = 0; Frame < FrameCount; ++Frame)
			{
				const FQuat4f KeyRotation = FQuat4f(FrameTransforms[Frame][BoneIndex].GetRotation());
				Rotations[Frame] = FGLTFCoreUtilities::ConvertRotation(KeyRotation);
			}

			FGLTFJsonAccessor* JsonOutputAccessor = Builder.AddAccessor();
			JsonOutputAccessor->BufferView = Builder.AddBufferView(Rotations);
			JsonOutputAccessor->ComponentType = EGLTFJsonComponentType::Float;
			JsonOutputAccessor->Count = Rotations.Num();
			JsonOutputAccessor->Type = EGLTFJsonAccessorType::Vec4;

			FGLTFJsonAnimationSampler* JsonSampler = JsonAnimation->Samplers.Add();
			JsonSampler->Input = JsonInputAccessor;
			JsonSampler->Output = JsonOutputAccessor;
			JsonSampler->Interpolation = Interpolation;

			FGLTFJsonAnimationChannel JsonChannel;
			JsonChannel.Sampler = JsonSampler;
			JsonChannel.Target.Path = EGLTFJsonTargetPath::Rotation;
			JsonChannel.Target.Node = Node;
			JsonAnimation->Channels.Add(JsonChannel);
		}

		{
			TArray<FGLTFVector3> Scales;
			Scales.AddUninitialized(FrameCount);

			for (int32 Frame = 0; Frame < FrameCount; ++Frame)
			{
				const FVector3f KeyScale = FVector3f(FrameTransforms[Frame][BoneIndex].GetScale3D());
				Scales[Frame] = FGLTFCoreUtilities::ConvertScale(KeyScale);
			}

			FGLTFJsonAccessor* JsonOutputAccessor = Builder.AddAccessor();
			JsonOutputAccessor->BufferView = Builder.AddBufferView(Scales);
			JsonOutputAccessor->ComponentType = EGLTFJsonComponentType::Float;
			JsonOutputAccessor->Count = Scales.Num();
			JsonOutputAccessor->Type = EGLTFJsonAccessorType::Vec3;

			FGLTFJsonAnimationSampler* JsonSampler = JsonAnimation->Samplers.Add();
			JsonSampler->Input = JsonInputAccessor;
			JsonSampler->Output = JsonOutputAccessor;
			JsonSampler->Interpolation = Interpolation;

			FGLTFJsonAnimationChannel JsonChannel;
			JsonChannel.Sampler = JsonSampler;
			JsonChannel.Target.Path = EGLTFJsonTargetPath::Scale;
			JsonChannel.Target.Node = Node;
			JsonAnimation->Channels.Add(JsonChannel);
		}
	}
}

void FGLTFDelayedLevelSequenceTask::Process()
{
	ULevelSequence* Sequence = const_cast<ULevelSequence*>(LevelSequence);
	const UMovieScene* MovieScene = Sequence->GetMovieScene();

	ALevelSequenceActor* LevelSequenceActor;
	UWorld* World = Level->GetWorld();
	ULevelSequencePlayer* Player = ULevelSequencePlayer::CreateLevelSequencePlayer(World, Sequence, FMovieSceneSequencePlaybackSettings(), LevelSequenceActor);

	FFrameRate TickResolution = MovieScene->GetTickResolution();
	FFrameRate DisplayRate = MovieScene->GetDisplayRate(); // TODO: add option to switch between DisplayRate and TickResolution?
	TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();

	int32 FrameOffset = FFrameRate::TransformTime(FFrameTime(MovieScene::DiscreteInclusiveLower(PlaybackRange)), TickResolution, DisplayRate).RoundToFrame().Value;
	int32 FrameCount = FFrameRate::TransformTime(FFrameTime(FFrameNumber(MovieScene::DiscreteSize(PlaybackRange))), TickResolution, DisplayRate).RoundToFrame().Value + 1;

	JsonAnimation->Name = Sequence->GetName() + TEXT("_") + FString::FromInt(JsonAnimation->Index); // Ensure unique name due to limitation in certain gltf viewers

	TArray<float> Timestamps;
	TArray<FFrameTime> FrameTimes;
	Timestamps.AddUninitialized(FrameCount);
	FrameTimes.AddUninitialized(FrameCount);

	for (int32 Frame = 0; Frame < FrameCount; ++Frame)
	{
		Timestamps[Frame] = static_cast<float>(DisplayRate.AsSeconds(Frame));
		FrameTimes[Frame] = FFrameRate::TransformTime(FFrameTime(FrameOffset + Frame), DisplayRate, TickResolution);
	}

	// TODO: add animation data accessor converters to reuse track information

	FGLTFJsonAccessor* JsonInputAccessor = Builder.AddAccessor();
	JsonInputAccessor->BufferView = Builder.AddBufferView(Timestamps);
	JsonInputAccessor->ComponentType = EGLTFJsonComponentType::Float;
	JsonInputAccessor->Type = EGLTFJsonAccessorType::Scalar;
	JsonInputAccessor->Count = FrameCount;
	JsonInputAccessor->MinMaxLength = 1;
	JsonInputAccessor->Min[0] = 0;
	JsonInputAccessor->Max[0] = Timestamps[FrameCount - 1];

	for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
	{
		for (TWeakObjectPtr<UObject> Object : Player->FindBoundObjects(Binding.GetObjectGuid(), MovieSceneSequenceID::Root))
		{
			FGLTFJsonNode* Node;
			FTransform DefaultTransform;

			if (const AActor* Actor = Cast<AActor>(Object.Get()))
			{
				Node = Builder.AddUniqueNode(Actor);
				DefaultTransform = Actor->GetRootComponent()->GetRelativeTransform();
			}
			else if (const USceneComponent* SceneComponent = Cast<USceneComponent>(Object.Get()))
			{
				Node = Builder.AddUniqueNode(SceneComponent);
				DefaultTransform = SceneComponent->GetRelativeTransform();
			}
			else
			{
				// TODO: report warning
				continue;
			}

			if (Node == nullptr)
			{
				// TODO: report warning
				continue;
			}

			FVector3f DefaultTranslation = FVector3f(DefaultTransform.GetTranslation());
			FRotator3f DefaultRotator = FRotator3f(DefaultTransform.GetRotation().Rotator());
			FVector3f DefaultScale = FVector3f(DefaultTransform.GetScale3D());

			for (UMovieSceneTrack* Track : Binding.GetTracks())
			{
				UMovieScene3DTransformTrack* TransformTrack = Cast<UMovieScene3DTransformTrack>(Track);
				if (TransformTrack == nullptr)
				{
					// TODO: report warning
					continue;
				}

				for (UMovieSceneSection* Section : TransformTrack->GetAllSections())
				{
					// TODO: add support for Section->GetCompletionMode() (i.e. when finished)

					UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(Section);
					if (TransformSection == nullptr)
					{
						// TODO: report warning
						continue;
					}

					const FOptionalMovieSceneBlendType BlendType = TransformSection->GetBlendType();
					if (BlendType.IsValid() && BlendType.Get() != EMovieSceneBlendType::Absolute)
					{
						// TODO: report warning
						continue;
					}

					// TODO: do we need to account for TransformSection->GetOffsetTime() ?

					TArrayView<FMovieSceneDoubleChannel*> Channels = TransformSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
					const EMovieSceneTransformChannel ChannelMask = TransformSection->GetMask().GetChannels();

					if (EnumHasAnyFlags(ChannelMask, EMovieSceneTransformChannel::Translation))
					{
						constexpr int32 IndexX = 0;
						constexpr int32 IndexY = 1;
						constexpr int32 IndexZ = 2;

						const bool MaskX = EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::TranslationX) && Channels.IsValidIndex(IndexX);
						const bool MaskY = EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::TranslationY) && Channels.IsValidIndex(IndexY);
						const bool MaskZ = EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::TranslationZ) && Channels.IsValidIndex(IndexZ);

						TArray<FGLTFVector3> Translations;
						Translations.AddUninitialized(FrameCount);

						for (int32 Frame = 0; Frame < FrameCount; ++Frame)
						{
							const FFrameTime& FrameTime = FrameTimes[Frame];

							FVector3f Translation = DefaultTranslation;
							if (MaskX) Channels[IndexX]->Evaluate(FrameTime, Translation.X);
							if (MaskY) Channels[IndexY]->Evaluate(FrameTime, Translation.Y);
							if (MaskZ) Channels[IndexZ]->Evaluate(FrameTime, Translation.Z);

							Translations[Frame] = FGLTFCoreUtilities::ConvertPosition(Translation, Builder.ExportOptions->ExportUniformScale);
						}

						FGLTFJsonAccessor* JsonOutputAccessor = Builder.AddAccessor();
						JsonOutputAccessor->BufferView = Builder.AddBufferView(Translations);
						JsonOutputAccessor->ComponentType = EGLTFJsonComponentType::Float;
						JsonOutputAccessor->Count = Translations.Num();
						JsonOutputAccessor->Type = EGLTFJsonAccessorType::Vec3;

						FGLTFJsonAnimationSampler* JsonSampler = JsonAnimation->Samplers.Add();
						JsonSampler->Input = JsonInputAccessor;
						JsonSampler->Output = JsonOutputAccessor;
						JsonSampler->Interpolation = EGLTFJsonInterpolation::Linear;

						FGLTFJsonAnimationChannel JsonChannel;
						JsonChannel.Sampler = JsonSampler;
						JsonChannel.Target.Path = EGLTFJsonTargetPath::Translation;
						JsonChannel.Target.Node = Node;
						JsonAnimation->Channels.Add(JsonChannel);
					}

					if (EnumHasAnyFlags(ChannelMask, EMovieSceneTransformChannel::Rotation))
					{
						constexpr int32 IndexX = 3;
						constexpr int32 IndexY = 4;
						constexpr int32 IndexZ = 5;

						const bool MaskX = EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::RotationX) && Channels.IsValidIndex(IndexX);
						const bool MaskY = EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::RotationY) && Channels.IsValidIndex(IndexY);
						const bool MaskZ = EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::RotationZ) && Channels.IsValidIndex(IndexZ);

						TArray<FGLTFQuaternion> Rotations;
						Rotations.AddUninitialized(FrameCount);

						for (int32 Frame = 0; Frame < FrameCount; ++Frame)
						{
							const FFrameTime& FrameTime = FrameTimes[Frame];

							FRotator3f Rotator = DefaultRotator;
							if (MaskX) Channels[IndexX]->Evaluate(FrameTime, Rotator.Roll);
							if (MaskY) Channels[IndexY]->Evaluate(FrameTime, Rotator.Pitch);
							if (MaskZ) Channels[IndexZ]->Evaluate(FrameTime, Rotator.Yaw);

							Rotations[Frame] = FGLTFCoreUtilities::ConvertRotation(Rotator.Quaternion());
						}

						FGLTFJsonAccessor* JsonOutputAccessor = Builder.AddAccessor();
						JsonOutputAccessor->BufferView = Builder.AddBufferView(Rotations);
						JsonOutputAccessor->ComponentType = EGLTFJsonComponentType::Float;
						JsonOutputAccessor->Count = Rotations.Num();
						JsonOutputAccessor->Type = EGLTFJsonAccessorType::Vec4;

						FGLTFJsonAnimationSampler* JsonSampler = JsonAnimation->Samplers.Add();
						JsonSampler->Input = JsonInputAccessor;
						JsonSampler->Output = JsonOutputAccessor;
						JsonSampler->Interpolation = EGLTFJsonInterpolation::Linear;

						FGLTFJsonAnimationChannel JsonChannel;
						JsonChannel.Sampler = JsonSampler;
						JsonChannel.Target.Path = EGLTFJsonTargetPath::Rotation;
						JsonChannel.Target.Node = Node;
						JsonAnimation->Channels.Add(JsonChannel);
					}

					if (EnumHasAnyFlags(ChannelMask, EMovieSceneTransformChannel::Scale))
					{
						constexpr int32 IndexX = 6;
						constexpr int32 IndexY = 7;
						constexpr int32 IndexZ = 8;

						const bool MaskX = EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::ScaleX) && Channels.IsValidIndex(IndexX);
						const bool MaskY = EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::ScaleY) && Channels.IsValidIndex(IndexY);
						const bool MaskZ = EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::ScaleZ) && Channels.IsValidIndex(IndexZ);

						TArray<FGLTFVector3> Scales;
						Scales.AddUninitialized(FrameCount);

						for (int32 Frame = 0; Frame < FrameCount; ++Frame)
						{
							const FFrameTime& FrameTime = FrameTimes[Frame];

							FVector3f Scale = DefaultScale;
							if (MaskX) Channels[IndexX]->Evaluate(FrameTime, Scale.X);
							if (MaskY) Channels[IndexY]->Evaluate(FrameTime, Scale.Y);
							if (MaskZ) Channels[IndexZ]->Evaluate(FrameTime, Scale.Z);

							Scales[Frame] = FGLTFCoreUtilities::ConvertScale(Scale);
						}

						FGLTFJsonAccessor* JsonOutputAccessor = Builder.AddAccessor();
						JsonOutputAccessor->BufferView = Builder.AddBufferView(Scales);
						JsonOutputAccessor->ComponentType = EGLTFJsonComponentType::Float;
						JsonOutputAccessor->Count = Scales.Num();
						JsonOutputAccessor->Type = EGLTFJsonAccessorType::Vec3;

						FGLTFJsonAnimationSampler* JsonSampler = JsonAnimation->Samplers.Add();
						JsonSampler->Input = JsonInputAccessor;
						JsonSampler->Output = JsonOutputAccessor;
						JsonSampler->Interpolation = EGLTFJsonInterpolation::Linear;

						FGLTFJsonAnimationChannel JsonChannel;
						JsonChannel.Sampler = JsonSampler;
						JsonChannel.Target.Path = EGLTFJsonTargetPath::Scale;
						JsonChannel.Target.Node = Node;
						JsonAnimation->Channels.Add(JsonChannel);
					}
				}
			}
		}
	}

	World->DestroyActor(LevelSequenceActor);
}
