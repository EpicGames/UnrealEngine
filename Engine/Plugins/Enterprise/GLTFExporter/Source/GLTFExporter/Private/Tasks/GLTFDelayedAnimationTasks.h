// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/GLTFDelayedTask.h"
#include "Builders/GLTFConvertBuilder.h"
#include "LevelSequence.h"
#include "Animation/AnimSequence.h"

class FGLTFDelayedAnimSequenceTask : public FGLTFDelayedTask
{
public:

	FGLTFDelayedAnimSequenceTask(FGLTFConvertBuilder& Builder,  FGLTFJsonNode* RootNode, const USkeletalMesh* SkeletalMesh, const UAnimSequence* AnimSequence, FGLTFJsonAnimation* JsonAnimation)
		: FGLTFDelayedTask(EGLTFTaskPriority::Animation)
		, Builder(Builder)
		, RootNode(RootNode)
		, SkeletalMesh(SkeletalMesh)
		, AnimSequence(AnimSequence)
		, JsonAnimation(JsonAnimation)
	{
	}

	virtual FString GetName() override
	{
		return AnimSequence->GetName();
	}

	virtual void Process() override;

private:

	FGLTFConvertBuilder& Builder;
	FGLTFJsonNode* RootNode;
	const USkeletalMesh* SkeletalMesh;
	const UAnimSequence* AnimSequence;
	FGLTFJsonAnimation* JsonAnimation;
};

class FGLTFDelayedLevelSequenceTask : public FGLTFDelayedTask
{
public:

	FGLTFDelayedLevelSequenceTask(FGLTFConvertBuilder& Builder, const ULevel* Level, const ULevelSequence* LevelSequence, FGLTFJsonAnimation* JsonAnimation)
		: FGLTFDelayedTask(EGLTFTaskPriority::Animation)
		, Builder(Builder)
		, Level(Level)
		, LevelSequence(LevelSequence)
		, JsonAnimation(JsonAnimation)
	{
	}

	virtual FString GetName() override
	{
		return LevelSequence->GetName();
	}

	virtual void Process() override;

private:

	FGLTFConvertBuilder& Builder;
	const ULevel* Level;
	const ULevelSequence* LevelSequence;
	FGLTFJsonAnimation* JsonAnimation;
};
