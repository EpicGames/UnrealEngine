// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/GLTFDelayedMaterialTasks.h"
#include "Utilities/GLTFCoreUtilities.h"
#include "Converters/GLTFNameUtilities.h"
#include "Converters/GLTFMaterialUtilities.h"
#include "Builders/GLTFContainerBuilder.h"
#include "Utilities/GLTFProxyMaterialUtilities.h"
#include "MaterialDomain.h"
#include "Materials/GLTFProxyMaterialInfo.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionTextureSampleParameter2D.h"
#include "RHIShaderPlatform.h"

namespace
{
	// Component masks
	const FLinearColor RedMask(1.0f, 0.0f, 0.0f, 0.0f);
	const FLinearColor GreenMask(0.0f, 1.0f, 0.0f, 0.0f);
	const FLinearColor BlueMask(0.0f, 0.0f, 1.0f, 0.0f);
	const FLinearColor AlphaMask(0.0f, 0.0f, 0.0f, 1.0f);
	const FLinearColor RgbMask = RedMask + GreenMask + BlueMask;
	const FLinearColor RgbaMask = RgbMask + AlphaMask;

	// Property-specific component masks
	const FLinearColor BaseColorMask = RgbMask;
	const FLinearColor OpacityMask = AlphaMask;
	const FLinearColor MetallicMask = BlueMask;
	const FLinearColor RoughnessMask = GreenMask;
	const FLinearColor OcclusionMask = RedMask;
	const FLinearColor ClearCoatMask = RedMask;
	const FLinearColor ClearCoatRoughnessMask = GreenMask;

	// Ideal masks for texture-inputs (doesn't require baking)
	const TArray<FLinearColor> DefaultColorInputMasks = { RgbMask, RgbaMask };
	const TArray<FLinearColor> BaseColorInputMasks = { BaseColorMask };
	const TArray<FLinearColor> OpacityInputMasks = { OpacityMask };
	const TArray<FLinearColor> MetallicInputMasks = { MetallicMask };
	const TArray<FLinearColor> RoughnessInputMasks = { RoughnessMask };
	const TArray<FLinearColor> OcclusionInputMasks = { OcclusionMask };
	const TArray<FLinearColor> ClearCoatInputMasks = { ClearCoatMask };
	const TArray<FLinearColor> ClearCoatRoughnessInputMasks = { ClearCoatRoughnessMask };
}

void FGLTFDelayedMaterialTask::Process()
{
	const UMaterial* BaseMaterial = Material->GetMaterial();
	if (!BaseMaterial)
	{
		Builder.LogWarning(FString::Printf(
			TEXT("Material %s does not have a base material. Aborting."),
			*Material->GetName()));
		return;
	}

	if (BaseMaterial->MaterialDomain != MD_Surface)
	{
		Builder.LogWarning(FString::Printf(
			TEXT("Non-surface materials not supported, Material %s will be treated as surface."),
			*Material->GetName()));
	}

	// Abort process if the material has no valid resource because no proxy can be created without it
	// See FExportMaterialProxy constructor which requires a valid resource to proceed
	if (!Material->GetMaterialResource(GMaxRHIShaderPlatform))
	{
		Builder.LogWarning(FString::Printf(
			TEXT("Cannot bake material %s because it has no valid resource."),
			*Material->GetName()));
		return;
	}

	JsonMaterial->Name = GetMaterialName();
	JsonMaterial->DoubleSided = Material->IsTwoSided();
	ConvertAlphaMode(JsonMaterial->AlphaMode);

	FString WarningMessages;
	EMaterialShadingModel UEShadingModel = FGLTFMaterialUtilities::GetShadingModel(Material, WarningMessages);
	if (!WarningMessages.IsEmpty())
	{
		Builder.LogWarning(WarningMessages);
	}

	if (HandleGLTFImported(UEShadingModel))
	{
		return;
	}

	JsonMaterial->AlphaCutoff = Material->GetOpacityMaskClipValue();

	ConvertShadingModel(UEShadingModel, JsonMaterial->ShadingModel);
	ApplyExportOptionsToShadingModel(JsonMaterial->ShadingModel, UEShadingModel);

	if (!ensure((JsonMaterial->ShadingModel != EGLTFJsonShadingModel::None) && ((int)JsonMaterial->ShadingModel < (int)EGLTFJsonShadingModel::NumShadingModels)))
	{
		Builder.LogWarning(FString::Printf(TEXT("Failed to export %s material. ShadingModel is unexpected."), *Material->GetName()));
		return;
	}

	if (FGLTFProxyMaterialUtilities::IsProxyMaterial(BaseMaterial))
	{
		GetProxyParameters(*JsonMaterial);
		return;
	}

#if WITH_EDITOR
	{
		if (JsonMaterial->ShadingModel == EGLTFJsonShadingModel::Transmission)
		{
			const FMaterialPropertyEx BaseColorProperty = MP_BaseColor;
			const FMaterialPropertyEx OpacityProperty = MP_Opacity;

			TryGetTransmissionBaseColorAndOpacity(*JsonMaterial, BaseColorProperty, OpacityProperty);
		}
		else
		{
			const FMaterialPropertyEx BaseColorProperty = JsonMaterial->ShadingModel == EGLTFJsonShadingModel::Unlit ? MP_EmissiveColor : MP_BaseColor;
			const FMaterialPropertyEx OpacityProperty = JsonMaterial->AlphaMode == EGLTFJsonAlphaMode::Mask ? MP_OpacityMask : MP_Opacity;

			// TODO: check if a property is active before trying to get it (i.e. Material->IsPropertyActive)
			if (JsonMaterial->AlphaMode == EGLTFJsonAlphaMode::Opaque)
			{
				if (!TryGetConstantColor(JsonMaterial->PBRMetallicRoughness.BaseColorFactor, BaseColorProperty))
				{
					if (!TryGetSourceTexture(JsonMaterial->PBRMetallicRoughness.BaseColorTexture, BaseColorProperty, DefaultColorInputMasks))
					{
						if (!TryGetBakedMaterialProperty(JsonMaterial->PBRMetallicRoughness.BaseColorTexture, JsonMaterial->PBRMetallicRoughness.BaseColorFactor, BaseColorProperty, TEXT("BaseColor")))
						{
							Builder.LogWarning(FString::Printf(TEXT("Failed to export %s for material %s"), *BaseColorProperty.ToString(), *Material->GetName()));
						}
					}
				}

				JsonMaterial->PBRMetallicRoughness.BaseColorFactor.A = 1.0f; // make sure base color is opaque
			}
			else
			{
				if (!TryGetBaseColorAndOpacity(JsonMaterial->PBRMetallicRoughness, BaseColorProperty, OpacityProperty))
				{
					Builder.LogWarning(FString::Printf(TEXT("Failed to export %s and %s for material %s"), *BaseColorProperty.ToString(), *OpacityProperty.ToString(), *Material->GetName()));
				}
			}
		}

		if (JsonMaterial->ShadingModel != EGLTFJsonShadingModel::Unlit)
		{
			const FMaterialPropertyEx MetallicProperty = MP_Metallic;
			const FMaterialPropertyEx RoughnessProperty = MP_Roughness;

			if (!TryGetMetallicAndRoughness(JsonMaterial->PBRMetallicRoughness, MetallicProperty, RoughnessProperty))
			{
				Builder.LogWarning(FString::Printf(TEXT("Failed to export %s and %s for material %s"), *MetallicProperty.ToString(), *RoughnessProperty.ToString(), *Material->GetName()));
			}

			const FMaterialPropertyEx EmissiveProperty = MP_EmissiveColor;
			if (!TryGetEmissive(*JsonMaterial, EmissiveProperty))
			{
				Builder.LogWarning(FString::Printf(TEXT("Failed to export %s for material %s"), *EmissiveProperty.ToString(), *Material->GetName()));
			}

			const FMaterialPropertyEx NormalProperty = JsonMaterial->ShadingModel == EGLTFJsonShadingModel::ClearCoat ? FMaterialPropertyEx::ClearCoatBottomNormal : FMaterialPropertyEx(MP_Normal);
			if (IsPropertyNonDefault(NormalProperty))
			{
				if (!TryGetSourceTexture(JsonMaterial->NormalTexture, NormalProperty, DefaultColorInputMasks))
				{
					if (!TryGetBakedMaterialProperty(JsonMaterial->NormalTexture, NormalProperty, TEXT("Normal")))
					{
						Builder.LogWarning(FString::Printf(TEXT("Failed to export %s for material %s"), *NormalProperty.ToString(), *Material->GetName()));
					}
				}
			}

			const FMaterialPropertyEx AmbientOcclusionProperty = MP_AmbientOcclusion;
			if (IsPropertyNonDefault(AmbientOcclusionProperty))
			{
				if (!TryGetSourceTexture(JsonMaterial->OcclusionTexture, AmbientOcclusionProperty, OcclusionInputMasks))
				{
					if (!TryGetBakedMaterialProperty(JsonMaterial->OcclusionTexture, AmbientOcclusionProperty, TEXT("Occlusion")))
					{
						Builder.LogWarning(FString::Printf(TEXT("Failed to export %s for material %s"), *AmbientOcclusionProperty.ToString(), *Material->GetName()));
					}
				}
			}

			const FMaterialPropertyEx SpecularProperty = MP_Specular;
			if (IsPropertyNonDefault(SpecularProperty))
			{
				if (!TryGetSpecular(*JsonMaterial, SpecularProperty))
				{
					Builder.LogWarning(FString::Printf(TEXT("Failed to export %s for material %s"), *SpecularProperty.ToString(), *Material->GetName()));
				}
			}

			if (JsonMaterial->AlphaMode == EGLTFJsonAlphaMode::Blend)
			{
				bool bRefractionSet = false;
				if (BaseMaterial->RefractionMethod == ERefractionMode::RM_IndexOfRefraction)
				{
					const FMaterialPropertyEx RefractionProperty = MP_Refraction;
					if (IsPropertyNonDefault(RefractionProperty))
					{
						if (!TryGetRefraction(*JsonMaterial, RefractionProperty))
						{
							Builder.LogWarning(FString::Printf(TEXT("Failed to export %s for material %s"), *RefractionProperty.ToString(), *Material->GetName()));
						}
						else
						{
							bRefractionSet = true;
						}
					}
				}
				if (!bRefractionSet)
				{
					// in this case UE takes Refraction as IOR 1.0 (Air)
					JsonMaterial->IOR.Value = 1.0f;
				}
			}

			if (JsonMaterial->ShadingModel == EGLTFJsonShadingModel::ClearCoat)
			{
				const FMaterialPropertyEx ClearCoatProperty = MP_CustomData0;
				const FMaterialPropertyEx ClearCoatRoughnessProperty = MP_CustomData1;

				if (!TryGetClearCoatRoughness(JsonMaterial->ClearCoat, ClearCoatProperty, ClearCoatRoughnessProperty))
				{
					Builder.LogWarning(FString::Printf(TEXT("Failed to export %s and %s for material %s"), *ClearCoatProperty.ToString(), *ClearCoatRoughnessProperty.ToString(), *Material->GetName()));
				}

				const FMaterialPropertyEx ClearCoatNormalProperty = MP_Normal;
				if (IsPropertyNonDefault(ClearCoatNormalProperty))
				{
					if (!TryGetSourceTexture(JsonMaterial->ClearCoat.ClearCoatNormalTexture, ClearCoatNormalProperty, DefaultColorInputMasks))
					{
						if (!TryGetBakedMaterialProperty(JsonMaterial->ClearCoat.ClearCoatNormalTexture, ClearCoatNormalProperty, TEXT("ClearCoatNormal")))
						{
							Builder.LogWarning(FString::Printf(TEXT("Failed to export %s for material %s"), *ClearCoatNormalProperty.ToString(), *Material->GetName()));
						}
					}
				}
			}
			else if (JsonMaterial->ShadingModel == EGLTFJsonShadingModel::Sheen)
			{
				const FMaterialPropertyEx FuzzColorProperty = MP_SubsurfaceColor;
				const FMaterialPropertyEx ClothProperty = MP_CustomData0;

				// We are not checking (IsPropertyNonDefault(FuzzColorProperty) || IsPropertyNonDefault(ClothProperty)) because UE's defaults are white/1 respectively compared to glTF black/0
				if (!TryGetFuzzColorAndCloth(*JsonMaterial, FuzzColorProperty, ClothProperty))
				{
					Builder.LogWarning(FString::Printf(TEXT("Failed to export %s and %s for material %s"), *FuzzColorProperty.ToString(), *ClothProperty.ToString(), *Material->GetName()));
				}
			}

			if (JsonMaterial->ShadingModel != EGLTFJsonShadingModel::SpecularGlossiness)
			{
				const FMaterialPropertyEx AnisotropyProperty = MP_Anisotropy;
				if (IsPropertyNonDefault(AnisotropyProperty))
				{
					if (!TryGetSourceTexture(JsonMaterial->Anisotropy.AnisotropyTexture, AnisotropyProperty, DefaultColorInputMasks))
					{
						if (!TryGetBakedMaterialProperty(JsonMaterial->Anisotropy.AnisotropyTexture, JsonMaterial->Anisotropy.AnisotropyStrength, 1.0f, AnisotropyProperty,
							[](FColor& Color, const uint8& ChannelValueToMove)
							{
								Color.B = ChannelValueToMove;
								
								Color.R = 255;
								Color.G = 255;
								Color.A = 255;
							}
						))
						{
							Builder.LogWarning(FString::Printf(TEXT("Failed to export %s for material %s"), *AnisotropyProperty.ToString(), *Material->GetName()));
						}
					}
				}
			}
		}
	}

	// Warn if properties baked using mesh data are using overlapping UVs
	if (MeshData != nullptr && MeshDataBakedProperties.Num() > 0)
	{
		const FGLTFMeshData* ParentMeshData = MeshData->GetParent();

		const float UVOverlapThreshold = 1.0f / 100.0f;
		const float UVOverlap = UVOverlapChecker.GetOrAdd(&ParentMeshData->Description, SectionIndices, MeshData->BakeUsingTexCoord);

		if (UVOverlap > UVOverlapThreshold)
		{
			FString SectionString = TEXT("mesh section");
			SectionString += SectionIndices.Num() > 1 ? TEXT("s ") : TEXT(" ");
			SectionString += FString::JoinBy(SectionIndices, TEXT(", "), FString::FromInt);

			Builder.LogWarning(FString::Printf(
				TEXT("Material %s is baked using mesh data from %s but the lightmap UV (channel %d) are overlapping by %.2f%% (in %s) and may produce incorrect results"),
				*Material->GetName(),
				*ParentMeshData->Name,
				MeshData->BakeUsingTexCoord,
				UVOverlap * 100,
				*SectionString));
		}
	}
#else
	Builder.LogError(FString::Printf(
		TEXT("Can't properly export material %s in a runtime environment without a glTF proxy. In the Unreal Editor's Content Browser, please right-click on the material, then select Create glTF Proxy Material"),
		*Material->GetName()));
#endif
}

FString FGLTFDelayedMaterialTask::GetMaterialName() const
{
	FString MaterialName = Material->GetName();

	if (MeshData != nullptr)
	{
		MaterialName += TEXT("_") + MeshData->Name;
	}

	return MaterialName;
}

FString FGLTFDelayedMaterialTask::GetBakedTextureName(const FString& PropertyName) const
{
	return GetMaterialName() + TEXT("_") + PropertyName;
}

void FGLTFDelayedMaterialTask::GetProxyParameters(FGLTFJsonMaterial& OutMaterial) const
{
	GetProxyParameter(FGLTFProxyMaterialInfo::BaseColor, OutMaterial.PBRMetallicRoughness.BaseColorTexture);
	GetProxyParameter(FGLTFProxyMaterialInfo::BaseColorFactor, OutMaterial.PBRMetallicRoughness.BaseColorFactor);

	if (OutMaterial.ShadingModel != EGLTFJsonShadingModel::Unlit)
	{
		GetProxyParameter(FGLTFProxyMaterialInfo::Emissive, OutMaterial.EmissiveTexture);
		GetProxyParameter(FGLTFProxyMaterialInfo::EmissiveFactor, OutMaterial.EmissiveFactor);
		GetProxyParameter(FGLTFProxyMaterialInfo::EmissiveStrength, OutMaterial.EmissiveStrength);

		GetProxyParameter(FGLTFProxyMaterialInfo::MetallicRoughness, OutMaterial.PBRMetallicRoughness.MetallicRoughnessTexture);
		GetProxyParameter(FGLTFProxyMaterialInfo::MetallicFactor, OutMaterial.PBRMetallicRoughness.MetallicFactor);
		GetProxyParameter(FGLTFProxyMaterialInfo::RoughnessFactor, OutMaterial.PBRMetallicRoughness.RoughnessFactor);

		if (HasProxyParameter(FGLTFProxyMaterialInfo::Normal.Texture))
		{
			if (OutMaterial.ShadingModel == EGLTFJsonShadingModel::ClearCoat && !FGLTFMaterialUtilities::IsClearCoatBottomNormalEnabled())
			{
				Builder.LogWarning(FString::Printf(
					TEXT("Proxy material %s won't be exported with (bottom) normal because ClearCoatEnableSecondNormal in project rendering settings is disabled"),
					*Material->GetName()));
			}
			else
			{
				GetProxyParameter(FGLTFProxyMaterialInfo::Normal, OutMaterial.NormalTexture);
				GetProxyParameter(FGLTFProxyMaterialInfo::NormalScale, OutMaterial.NormalTexture.Scale);
			}
		}

		GetProxyParameter(FGLTFProxyMaterialInfo::Occlusion, OutMaterial.OcclusionTexture);
		GetProxyParameter(FGLTFProxyMaterialInfo::OcclusionStrength, OutMaterial.OcclusionTexture.Strength);

		GetProxyParameter(FGLTFProxyMaterialInfo::SpecularFactor, OutMaterial.Specular.Factor);
		GetProxyParameter(FGLTFProxyMaterialInfo::SpecularTexture, OutMaterial.Specular.Texture);

		GetProxyParameter(FGLTFProxyMaterialInfo::IOR, OutMaterial.IOR.Value);

		if (OutMaterial.ShadingModel == EGLTFJsonShadingModel::ClearCoat)
		{
			GetProxyParameter(FGLTFProxyMaterialInfo::ClearCoat, OutMaterial.ClearCoat.ClearCoatTexture);
			GetProxyParameter(FGLTFProxyMaterialInfo::ClearCoatFactor, OutMaterial.ClearCoat.ClearCoatFactor);

			GetProxyParameter(FGLTFProxyMaterialInfo::ClearCoatRoughness, OutMaterial.ClearCoat.ClearCoatRoughnessTexture);
			GetProxyParameter(FGLTFProxyMaterialInfo::ClearCoatRoughnessFactor, OutMaterial.ClearCoat.ClearCoatRoughnessFactor);

			GetProxyParameter(FGLTFProxyMaterialInfo::ClearCoatNormal, OutMaterial.ClearCoat.ClearCoatNormalTexture);
			GetProxyParameter(FGLTFProxyMaterialInfo::ClearCoatNormalScale, OutMaterial.ClearCoat.ClearCoatNormalTexture.Scale);
		}
		else if (OutMaterial.ShadingModel == EGLTFJsonShadingModel::Sheen)
		{
			GetProxyParameter(FGLTFProxyMaterialInfo::SheenColorFactor, OutMaterial.Sheen.ColorFactor);
			GetProxyParameter(FGLTFProxyMaterialInfo::SheenColorTexture , OutMaterial.Sheen.ColorTexture);
			GetProxyParameter(FGLTFProxyMaterialInfo::SheenRoughnessFactor, OutMaterial.Sheen.RoughnessFactor);
			GetProxyParameter(FGLTFProxyMaterialInfo::SheenRoughnessTexture, OutMaterial.Sheen.RoughnessTexture);
		}
		else if (OutMaterial.ShadingModel == EGLTFJsonShadingModel::Transmission)
		{
			GetProxyParameter(FGLTFProxyMaterialInfo::TransmissionFactor, OutMaterial.Transmission.Factor);
			GetProxyParameter(FGLTFProxyMaterialInfo::TransmissionTexture, OutMaterial.Transmission.Texture);
		}

		if (OutMaterial.ShadingModel != EGLTFJsonShadingModel::SpecularGlossiness)
		{
			GetProxyParameter(FGLTFProxyMaterialInfo::IridescenceFactor, OutMaterial.Iridescence.IridescenceFactor);
			GetProxyParameter(FGLTFProxyMaterialInfo::IridescenceTexture, OutMaterial.Iridescence.IridescenceTexture);
			GetProxyParameter(FGLTFProxyMaterialInfo::IridescenceIOR, OutMaterial.Iridescence.IridescenceIOR);
			GetProxyParameter(FGLTFProxyMaterialInfo::IridescenceThicknessMinimum, OutMaterial.Iridescence.IridescenceThicknessMinimum);
			GetProxyParameter(FGLTFProxyMaterialInfo::IridescenceThicknessMaximum, OutMaterial.Iridescence.IridescenceThicknessMaximum);
			GetProxyParameter(FGLTFProxyMaterialInfo::IridescenceThicknessTexture, OutMaterial.Iridescence.IridescenceThicknessTexture);

			GetProxyParameter(FGLTFProxyMaterialInfo::AnisotropyStrength, OutMaterial.Anisotropy.AnisotropyStrength);
			GetProxyParameter(FGLTFProxyMaterialInfo::AnisotropyRotation, OutMaterial.Anisotropy.AnisotropyRotation);
			GetProxyParameter(FGLTFProxyMaterialInfo::AnisotropyTexture, OutMaterial.Anisotropy.AnisotropyTexture);
		}
	}
}

void FGLTFDelayedMaterialTask::GetProxyParameter(const TGLTFProxyMaterialParameterInfo<float>& ParameterInfo, float& OutValue) const
{
	ParameterInfo.Get(Material, OutValue, true);
}

void FGLTFDelayedMaterialTask::GetProxyParameter(const TGLTFProxyMaterialParameterInfo<FLinearColor>& ParameterInfo, FGLTFJsonColor3& OutValue) const
{
	FLinearColor Value;
	if (ParameterInfo.Get(Material, Value, true))
	{
		OutValue = FGLTFCoreUtilities::ConvertColor3(Value);
	}
}

void FGLTFDelayedMaterialTask::GetProxyParameter(const TGLTFProxyMaterialParameterInfo<FLinearColor>& ParameterInfo, FGLTFJsonColor4& OutValue) const
{
	FLinearColor Value;
	if (ParameterInfo.Get(Material, Value, true))
	{
		OutValue = FGLTFCoreUtilities::ConvertColor(Value);
	}
}

void FGLTFDelayedMaterialTask::GetProxyParameter(const FGLTFProxyMaterialTextureParameterInfo& ParameterInfo, FGLTFJsonTextureInfo& OutValue) const
{
	UTexture* Texture;
	if (!ParameterInfo.Texture.Get(Material, Texture, true) || Texture == nullptr)
	{
		return;
	}

	TextureAddress TextureAddressX = TextureAddress::TA_Wrap;
	TextureAddress TextureAddressY = TextureAddress::TA_Wrap;

	FLinearColor TilingMethod;
	if (ParameterInfo.TilingMethod.Get(Material, TilingMethod, true))
	{
		TextureAddressX = FGLTFMaterialUtilities::ToTextureAddress(TilingMethod.R);
		TextureAddressY = FGLTFMaterialUtilities::ToTextureAddress(TilingMethod.G);
	}

	const bool bSRGB = ParameterInfo == FGLTFProxyMaterialInfo::BaseColor || ParameterInfo == FGLTFProxyMaterialInfo::Emissive;
	OutValue.Index = Builder.AddUniqueTexture(Texture, bSRGB, TextureAddressX, TextureAddressY);

	float UVIndex;
	if (ParameterInfo.UVIndex.Get(Material, UVIndex, true))
	{
		OutValue.TexCoord = FMath::RoundToInt(UVIndex);
	}

	FLinearColor UVOffset;
	if (ParameterInfo.UVOffset.Get(Material, UVOffset, true))
	{
		OutValue.Transform.Offset.X = UVOffset.R;
		OutValue.Transform.Offset.Y = UVOffset.G;
	}

	FLinearColor UVScale;
	if (ParameterInfo.UVScale.Get(Material, UVScale, true))
	{
		OutValue.Transform.Scale.X = UVScale.R;
		OutValue.Transform.Scale.Y = UVScale.G;
	}

	float UVRotation;
	if (ParameterInfo.UVRotation.Get(Material, UVRotation, true))
	{
		OutValue.Transform.Rotation = UVRotation;
	}

	if (!Builder.ExportOptions->bExportTextureTransforms && !OutValue.Transform.IsExactlyDefault())
	{
		Builder.LogWarning(FString::Printf(
			TEXT("Texture coordinates [%d] in %s for material %s are transformed, but texture transform is disabled by export options"),
			OutValue.TexCoord,
			*ParameterInfo.Texture.ToString(),
			*Material->GetName()));
		OutValue.Transform = {};
	}
}

void FGLTFDelayedMaterialTask::ConvertShadingModel(EMaterialShadingModel& UEShadingModel, EGLTFJsonShadingModel& OutGLTFShadingModel) const
{
	const EBlendMode BlendMode = Material->GetBlendMode();
	if (UEShadingModel == MSM_ClearCoat && BlendMode != BLEND_Opaque && BlendMode != BLEND_Masked)
	{
		// NOTE: Unreal seems to disable clear coat when blend mode anything but opaque or masked
		UEShadingModel = MSM_DefaultLit;
	}

	OutGLTFShadingModel = FGLTFCoreUtilities::ConvertShadingModel(UEShadingModel);
}

void FGLTFDelayedMaterialTask::ApplyExportOptionsToShadingModel(EGLTFJsonShadingModel& ShadingModel, const EMaterialShadingModel& UEMaterialShadingModel) const
{
	switch (ShadingModel)
	{
		case EGLTFJsonShadingModel::Default:
			break;

		case EGLTFJsonShadingModel::Unlit:
			if (!Builder.ExportOptions->bExportUnlitMaterials)
			{
				Builder.LogWarning(FString::Printf(
					TEXT("Shading model (%s) in material %s disabled by export options, will export as %s"),
					*FGLTFNameUtilities::GetName(UEMaterialShadingModel),
					*Material->GetName(),
					*FGLTFNameUtilities::GetName(MSM_DefaultLit)));

				ShadingModel = EGLTFJsonShadingModel::Default;
			}
			break;

		case EGLTFJsonShadingModel::ClearCoat:
			if (!Builder.ExportOptions->bExportClearCoatMaterials)
			{
				Builder.LogWarning(FString::Printf(
					TEXT("Shading model (%s) in material %s disabled by export options, will export as %s"),
					*FGLTFNameUtilities::GetName(UEMaterialShadingModel),
					*Material->GetName(),
					*FGLTFNameUtilities::GetName(MSM_DefaultLit)));

				ShadingModel = EGLTFJsonShadingModel::Default;
			}
			break;

		case EGLTFJsonShadingModel::Sheen:
			if (ShadingModel == EGLTFJsonShadingModel::Sheen && !Builder.ExportOptions->bExportClothMaterials)
			{
				Builder.LogWarning(FString::Printf(
					TEXT("Shading model (%s) in material %s disabled by export options, will export as %s"),
					*FGLTFNameUtilities::GetName(UEMaterialShadingModel),
					*Material->GetName(),
					*FGLTFNameUtilities::GetName(MSM_DefaultLit)));

				ShadingModel = EGLTFJsonShadingModel::Default;
			}
			break;

		case EGLTFJsonShadingModel::Transmission:
			if (!Builder.ExportOptions->bExportThinTranslucentMaterials)
			{
				Builder.LogWarning(FString::Printf(
					TEXT("Shading model (%s) in material %s disabled by export options, will export as %s"),
					*FGLTFNameUtilities::GetName(UEMaterialShadingModel),
					*Material->GetName(),
					*FGLTFNameUtilities::GetName(MSM_DefaultLit)));

				ShadingModel = EGLTFJsonShadingModel::Default;
			}
			break;

		case EGLTFJsonShadingModel::SpecularGlossiness:
			if (!Builder.ExportOptions->bExportSpecularGlossinessMaterials)
			{
				Builder.LogWarning(FString::Printf(
					TEXT("Shading model (%s) in material %s disabled by export options, will export as %s"),
					*FGLTFNameUtilities::GetName(UEMaterialShadingModel),
					*Material->GetName(),
					*FGLTFNameUtilities::GetName(MSM_DefaultLit)));

				ShadingModel = EGLTFJsonShadingModel::Default;
			}
			break;

		default:
			{
				Builder.LogWarning(FString::Printf(
					TEXT("Unsupported shading model (%s) in material %s, will export as %s"),
					*FGLTFNameUtilities::GetName(UEMaterialShadingModel),
					*Material->GetName(),
					*FGLTFNameUtilities::GetName(MSM_DefaultLit)));

				ShadingModel = EGLTFJsonShadingModel::Default;
			}
			break;
	}
}

void FGLTFDelayedMaterialTask::ConvertAlphaMode(EGLTFJsonAlphaMode& OutAlphaMode) const
{
	const EBlendMode BlendMode = Material->GetBlendMode();

	OutAlphaMode = FGLTFCoreUtilities::ConvertAlphaMode(BlendMode);
	if (OutAlphaMode == EGLTFJsonAlphaMode::None)
	{
		OutAlphaMode = EGLTFJsonAlphaMode::Blend;

		Builder.LogWarning(FString::Printf(
			TEXT("Unsupported blend mode (%s) in material %s, will export as %s"),
			*FGLTFNameUtilities::GetName(BlendMode),
			*Material->GetName(),
			*FGLTFNameUtilities::GetName(BLEND_Translucent)));
		return;
	}
}

#if WITH_EDITOR

bool FGLTFDelayedMaterialTask::TryGetBaseColorAndOpacity(FGLTFJsonPBRMetallicRoughness& OutPBRParams, const FMaterialPropertyEx& BaseColorProperty, const FMaterialPropertyEx& OpacityProperty)
{
	const bool bIsBaseColorConstant = TryGetConstantColor(OutPBRParams.BaseColorFactor, BaseColorProperty);
	const bool bIsOpacityConstant = TryGetConstantScalar(OutPBRParams.BaseColorFactor.A, OpacityProperty);

	if (bIsBaseColorConstant && bIsOpacityConstant)
	{
		return true;
	}

	// NOTE: since we always bake the properties (for now) when at least property is non-const, we need
	// to reset the constant factors to their defaults. Otherwise the baked value of a constant property
	// would be scaled with the factor, i.e a double scaling.
	OutPBRParams.BaseColorFactor = { 1.0f, 1.0f, 1.0f, 1.0f };

	const UTexture2D* BaseColorTexture;
	const UTexture2D* OpacityTexture;
	int32 BaseColorTexCoord;
	int32 OpacityTexCoord;
	FGLTFJsonTextureTransform BaseColorTransform;
	FGLTFJsonTextureTransform OpacityTransform;

	const bool bHasBaseColorSourceTexture = TryGetSourceTexture(BaseColorTexture, BaseColorTexCoord, BaseColorTransform, BaseColorProperty, BaseColorInputMasks);
	const bool bHasOpacitySourceTexture = TryGetSourceTexture(OpacityTexture, OpacityTexCoord, OpacityTransform, OpacityProperty, OpacityInputMasks);

	// Detect the "happy path" where both inputs share the same texture and are correctly masked.
	if (bHasBaseColorSourceTexture &&
		bHasOpacitySourceTexture &&
		BaseColorTexture == OpacityTexture &&
		BaseColorTexCoord == OpacityTexCoord &&
		BaseColorTransform.IsExactlyEqual(OpacityTransform))
	{
		OutPBRParams.BaseColorTexture.Index = Builder.AddUniqueTexture(BaseColorTexture, true);
		OutPBRParams.BaseColorTexture.TexCoord = BaseColorTexCoord;
		OutPBRParams.BaseColorTexture.Transform = BaseColorTransform;
		return true;
	}

	if (Builder.ExportOptions->BakeMaterialInputs == EGLTFMaterialBakeMode::Disabled)
	{
		Builder.LogWarning(FString::Printf(
			TEXT("%s and %s for material %s needs to bake, but material baking is disabled by export options"),
			*BaseColorProperty.ToString(),
			*OpacityProperty.ToString(),
			*Material->GetName()));
		return false;
	}

	const FIntPoint TextureSize = GetBakeSize(BaseColorProperty, OpacityProperty);
	const TextureAddress TextureAddress = Builder.GetBakeTilingForMaterialProperty(Material, GetPropertyGroup(BaseColorProperty));
	const TextureFilter TextureFilter = Builder.GetBakeFilterForMaterialProperty(Material, GetPropertyGroup(BaseColorProperty));

	const FGLTFPropertyBakeOutput BaseColorBakeOutput = BakeMaterialProperty(BaseColorProperty, BaseColorTexCoord, BaseColorTransform, TextureSize, false);
	const FGLTFPropertyBakeOutput OpacityBakeOutput = BakeMaterialProperty(OpacityProperty, OpacityTexCoord, OpacityTransform, TextureSize, false);
	const float BaseColorScale = BaseColorProperty == MP_EmissiveColor ? BaseColorBakeOutput.EmissiveScale : 1;

	// Detect when both baked properties are constants, which means we can avoid exporting a texture
	if (BaseColorBakeOutput.bIsConstant && OpacityBakeOutput.bIsConstant)
	{
		FLinearColor BaseColorFactor(BaseColorBakeOutput.ConstantValue * BaseColorScale);
		BaseColorFactor.A = OpacityBakeOutput.ConstantValue.R;

		OutPBRParams.BaseColorFactor = FGLTFCoreUtilities::ConvertColor(BaseColorFactor);
		return true;
	}

	if (Builder.ExportOptions->TextureImageFormat == EGLTFTextureImageFormat::None)
	{
		return true;
	}

	int32 CombinedTexCoord;
	FGLTFJsonTextureTransform CombinedTransform;

	// If one is constant, it means we can use the other's texture coords
	if (BaseColorBakeOutput.bIsConstant || BaseColorTexCoord == OpacityTexCoord)
	{
		CombinedTexCoord = OpacityTexCoord;
		CombinedTransform = OpacityTransform; // If texcoord the same, then transform also the same
	}
	else if (OpacityBakeOutput.bIsConstant)
	{
		CombinedTexCoord = BaseColorTexCoord;
		CombinedTransform = BaseColorTransform;
	}
	else
	{
		// TODO: report error (texture coordinate conflict)
		return false;
	}

	TGLTFSharedArray<FColor> CombinedPixels;
	CombinePixels(
		*BaseColorBakeOutput.Pixels,
		*OpacityBakeOutput.Pixels,
		*CombinedPixels,
		[](const FColor& BaseColor, const FColor& Opacity) -> FColor
		{
			return FColor(BaseColor.R, BaseColor.G, BaseColor.B, Opacity.R);
		});

	FGLTFJsonTexture* CombinedTexture = FGLTFMaterialUtilities::AddTexture(
		Builder,
		CombinedPixels,
		TextureSize,
		false,
		false,
		GetBakedTextureName(TEXT("BaseColor")),
		TextureAddress,
		TextureFilter);

	OutPBRParams.BaseColorTexture.Index = CombinedTexture;
	OutPBRParams.BaseColorTexture.TexCoord = CombinedTexCoord;
	OutPBRParams.BaseColorTexture.Transform = CombinedTransform;

	// TODO: add warning if BaseColorScale exceeds 1.0 (maybe suggest disabling unlit materials?)
	OutPBRParams.BaseColorFactor = FGLTFCoreUtilities::ConvertColor(FLinearColor(BaseColorScale, BaseColorScale, BaseColorScale));

	return true;
}

bool FGLTFDelayedMaterialTask::TryGetMetallicAndRoughness(FGLTFJsonPBRMetallicRoughness& OutPBRParams, const FMaterialPropertyEx& MetallicProperty, const FMaterialPropertyEx& RoughnessProperty)
{
	const bool bIsMetallicConstant = TryGetConstantScalar(OutPBRParams.MetallicFactor, MetallicProperty);
	const bool bIsRoughnessConstant = TryGetConstantScalar(OutPBRParams.RoughnessFactor, RoughnessProperty);

	if (bIsMetallicConstant && bIsRoughnessConstant)
	{
		return true;
	}

	// NOTE: since we always bake the properties (for now) when at least one property is non-const, we need
	// to reset the constant factors to their defaults. Otherwise the baked value of a constant property
	// would be scaled with the factor, i.e a double scaling.
	OutPBRParams.MetallicFactor = 1.0f;
	OutPBRParams.RoughnessFactor = 1.0f;

	const UTexture2D* MetallicTexture;
	const UTexture2D* RoughnessTexture;
	int32 MetallicTexCoord;
	int32 RoughnessTexCoord;
	FGLTFJsonTextureTransform MetallicTransform;
	FGLTFJsonTextureTransform RoughnessTransform;

	const bool bHasMetallicSourceTexture = TryGetSourceTexture(MetallicTexture, MetallicTexCoord, MetallicTransform, MetallicProperty, MetallicInputMasks);
	const bool bHasRoughnessSourceTexture = TryGetSourceTexture(RoughnessTexture, RoughnessTexCoord, RoughnessTransform, RoughnessProperty, RoughnessInputMasks);

	// Detect the "happy path" where both inputs share the same texture and are correctly masked.
	if (bHasMetallicSourceTexture &&
		bHasRoughnessSourceTexture &&
		MetallicTexture == RoughnessTexture &&
		MetallicTexCoord == RoughnessTexCoord &&
		MetallicTransform.IsExactlyEqual(RoughnessTransform))
	{
		OutPBRParams.MetallicRoughnessTexture.Index = Builder.AddUniqueTexture(MetallicTexture, false);
		OutPBRParams.MetallicRoughnessTexture.TexCoord = MetallicTexCoord;
		OutPBRParams.MetallicRoughnessTexture.Transform = MetallicTransform;
		return true;
	}

	if (Builder.ExportOptions->BakeMaterialInputs == EGLTFMaterialBakeMode::Disabled)
	{
		Builder.LogWarning(FString::Printf(
			TEXT("%s and %s for material %s needs to bake, but material baking is disabled by export options"),
			*MetallicProperty.ToString(),
			*RoughnessProperty.ToString(),
			*Material->GetName()));
		return false;
	}

	const FIntPoint TextureSize = GetBakeSize(MetallicProperty, RoughnessProperty);
	const TextureAddress TextureAddress = Builder.GetBakeTilingForMaterialProperty(Material, GetPropertyGroup(MetallicProperty));
	const TextureFilter TextureFilter = Builder.GetBakeFilterForMaterialProperty(Material, GetPropertyGroup(MetallicProperty));

	FGLTFPropertyBakeOutput MetallicBakeOutput = BakeMaterialProperty(MetallicProperty, MetallicTexCoord, MetallicTransform, TextureSize, false);
	FGLTFPropertyBakeOutput RoughnessBakeOutput = BakeMaterialProperty(RoughnessProperty, RoughnessTexCoord, RoughnessTransform, TextureSize, false);

	// Detect when both baked properties are constants, which means we can use factors and avoid exporting a texture
	if (MetallicBakeOutput.bIsConstant && RoughnessBakeOutput.bIsConstant)
	{
		OutPBRParams.MetallicFactor = MetallicBakeOutput.ConstantValue.R;
		OutPBRParams.RoughnessFactor = RoughnessBakeOutput.ConstantValue.R;
		return true;
	}

	if (Builder.ExportOptions->TextureImageFormat == EGLTFTextureImageFormat::None)
	{
		return true;
	}

	int32 CombinedTexCoord;
	FGLTFJsonTextureTransform CombinedTransform;

	// If one is constant, it means we can use the other's texture coords
	if (MetallicBakeOutput.bIsConstant || MetallicTexCoord == RoughnessTexCoord)
	{
		CombinedTexCoord = RoughnessTexCoord;
		CombinedTransform = RoughnessTransform; // If texcoord the same, then transform also the same
	}
	else if (RoughnessBakeOutput.bIsConstant)
	{
		CombinedTexCoord = MetallicTexCoord;
		CombinedTransform = MetallicTransform;
	}
	else
	{
		// TODO: report error (texture coordinate conflict)
		return false;
	}

	TGLTFSharedArray<FColor> CombinedPixels;
	CombinePixels(
		*MetallicBakeOutput.Pixels,
		*RoughnessBakeOutput.Pixels,
		*CombinedPixels,
		[](const FColor& Metallic, const FColor& Roughness) -> FColor
		{
			return FColor(0, Roughness.R, Metallic.R);
		});

	FGLTFJsonTexture* CombinedTexture = FGLTFMaterialUtilities::AddTexture(
		Builder,
		CombinedPixels,
		TextureSize,
		true, // NOTE: we can ignore alpha in everything but TryGetBaseColorAndOpacity and TryGetSpecular
		false,
		GetBakedTextureName(TEXT("MetallicRoughness")),
		TextureAddress,
		TextureFilter);

	OutPBRParams.MetallicRoughnessTexture.Index = CombinedTexture;
	OutPBRParams.MetallicRoughnessTexture.TexCoord = CombinedTexCoord;
	OutPBRParams.MetallicRoughnessTexture.Transform = CombinedTransform;

	return true;
}

bool FGLTFDelayedMaterialTask::TryGetClearCoatRoughness(FGLTFJsonClearCoatExtension& OutExtParams, const FMaterialPropertyEx& IntensityProperty, const FMaterialPropertyEx& RoughnessProperty)
{
	const bool bIsIntensityConstant = TryGetConstantScalar(OutExtParams.ClearCoatFactor, IntensityProperty);
	const bool bIsRoughnessConstant = TryGetConstantScalar(OutExtParams.ClearCoatRoughnessFactor, RoughnessProperty);

	if (bIsIntensityConstant && bIsRoughnessConstant)
	{
		return true;
	}

	// NOTE: since we always bake the properties (for now) when at least one property is non-const, we need
	// to reset the constant factors to their defaults. Otherwise the baked value of a constant property
	// would be scaled with the factor, i.e a double scaling.
	OutExtParams.ClearCoatFactor = 1.0f;
	OutExtParams.ClearCoatRoughnessFactor = 1.0f;

	const UTexture2D* IntensityTexture;
	const UTexture2D* RoughnessTexture;
	int32 IntensityTexCoord;
	int32 RoughnessTexCoord;
	FGLTFJsonTextureTransform IntensityTransform;
	FGLTFJsonTextureTransform RoughnessTransform;

	const bool bHasIntensitySourceTexture = TryGetSourceTexture(IntensityTexture, IntensityTexCoord, IntensityTransform, IntensityProperty, ClearCoatInputMasks);
	const bool bHasRoughnessSourceTexture = TryGetSourceTexture(RoughnessTexture, RoughnessTexCoord, RoughnessTransform, RoughnessProperty, ClearCoatRoughnessInputMasks);

	// Detect the "happy path" where both inputs share the same texture and are correctly masked.
	if (bHasIntensitySourceTexture &&
		bHasRoughnessSourceTexture &&
		IntensityTexture == RoughnessTexture &&
		IntensityTexCoord == RoughnessTexCoord &&
		IntensityTransform.IsExactlyEqual(RoughnessTransform))
	{
		FGLTFJsonTexture* Texture = Builder.AddUniqueTexture(IntensityTexture, false);
		OutExtParams.ClearCoatTexture.Index = Texture;
		OutExtParams.ClearCoatTexture.TexCoord = IntensityTexCoord;
		OutExtParams.ClearCoatRoughnessTexture.Index = Texture;
		OutExtParams.ClearCoatRoughnessTexture.TexCoord = IntensityTexCoord;
		OutExtParams.ClearCoatRoughnessTexture.Transform = IntensityTransform;
		return true;
	}

	if (Builder.ExportOptions->BakeMaterialInputs == EGLTFMaterialBakeMode::Disabled)
	{
		Builder.LogWarning(FString::Printf(
			TEXT("%s and %s for material %s needs to bake, but material baking is disabled by export options"),
			*IntensityProperty.ToString(),
			*RoughnessProperty.ToString(),
			*Material->GetName()));
		return false;
	}

	const FIntPoint TextureSize = GetBakeSize(IntensityProperty, RoughnessProperty);
	const TextureAddress TextureAddress = Builder.GetBakeTilingForMaterialProperty(Material,GetPropertyGroup(IntensityProperty));
	const TextureFilter TextureFilter = Builder.GetBakeFilterForMaterialProperty(Material, GetPropertyGroup(IntensityProperty));

	const FGLTFPropertyBakeOutput IntensityBakeOutput = BakeMaterialProperty(IntensityProperty, IntensityTexCoord, IntensityTransform, TextureSize, false);
	const FGLTFPropertyBakeOutput RoughnessBakeOutput = BakeMaterialProperty(RoughnessProperty, RoughnessTexCoord, RoughnessTransform, TextureSize, false);

	// Detect when both baked properties are constants, which means we can use factors and avoid exporting a texture
	if (IntensityBakeOutput.bIsConstant && RoughnessBakeOutput.bIsConstant)
	{
		OutExtParams.ClearCoatFactor = IntensityBakeOutput.ConstantValue.R;
		OutExtParams.ClearCoatRoughnessFactor = RoughnessBakeOutput.ConstantValue.R;
		return true;
	}

	if (Builder.ExportOptions->TextureImageFormat == EGLTFTextureImageFormat::None)
	{
		return true;
	}

	int32 CombinedTexCoord;
	FGLTFJsonTextureTransform CombinedTransform;

	// If one is constant, it means we can use the other's texture coords
	if (IntensityBakeOutput.bIsConstant || IntensityTexCoord == RoughnessTexCoord)
	{
		CombinedTexCoord = RoughnessTexCoord;
		CombinedTransform = RoughnessTransform; // If texcoord the same, then transform also the same
	}
	else if (RoughnessBakeOutput.bIsConstant)
	{
		CombinedTexCoord = IntensityTexCoord;
		CombinedTransform = IntensityTransform;
	}
	else
	{
		// TODO: report error (texture coordinate conflict)
		return false;
	}

	TGLTFSharedArray<FColor> CombinedPixels;
	CombinePixels(
		*IntensityBakeOutput.Pixels,
		*RoughnessBakeOutput.Pixels,
		*CombinedPixels,
		[](const FColor& Intensity, const FColor& Roughness) -> FColor
		{
			return FColor(Intensity.R, Roughness.R, 0);
		});

	FGLTFJsonTexture* CombinedTexture = FGLTFMaterialUtilities::AddTexture(
		Builder,
		CombinedPixels,
		TextureSize,
		true, // NOTE: we can ignore alpha in everything but TryGetBaseColorAndOpacity
		false,
		GetBakedTextureName(TEXT("ClearCoatRoughness")),
		TextureAddress,
		TextureFilter);

	OutExtParams.ClearCoatTexture.Index = CombinedTexture;
	OutExtParams.ClearCoatTexture.TexCoord = CombinedTexCoord;
	OutExtParams.ClearCoatTexture.Transform = CombinedTransform;
	OutExtParams.ClearCoatRoughnessTexture.Index = CombinedTexture;
	OutExtParams.ClearCoatRoughnessTexture.TexCoord = CombinedTexCoord;
	OutExtParams.ClearCoatRoughnessTexture.Transform = CombinedTransform;

	return true;
}

bool FGLTFDelayedMaterialTask::TryGetEmissive(FGLTFJsonMaterial& OutMaterial, const FMaterialPropertyEx& EmissiveProperty)
{
	FLinearColor ConstantColor;
	if (TryGetConstantColor(ConstantColor, MP_EmissiveColor))
	{
		const float EmissiveStrength = ConstantColor.GetMax();
		if (EmissiveStrength > 1.0)
		{
			ConstantColor *= 1.0f / EmissiveStrength;
			OutMaterial.EmissiveStrength = EmissiveStrength;
		}

		OutMaterial.EmissiveFactor = FGLTFCoreUtilities::ConvertColor3(ConstantColor);
	}
	else if (TryGetSourceTexture(OutMaterial.EmissiveTexture, EmissiveProperty, DefaultColorInputMasks))
	{
		// TODO: what if texture is HDR and one or more pixel components exceeds 1.0? Should we process it and add emissive strength?
		OutMaterial.EmissiveFactor = FGLTFJsonColor3::White; // make sure texture is not multiplied with black
	}
	else
	{
		if (Builder.ExportOptions->BakeMaterialInputs == EGLTFMaterialBakeMode::Disabled)
		{
			Builder.LogWarning(FString::Printf(
				TEXT("%s for material %s needs to bake, but material baking is disabled by export options"),
				*EmissiveProperty.ToString(),
				*Material->GetName()));
			return false;
		}

		FGLTFPropertyBakeOutput PropertyBakeOutput = BakeMaterialProperty(EmissiveProperty, OutMaterial.EmissiveTexture.TexCoord, OutMaterial.EmissiveTexture.Transform);

		if (PropertyBakeOutput.bIsConstant)
		{
			FLinearColor EmissiveColor = PropertyBakeOutput.ConstantValue;
			const float EmissiveScale = PropertyBakeOutput.EmissiveScale;

			if (EmissiveScale < 1.0f)
			{
				EmissiveColor *= EmissiveScale;
			}
			else
			{
				OutMaterial.EmissiveStrength = EmissiveScale;
			}

			OutMaterial.EmissiveFactor = FGLTFCoreUtilities::ConvertColor3(EmissiveColor);
		}
		else
		{
			if (Builder.ExportOptions->TextureImageFormat == EGLTFTextureImageFormat::None)
			{
				OutMaterial.EmissiveTexture.Index = nullptr;
				return true;
			}

			if (!StoreBakedPropertyTexture(EmissiveProperty, OutMaterial.EmissiveTexture, PropertyBakeOutput, TEXT("Emissive")))
			{
				return false;
			}

			const float EmissiveScale = PropertyBakeOutput.EmissiveScale;
			if (EmissiveScale < 1.0f)
			{
				OutMaterial.EmissiveFactor = { EmissiveScale, EmissiveScale, EmissiveScale };
			}
			else
			{
				OutMaterial.EmissiveFactor = FGLTFJsonColor3::White;
				OutMaterial.EmissiveStrength = EmissiveScale;
			}
		}
	}

	if (OutMaterial.EmissiveStrength > 1.0f && !Builder.ExportOptions->bExportEmissiveStrength)
	{
		// TODO: add warning about clamping emissive strength?
		OutMaterial.EmissiveStrength = 1.0f;
	}

	return true;
}

bool FGLTFDelayedMaterialTask::TryGetSpecular(FGLTFJsonMaterial& OutMaterial, const FMaterialPropertyEx& SpecularProperty)
{
	float Factor;
	if (TryGetConstantScalar(Factor, MP_Specular))
	{
		OutMaterial.Specular.Factor = Factor;
	}
	else if (TryGetSourceTexture(OutMaterial.Specular.Texture, SpecularProperty, { AlphaMask }))
	{
		OutMaterial.Specular.Factor = 1.0f;
	}
	else
	{
		if (!TryGetBakedMaterialProperty(OutMaterial.Specular.Texture, OutMaterial.Specular.Factor, 1.0f, SpecularProperty, 
			[](FColor& Color, const uint8& ChannelValueToMove)
			{
				Color.A = ChannelValueToMove;

				Color.R = 255;
				Color.G = 255;
				Color.B = 255;
			}
		))
		{
			return false;
		}
	}

	return true;
}

bool FGLTFDelayedMaterialTask::TryGetRefraction(FGLTFJsonMaterial& OutMaterial, const FMaterialPropertyEx& RefractionProperty)
{
	if (!TryGetConstantScalar(OutMaterial.IOR.Value, RefractionProperty))
	{
		if (Builder.ExportOptions->BakeMaterialInputs == EGLTFMaterialBakeMode::Disabled)
		{
			Builder.LogWarning(FString::Printf(
				TEXT("%s for material %s needs to bake, but material baking is disabled by export options"),
				*RefractionProperty.ToString(),
				*Material->GetName()));
			return false;
		}

		int32 TempTexCoord;
		FGLTFJsonTextureTransform TempTransform;
		const FIntPoint TextureSize(128);
		//Note: Refraction baking limits range to [1, Infinity]
		FGLTFPropertyBakeOutput PropertyBakeOutput = BakeMaterialProperty(RefractionProperty, TempTexCoord, TempTransform, TextureSize, false);
		if (PropertyBakeOutput.bIsConstant)
		{
			OutMaterial.IOR.Value = PropertyBakeOutput.ConstantValue.R;
		}
		else
		{
			//Calculate an avarage constant value from baked texture:
			Builder.LogWarning(FString::Printf(
				TEXT("In material %s : The %s property is using non const value. Avarage const value approximation is used as the glTF 2.0 standard only supports const value for said property (via khr_materials_ior). "),
				*Material->GetName(),
				*RefractionProperty.ToString()));

			uint64 Red, Green, Blue;
			Red = Green = Blue = 0;
			for (const FColor& Pixel : *PropertyBakeOutput.Pixels)
			{
				Red += Pixel.R;
				Green += Pixel.G;
				Blue += Pixel.B;
			}

			OutMaterial.IOR.Value = (Red + Green + Blue) / (3 * PropertyBakeOutput.Pixels.Get().Num() * 255.);
		}

		//Baking shifts the Range of Refraction from, to:
		// [1,Infinity] -> [1,0]
		OutMaterial.IOR.Value = 1. / OutMaterial.IOR.Value;
	}

	return true;
}

bool FGLTFDelayedMaterialTask::TryGetFuzzColorAndCloth(FGLTFJsonMaterial& OutMaterial, const FMaterialPropertyEx& FuzzColorProperty, const FMaterialPropertyEx& ClothProperty)
{
	//SheenColorFactor + SheenColorTexture(RGB)
	if (!TryGetConstantColor(OutMaterial.Sheen.ColorFactor, FuzzColorProperty))
	{
		if (!TryGetSourceTexture(OutMaterial.Sheen.ColorTexture, FuzzColorProperty, { RgbMask }))
		{
			if (!TryGetBakedMaterialProperty(OutMaterial.Sheen.ColorTexture, OutMaterial.Sheen.ColorFactor, FuzzColorProperty, TEXT("SheenColorTexture")))
			{
				return false;
			}
		}
		else
		{
			OutMaterial.Sheen.ColorFactor = FGLTFJsonColor3::White;
		}
	}

	//SheenRoughnessFactor + SheenRoughnessTexture(A)
	if (!TryGetConstantScalar(OutMaterial.Sheen.RoughnessFactor, ClothProperty))
	{
		if (!TryGetSourceTexture(OutMaterial.Sheen.RoughnessTexture, ClothProperty, { AlphaMask }))
		{
			if (!TryGetBakedMaterialProperty(OutMaterial.Sheen.RoughnessTexture, OutMaterial.Sheen.RoughnessFactor, 1.0f, ClothProperty,
				[](FColor& Color, const uint8& ChannelValueToMove)
				{
					Color.A = ChannelValueToMove;

					Color.R = 255;
					Color.G = 255;
					Color.B = 255;
				},
				TEXT("SheenRoughnessTexture")))
			{
				return false;
			}
		}
		else
		{
			OutMaterial.Sheen.RoughnessFactor = 1.0f;
		}
	}

	return true;
}

bool FGLTFDelayedMaterialTask::TryGetTransmissionBaseColorAndOpacity(FGLTFJsonMaterial& OutMaterial, const FMaterialPropertyEx& BaseColorProperty, const FMaterialPropertyEx& OpacityProperty)
{
	const bool bIsBaseColorConstant = TryGetConstantColor(OutMaterial.PBRMetallicRoughness.BaseColorFactor, BaseColorProperty);

	float Opacity = 0.0f;
	const bool bIsOpacityConstant = TryGetConstantScalar(Opacity, OpacityProperty);

	if (bIsBaseColorConstant)
	{
		OutMaterial.PBRMetallicRoughness.BaseColorFactor.A = 1.0f;
	}
	if (bIsOpacityConstant)
	{
		OutMaterial.Transmission.Factor = 1 - Opacity;
	}

	if (bIsBaseColorConstant && bIsOpacityConstant)
	{
		return true;
	}

	if (!bIsBaseColorConstant)
	{
		if (!TryGetBakedMaterialProperty(OutMaterial.PBRMetallicRoughness.BaseColorTexture, OutMaterial.PBRMetallicRoughness.BaseColorFactor, BaseColorProperty, TEXT("BaseColor")))
		{
			return false;
		}
	}

	if (!bIsOpacityConstant)
	{
		if (!TryGetBakedMaterialProperty(OutMaterial.Transmission.Texture, OutMaterial.Transmission.Factor, 1.0f, OpacityProperty,
			[](FColor& Color, const uint8& ChannelValueToMove)
			{
				Color.R = 255 - ChannelValueToMove;

				Color.G = 255;
				Color.B = 255;
				Color.A = 255;
			},
			TEXT("TransmissionTexture")))
		{
			return false;
		}
	}

	return true;
}

bool FGLTFDelayedMaterialTask::IsPropertyNonDefault(const FMaterialPropertyEx& Property) const
{
	// Custom outputs are by definition standalone and never part of material attributes
	if (!Property.IsCustomOutput())
	{
		const bool bUseMaterialAttributes = Material->GetMaterial()->bUseMaterialAttributes;
		if (bUseMaterialAttributes)
		{
			// TODO: check if attribute property connected, i.e. Material->GetMaterial()->MaterialAttributes.IsConnected(Property)
			return true;
		}
	}

	const FExpressionInput* MaterialInput = FGLTFMaterialUtilities::GetInputForProperty(Material, Property);
	if (MaterialInput == nullptr)
	{
		return false;
	}

	const UMaterialExpression* Expression = MaterialInput->Expression;
	if (Expression == nullptr)
	{
		return false;
	}

	if (Property == FMaterialPropertyEx::ClearCoatBottomNormal && !FGLTFMaterialUtilities::IsClearCoatBottomNormalEnabled())
	{
		Builder.LogWarning(FString::Printf(
			TEXT("Material %s won't be exported with clear coat bottom normal because ClearCoatEnableSecondNormal in project rendering settings is disabled"),
			*Material->GetName()));
		return false;
	}

	return true;
}

bool FGLTFDelayedMaterialTask::TryGetConstantColor(FGLTFJsonColor3& OutValue, const FMaterialPropertyEx& Property) const
{
	FLinearColor Value;
	if (TryGetConstantColor(Value, Property))
	{
		OutValue = FGLTFCoreUtilities::ConvertColor3(Value);
		return true;
	}

	return false;
}

bool FGLTFDelayedMaterialTask::TryGetConstantColor(FGLTFJsonColor4& OutValue, const FMaterialPropertyEx& Property) const
{
	FLinearColor Value;
	if (TryGetConstantColor(Value, Property))
	{
		OutValue = FGLTFCoreUtilities::ConvertColor(Value);
		return true;
	}

	return false;
}

bool FGLTFDelayedMaterialTask::TryGetConstantColor(FLinearColor& OutValue, const FMaterialPropertyEx& Property) const
{
	const bool bUseMaterialAttributes = Material->GetMaterial()->bUseMaterialAttributes;
	if (bUseMaterialAttributes)
	{
		// TODO: check if attribute property connected, i.e. Material->GetMaterial()->MaterialAttributes.IsConnected(Property)
		return false;
	}

	const FMaterialInput<FLinearColor>* MaterialInput = FGLTFMaterialUtilities::GetInputForProperty<FLinearColor>(Material, Property);
	if (MaterialInput == nullptr)
	{
		// TODO: report error
		return false;
	}

	if (MaterialInput->UseConstant)
	{
		OutValue = { MaterialInput->Constant };
		return true;
	}

	const UMaterialExpression* Expression = MaterialInput->Expression;
	if (Expression == nullptr)
	{
		OutValue = FLinearColor(FGLTFMaterialUtilities::GetPropertyDefaultValue(Property));
		return true;
	}

	if (const UMaterialExpressionVectorParameter* VectorParameter = ExactCast<UMaterialExpressionVectorParameter>(Expression))
	{
		FLinearColor Value = VectorParameter->DefaultValue;

		const UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Material);
		if (MaterialInstance != nullptr)
		{
			const FHashedMaterialParameterInfo ParameterInfo(VectorParameter->GetParameterName());
			if (!MaterialInstance->GetVectorParameterValue(ParameterInfo, Value))
			{
				// TODO: how to handle this?
			}
		}

		const uint32 MaskComponentCount = FGLTFMaterialUtilities::GetMaskComponentCount(*MaterialInput);

		if (MaskComponentCount > 0)
		{
			const FLinearColor Mask = FGLTFMaterialUtilities::GetMask(*MaterialInput);

			Value *= Mask;

			if (MaskComponentCount == 1)
			{
				const float ComponentValue = Value.R + Value.G + Value.B + Value.A;
				Value = { ComponentValue, ComponentValue, ComponentValue, ComponentValue };
			}
		}

		OutValue = Value;
		return true;
	}

	if (const UMaterialExpressionScalarParameter* ScalarParameter = ExactCast<UMaterialExpressionScalarParameter>(Expression))
	{
		float Value = ScalarParameter->DefaultValue;

		const UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Material);
		if (MaterialInstance != nullptr)
		{
			const FHashedMaterialParameterInfo ParameterInfo(ScalarParameter->GetParameterName());
			if (!MaterialInstance->GetScalarParameterValue(ParameterInfo, Value))
			{
				// TODO: how to handle this?
			}
		}

		OutValue = { Value, Value, Value, Value };
		return true;
	}

	if (const UMaterialExpressionConstant4Vector* Constant4Vector = ExactCast<UMaterialExpressionConstant4Vector>(Expression))
	{
		OutValue = Constant4Vector->Constant;
		return true;
	}

	if (const UMaterialExpressionConstant3Vector* Constant3Vector = ExactCast<UMaterialExpressionConstant3Vector>(Expression))
	{
		OutValue = Constant3Vector->Constant;
		return true;
	}

	if (const UMaterialExpressionConstant2Vector* Constant2Vector = ExactCast<UMaterialExpressionConstant2Vector>(Expression))
	{
		OutValue = { Constant2Vector->R, Constant2Vector->G, 0, 0 };
		return true;
	}

	if (const UMaterialExpressionConstant* Constant = ExactCast<UMaterialExpressionConstant>(Expression))
	{
		OutValue = { Constant->R, Constant->R, Constant->R, Constant->R };
		return true;
	}

	return false;
}

bool FGLTFDelayedMaterialTask::TryGetConstantScalar(float& OutValue, const FMaterialPropertyEx& Property) const
{
	const bool bUseMaterialAttributes = Material->GetMaterial()->bUseMaterialAttributes;
	if (bUseMaterialAttributes)
	{
		// TODO: check if attribute property connected, i.e. Material->GetMaterial()->MaterialAttributes.IsConnected(Property)
		return false;
	}

	const FMaterialInput<float>* MaterialInput = FGLTFMaterialUtilities::GetInputForProperty<float>(Material, Property);
	if (MaterialInput == nullptr)
	{
		// TODO: report error
		return false;
	}

	if (MaterialInput->UseConstant)
	{
		OutValue = MaterialInput->Constant;
		return true;
	}

	const UMaterialExpression* Expression = MaterialInput->Expression;
	if (Expression == nullptr)
	{
		OutValue = FGLTFMaterialUtilities::GetPropertyDefaultValue(Property).X;
		return true;
	}

	if (const UMaterialExpressionVectorParameter* VectorParameter = ExactCast<UMaterialExpressionVectorParameter>(Expression))
	{
		FLinearColor Value = VectorParameter->DefaultValue;

		const UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Material);
		if (MaterialInstance != nullptr)
		{
			const FHashedMaterialParameterInfo ParameterInfo(VectorParameter->GetParameterName());
			if (!MaterialInstance->GetVectorParameterValue(ParameterInfo, Value))
			{
				// TODO: how to handle this?
			}
		}

		const uint32 MaskComponentCount = FGLTFMaterialUtilities::GetMaskComponentCount(*MaterialInput);

		if (MaskComponentCount > 0)
		{
			const FLinearColor Mask = FGLTFMaterialUtilities::GetMask(*MaterialInput);
			Value *= Mask;
		}

		// TODO: is this a correct assumption, that the max component should be used as value?
		OutValue = Value.GetMax();
		return true;
	}

	if (const UMaterialExpressionScalarParameter* ScalarParameter = ExactCast<UMaterialExpressionScalarParameter>(Expression))
	{
		float Value = ScalarParameter->DefaultValue;

		const UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Material);
		if (MaterialInstance != nullptr)
		{
			const FHashedMaterialParameterInfo ParameterInfo(ScalarParameter->GetParameterName());
			if (!MaterialInstance->GetScalarParameterValue(ParameterInfo, Value))
			{
				// TODO: how to handle this?
			}
		}

		OutValue = Value;
		return true;
	}

	if (const UMaterialExpressionConstant4Vector* Constant4Vector = ExactCast<UMaterialExpressionConstant4Vector>(Expression))
	{
		OutValue = Constant4Vector->Constant.R;
		return true;
	}

	if (const UMaterialExpressionConstant3Vector* Constant3Vector = ExactCast<UMaterialExpressionConstant3Vector>(Expression))
	{
		OutValue = Constant3Vector->Constant.R;
		return true;
	}

	if (const UMaterialExpressionConstant2Vector* Constant2Vector = ExactCast<UMaterialExpressionConstant2Vector>(Expression))
	{
		OutValue = Constant2Vector->R;
		return true;
	}

	if (const UMaterialExpressionConstant* Constant = ExactCast<UMaterialExpressionConstant>(Expression))
	{
		OutValue = Constant->R;
		return true;
	}

	return false;
}

bool FGLTFDelayedMaterialTask::TryGetSourceTexture(FGLTFJsonTextureInfo& OutTexInfo, const FMaterialPropertyEx& Property, const TArray<FLinearColor>& AllowedMasks) const
{
	const UTexture2D* Texture;
	int32 TexCoord;
	FGLTFJsonTextureTransform Transform;

	if (TryGetSourceTexture(Texture, TexCoord, Transform, Property, AllowedMasks))
	{
		OutTexInfo.Index = Builder.AddUniqueTexture(Texture, FGLTFMaterialUtilities::IsSRGB(Property) || (Property == MP_Specular && Texture->SRGB));
		OutTexInfo.TexCoord = TexCoord;
		OutTexInfo.Transform = Transform;
		return true;
	}

	return false;
}

bool FGLTFDelayedMaterialTask::TryGetSourceTexture(const UTexture2D*& OutTexture, int32& OutTexCoord, FGLTFJsonTextureTransform& OutTransform, const FMaterialPropertyEx& Property, const TArray<FLinearColor>& AllowedMasks) const
{
	if (Builder.ExportOptions->TextureImageFormat == EGLTFTextureImageFormat::None)
	{
		return false;
	}

	const FExpressionInput* MaterialInput = FGLTFMaterialUtilities::GetInputForProperty(Material, Property);
	if (MaterialInput == nullptr)
	{
		// TODO: report error
		return false;
	}

	const UMaterialExpression* Expression = MaterialInput->Expression;
	if (Expression == nullptr)
	{
		return false;
	}

	const FLinearColor InputMask = FGLTFMaterialUtilities::GetMask(*MaterialInput);
	if (AllowedMasks.Num() > 0 && !AllowedMasks.Contains(InputMask))
	{
		return false;
	}

	// TODO: add support or warning for texture sampler settings that override texture asset addressing (i.e. wrap, clamp etc)?

	if (const UMaterialExpressionTextureSampleParameter2D* TextureParameter = ExactCast<UMaterialExpressionTextureSampleParameter2D>(Expression))
	{
		// TODO: handle non-default TextureParameter->SamplerType and TextureParameter->SamplerSource

		UTexture* ParameterValue = TextureParameter->Texture;

		const UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Material);
		if (MaterialInstance != nullptr)
		{
			const FHashedMaterialParameterInfo ParameterInfo(TextureParameter->GetParameterName());
			if (!MaterialInstance->GetTextureParameterValue(ParameterInfo, ParameterValue))
			{
				// TODO: how to handle this?
			}
		}

		// TODO: add support for UTextureRenderTarget2D?
		OutTexture = Cast<UTexture2D>(ParameterValue);

		if (OutTexture == nullptr)
		{
			if (ParameterValue == nullptr)
			{
				// TODO: report error (no texture parameter assigned)
			}
			else
			{
				// TODO: report error (incorrect texture type)
			}
			return false;
		}

		if (!FGLTFMaterialUtilities::TryGetTextureCoordinateIndex(TextureParameter, OutTexCoord, OutTransform))
		{
			// TODO: report error (failed to identify texture coordinate index)
			return false;
		}

		if (!Builder.ExportOptions->bExportTextureTransforms && !OutTransform.IsExactlyDefault())
		{
			Builder.LogWarning(FString::Printf(
				TEXT("Texture coordinates [%d] in %s for material %s are transformed, but texture transform is disabled by export options"),
				OutTexCoord,
				*Property.ToString(),
				*Material->GetName()));
			OutTransform = {};
		}

		return true;
	}

	if (const UMaterialExpressionTextureSample* TextureSampler = ExactCast<UMaterialExpressionTextureSample>(Expression))
	{
		// TODO: handle non-default TextureSampler->SamplerType and TextureSampler->SamplerSource

		// TODO: add support for texture object input expression

		// TODO: add support for UTextureRenderTarget2D?
		OutTexture = Cast<UTexture2D>(TextureSampler->Texture);

		if (OutTexture == nullptr)
		{
			if (TextureSampler->Texture == nullptr)
			{
				// TODO: report error (no texture sample assigned)
			}
			else
			{
				// TODO: report error (incorrect texture type)
			}
			return false;
		}

		if (!FGLTFMaterialUtilities::TryGetTextureCoordinateIndex(TextureSampler, OutTexCoord, OutTransform))
		{
			// TODO: report error (failed to identify texture coordinate index)
			return false;
		}

		if (!Builder.ExportOptions->bExportTextureTransforms && !OutTransform.IsExactlyDefault())
		{
			Builder.LogWarning(FString::Printf(
				TEXT("Texture coordinates [%d] in %s for material %s are transformed, but texture transform is disabled by export options"),
				OutTexCoord,
				*Property.ToString(),
				*Material->GetName()));
			OutTransform = {};
		}

		return true;
	}

	return false;
}

bool FGLTFDelayedMaterialTask::TryGetBakedMaterialProperty(FGLTFJsonTextureInfo& OutTexInfo, FGLTFJsonColor3& OutConstant, const FMaterialPropertyEx& Property, const FString& PropertyName)
{
	if (Builder.ExportOptions->BakeMaterialInputs == EGLTFMaterialBakeMode::Disabled)
	{
		Builder.LogWarning(FString::Printf(
			TEXT("%s for material %s needs to bake, but material baking is disabled by export options"),
			*Property.ToString(),
			*Material->GetName()));
		return false;
	}

	FGLTFPropertyBakeOutput PropertyBakeOutput = BakeMaterialProperty(Property, OutTexInfo.TexCoord, OutTexInfo.Transform);

	if (PropertyBakeOutput.bIsConstant)
	{
		OutConstant = FGLTFCoreUtilities::ConvertColor3(PropertyBakeOutput.ConstantValue);
		return true;
	}

	if (Builder.ExportOptions->TextureImageFormat == EGLTFTextureImageFormat::None)
	{
		OutTexInfo.Index = nullptr;
		return true;
	}

	if (StoreBakedPropertyTexture(Property, OutTexInfo, PropertyBakeOutput, PropertyName))
	{
		OutConstant = FGLTFJsonColor3::White; // make sure property is not zero
		return true;
	}

	return false;
}

bool FGLTFDelayedMaterialTask::TryGetBakedMaterialProperty(FGLTFJsonTextureInfo& OutTexInfo, FGLTFJsonColor4& OutConstant, const FMaterialPropertyEx& Property, const FString& PropertyName)
{
	if (Builder.ExportOptions->BakeMaterialInputs == EGLTFMaterialBakeMode::Disabled)
	{
		Builder.LogWarning(FString::Printf(
			TEXT("%s for material %s needs to bake, but material baking is disabled by export options"),
			*Property.ToString(),
			*Material->GetName()));
		return false;
	}

	FGLTFPropertyBakeOutput PropertyBakeOutput = BakeMaterialProperty(Property, OutTexInfo.TexCoord, OutTexInfo.Transform);

	if (PropertyBakeOutput.bIsConstant)
	{
		OutConstant = FGLTFCoreUtilities::ConvertColor(PropertyBakeOutput.ConstantValue);
		return true;
	}

	if (Builder.ExportOptions->TextureImageFormat == EGLTFTextureImageFormat::None)
	{
		OutTexInfo.Index = nullptr;
		return true;
	}

	if (StoreBakedPropertyTexture(Property, OutTexInfo, PropertyBakeOutput, PropertyName))
	{
		OutConstant = FGLTFJsonColor4::White; // make sure property is not zero
		return true;
	}

	return false;
}

inline bool FGLTFDelayedMaterialTask::TryGetBakedMaterialProperty(FGLTFJsonTextureInfo& OutTexInfo, float& OutConstant, const FMaterialPropertyEx& Property, const FString& PropertyName)
{
	if (Builder.ExportOptions->BakeMaterialInputs == EGLTFMaterialBakeMode::Disabled)
	{
		Builder.LogWarning(FString::Printf(
			TEXT("%s for material %s needs to bake, but material baking is disabled by export options"),
			*Property.ToString(),
			*Material->GetName()));
		return false;
	}

	FGLTFPropertyBakeOutput PropertyBakeOutput = BakeMaterialProperty(Property, OutTexInfo.TexCoord, OutTexInfo.Transform);

	if (PropertyBakeOutput.bIsConstant)
	{
		OutConstant = PropertyBakeOutput.ConstantValue.R;
		return true;
	}

	if (Builder.ExportOptions->TextureImageFormat == EGLTFTextureImageFormat::None)
	{
		OutTexInfo.Index = nullptr;
		return true;
	}

	if (StoreBakedPropertyTexture(Property, OutTexInfo, PropertyBakeOutput, PropertyName))
	{
		OutConstant = 1; // make sure property is not zero
		return true;
	}

	return false;
}

bool FGLTFDelayedMaterialTask::TryGetBakedMaterialProperty(FGLTFJsonTextureInfo& OutTexInfo, const FMaterialPropertyEx& Property, const FString& PropertyName)
{
	if (Builder.ExportOptions->BakeMaterialInputs == EGLTFMaterialBakeMode::Disabled)
	{
		Builder.LogWarning(FString::Printf(
			TEXT("%s for material %s needs to bake, but material baking is disabled by export options"),
			*Property.ToString(),
			*Material->GetName()));
		return false;
	}

	FGLTFPropertyBakeOutput PropertyBakeOutput = BakeMaterialProperty(Property, OutTexInfo.TexCoord, OutTexInfo.Transform);

	if (!PropertyBakeOutput.bIsConstant)
	{
		if (Builder.ExportOptions->TextureImageFormat == EGLTFTextureImageFormat::None)
		{
			OutTexInfo.Index = nullptr;
			return true;
		}

		return StoreBakedPropertyTexture(Property, OutTexInfo, PropertyBakeOutput, PropertyName);
	}

	const FVector4f MaskedConstant = FVector4f(PropertyBakeOutput.ConstantValue) * FGLTFMaterialUtilities::GetPropertyMask(Property);
	if (MaskedConstant == FGLTFMaterialUtilities::GetPropertyDefaultValue(Property))
	{
		// Constant value is the same as the property's default so we can set gltf to default.
		OutTexInfo.Index = nullptr;
		return true;
	}

	if (FGLTFMaterialUtilities::IsNormalMap(Property))
	{
		// TODO: In some cases baking normal can result in constant vector that differs slight from default (i.e 0,0,1).
		// Yet often, when looking at such a material, it should be exactly default. Needs further investigation.
		// Maybe because of incorrect sRGB conversion? For now, assume a constant normal is always default.
		OutTexInfo.Index = nullptr;
		return true;
	}

	// TODO: let function fail and investigate why in some cases a constant baking result is returned for a property
	// that is non-constant. This happens (for example) when baking AmbientOcclusion for a translucent material,
	// even though the same material when set to opaque will properly bake AmbientOcclusion to a texture.
	// For now, create a 1x1 texture with the constant value.

	FGLTFJsonTexture* Texture = FGLTFMaterialUtilities::AddTexture(
		Builder,
		PropertyBakeOutput.Pixels,
		PropertyBakeOutput.Size,
		true, // NOTE: we can ignore alpha in everything but TryGetBaseColorAndOpacity
		false, // Normal and ClearCoatBottomNormal are handled above
		GetBakedTextureName(PropertyName),
		TA_Clamp,
		TF_Nearest);

	OutTexInfo.Index = Texture;
	return true;
}

template <typename CallbackType>
bool FGLTFDelayedMaterialTask::TryGetBakedMaterialProperty(FGLTFJsonTextureInfo& Texture, float& Factor, const float& DefaultFactorValue, const FMaterialPropertyEx& Property, CallbackType Callback /*ModifyPixel(Pixel&,ChannelValue)*/, const FString& TextureName)
{
	if (Builder.ExportOptions->BakeMaterialInputs == EGLTFMaterialBakeMode::Disabled)
	{
		Builder.LogWarning(FString::Printf(
			TEXT("%s for material %s needs to bake, but material baking is disabled by export options"),
			TextureName.IsEmpty() ? *Property.ToString() : *TextureName,
			*Material->GetName()));
		return false;
	}

	FGLTFPropertyBakeOutput PropertyBakeOutput = BakeMaterialProperty(Property, Texture.TexCoord, Texture.Transform);

	if (PropertyBakeOutput.bIsConstant)
	{
		Factor = PropertyBakeOutput.ConstantValue.R;
	}
	else
	{
		if (Builder.ExportOptions->TextureImageFormat == EGLTFTextureImageFormat::None)
		{
			Texture.Index = nullptr;
			return true;
		}

		//Move Value to Alpha per documentation:
		//
		{
			const FExpressionInput* MaterialInput = FGLTFMaterialUtilities::GetInputForProperty(Material, Property);
			if (MaterialInput == nullptr)
			{
				// TODO: report error
				return false;
			}
			const FLinearColor Mask = FGLTFMaterialUtilities::GetMask(*MaterialInput);

			enum ERGBMask
			{
				RED = 0,
				GREEN = 1,
				BLUE = 2,
				ALPHA = 3,
				RGB = 4
			};

			ERGBMask IndexOfMask = ERGBMask(0);

			const TArray<FLinearColor> AllChannelMasks = { RedMask, GreenMask, BlueMask, AlphaMask, RgbMask };

			if (AllChannelMasks.Contains(Mask))
			{
				IndexOfMask = ERGBMask(AllChannelMasks.IndexOfByKey(Mask));
			}

			auto SwapValues = [](FColor& Color) {};

			switch (IndexOfMask)
			{
			case RED:
			case RGB:
				for (FColor& Pixel : *PropertyBakeOutput.Pixels)
				{
					Callback(Pixel, Pixel.R);
				}
				break;

			case GREEN:
				for (FColor& Pixel : *PropertyBakeOutput.Pixels)
				{
					Callback(Pixel, Pixel.G);
				}
				break;

			case BLUE:
				for (FColor& Pixel : *PropertyBakeOutput.Pixels)
				{
					Callback(Pixel, Pixel.B);
				}
				break;

			case ALPHA:
				for (FColor& Pixel : *PropertyBakeOutput.Pixels)
				{
					Callback(Pixel, Pixel.A);
				}
				break;

			default:
				break;
			}
		}

		const EGLTFMaterialPropertyGroup PropertyGroup = GetPropertyGroup(Property);
		const TextureAddress TextureAddress = Builder.GetBakeTilingForMaterialProperty(Material, PropertyGroup);
		const TextureFilter TextureFilter = Builder.GetBakeFilterForMaterialProperty(Material, PropertyGroup);

		FGLTFJsonTexture* JsonTexture = FGLTFMaterialUtilities::AddTexture(
			Builder,
			PropertyBakeOutput.Pixels,
			PropertyBakeOutput.Size,
			false,
			FGLTFMaterialUtilities::IsNormalMap(Property),
			GetBakedTextureName(TextureName.IsEmpty() ? Property.ToString() : TextureName),
			TextureAddress,
			TextureFilter);

		Texture.Index = JsonTexture;

		if (!PropertyBakeOutput.bIsConstant)
		{
			Factor = DefaultFactorValue;
		}
	}

	return true;
}

FGLTFPropertyBakeOutput FGLTFDelayedMaterialTask::BakeMaterialProperty(const FMaterialPropertyEx& Property, int32& OutTexCoord, FGLTFJsonTextureTransform& OutTransform)
{
	const FIntPoint TextureSize = GetBakeSize(Property);
	return BakeMaterialProperty(Property, OutTexCoord, OutTransform, TextureSize, true);
}

FGLTFPropertyBakeOutput FGLTFDelayedMaterialTask::BakeMaterialProperty(const FMaterialPropertyEx& Property, int32& OutTexCoord, FGLTFJsonTextureTransform& OutTransform, const FIntPoint& TextureSize, bool bFillAlpha)
{
	const FBox2f DefaultTexCoordBounds = { { 0.0f, 0.0f }, { 1.0f, 1.0f } };
	FBox2f TexCoordBounds;

	if (MeshData == nullptr)
	{
		FGLTFIndexArray TexCoords;
		FGLTFMaterialUtilities::GetAllTextureCoordinateIndices(Material, Property, TexCoords);

		if (TexCoords.Num() > 0)
		{
			OutTexCoord = TexCoords[0];

			if (TexCoords.Num() > 1)
			{
				Builder.LogWarning(FString::Printf(
					TEXT("%s for material %s uses multiple texture coordinates (%s), baked texture will be sampled using only the first (%d)"),
					*Property.ToString(),
					*Material->GetName(),
					*FString::JoinBy(TexCoords, TEXT(", "), FString::FromInt),
					OutTexCoord));
			}
		}
		else
		{
			OutTexCoord = 0;
		}

		TexCoordBounds = DefaultTexCoordBounds;
	}
	else
	{
		MeshDataBakedProperties.Add(Property);
		OutTexCoord = MeshData->BakeUsingTexCoord;
		TexCoordBounds = UVBoundsCalculator.GetOrAdd(&MeshData->Description, SectionIndices, OutTexCoord);

		if (Builder.ExportOptions->bExportTextureTransforms)
		{
			const FVector2f Scale = { 1.0f / (TexCoordBounds.Max.X - TexCoordBounds.Min.X), 1.0f / (TexCoordBounds.Max.Y - TexCoordBounds.Min.Y) };
			const FVector2f Offset = -TexCoordBounds.Min * Scale;
			OutTransform.Offset = FGLTFCoreUtilities::ConvertUV(Offset);
			OutTransform.Scale = FGLTFCoreUtilities::ConvertUV(Scale);
		}
		else if (!TexCoordBounds.Equals(DefaultTexCoordBounds))
		{
			Builder.LogSuggestion(FString::Printf(
				TEXT("Export and baking of %s for material %s could be improved, by enabling texture transform in export options"),
				*Property.ToString(),
				*Material->GetName()));
			TexCoordBounds = DefaultTexCoordBounds;
		}
	}

	// TODO: add support for calculating the ideal resolution to use for baking based on connected (texture) nodes

	return FGLTFMaterialUtilities::BakeMaterialProperty(
		TextureSize,
		Property,
		Material,
		TexCoordBounds,
		OutTexCoord,
		MeshData,
		SectionIndices,
		bFillAlpha,
		Builder.ExportOptions->bAdjustNormalmaps);
}

bool FGLTFDelayedMaterialTask::StoreBakedPropertyTexture(const FMaterialPropertyEx& Property, FGLTFJsonTextureInfo& OutTexInfo, FGLTFPropertyBakeOutput& PropertyBakeOutput, const FString& PropertyName) const
{
	const EGLTFMaterialPropertyGroup PropertyGroup = GetPropertyGroup(Property);
	const TextureAddress TextureAddress = Builder.GetBakeTilingForMaterialProperty(Material, PropertyGroup);
	const TextureFilter TextureFilter = Builder.GetBakeFilterForMaterialProperty(Material, PropertyGroup);

	FGLTFJsonTexture* Texture = FGLTFMaterialUtilities::AddTexture(
		Builder,
		PropertyBakeOutput.Pixels,
		PropertyBakeOutput.Size,
		true, // NOTE: we can ignore alpha in everything but TryGetBaseColorAndOpacity
		FGLTFMaterialUtilities::IsNormalMap(Property),
		GetBakedTextureName(PropertyName),
		TextureAddress,
		TextureFilter);

	OutTexInfo.Index = Texture;
	return true;
}

FIntPoint FGLTFDelayedMaterialTask::GetBakeSize(const FMaterialPropertyEx& Property) const
{
	FGLTFMaterialBakeSize BakeSize = Builder.GetBakeSizeForMaterialProperty(Material, GetPropertyGroup(Property));

	FIntPoint MaxSize;
	if (BakeSize.bAutoDetect && FGLTFMaterialUtilities::TryGetMaxTextureSize(Material, Property, MaxSize))
	{
		return MaxSize;
	}

	return { BakeSize.X, BakeSize.Y };
}

FIntPoint FGLTFDelayedMaterialTask::GetBakeSize(const FMaterialPropertyEx& PropertyA, const FMaterialPropertyEx& PropertyB) const
{
	FGLTFMaterialBakeSize BakeSize = Builder.GetBakeSizeForMaterialProperty(Material, GetPropertyGroup(PropertyA));

	FIntPoint MaxSize;
	if (BakeSize.bAutoDetect && FGLTFMaterialUtilities::TryGetMaxTextureSize(Material, PropertyA, PropertyB, MaxSize))
	{
		return MaxSize;
	}

	return { BakeSize.X, BakeSize.Y };
}

EGLTFMaterialPropertyGroup FGLTFDelayedMaterialTask::GetPropertyGroup(const FMaterialPropertyEx& Property)
{
	switch (Property.Type)
	{
		case MP_BaseColor:
		case MP_Opacity:
		case MP_OpacityMask:
			return EGLTFMaterialPropertyGroup::BaseColorOpacity;
		case MP_Metallic:
		case MP_Roughness:
			return EGLTFMaterialPropertyGroup::MetallicRoughness;
		case MP_EmissiveColor:
			return EGLTFMaterialPropertyGroup::EmissiveColor;
		case MP_Normal:
			return EGLTFMaterialPropertyGroup::Normal;
		case MP_AmbientOcclusion:
			return EGLTFMaterialPropertyGroup::AmbientOcclusion;
		case MP_CustomData0:
		case MP_CustomData1:
			return EGLTFMaterialPropertyGroup::ClearCoatRoughness;
		case MP_CustomOutput:
			if (Property == FMaterialPropertyEx::ClearCoatBottomNormal)
			{
				return EGLTFMaterialPropertyGroup::ClearCoatBottomNormal;
			}
		default:
			return EGLTFMaterialPropertyGroup::None;
	}
}

template <typename CallbackType>
void FGLTFDelayedMaterialTask::CombinePixels(const TArray<FColor>& FirstPixels, const TArray<FColor>& SecondPixels, TArray<FColor>& OutPixels, CallbackType Callback)
{
	const int32 Count = FMath::Max(FirstPixels.Num(), SecondPixels.Num());
	OutPixels.AddUninitialized(Count);

	if (FirstPixels.Num() == 1)
	{
		const FColor& FirstPixel = FirstPixels[0];
		for (int32 Index = 0; Index < Count; ++Index)
		{
			const FColor& SecondPixel = SecondPixels[Index];
			OutPixels[Index] = Callback(FirstPixel, SecondPixel);
		}
	}
	else if (SecondPixels.Num() == 1)
	{
		const FColor& SecondPixel = SecondPixels[0];
		for (int32 Index = 0; Index < Count; ++Index)
		{
			const FColor& FirstPixel = FirstPixels[Index];
			OutPixels[Index] = Callback(FirstPixel, SecondPixel);
		}
	}
	else
	{
		check(FirstPixels.Num() == SecondPixels.Num());
		for (int32 Index = 0; Index < Count; ++Index)
		{
			const FColor& FirstPixel = FirstPixels[Index];
			const FColor& SecondPixel = SecondPixels[Index];
			OutPixels[Index] = Callback(FirstPixel, SecondPixel);
		}
	}
}

#endif

bool FGLTFDelayedMaterialTask::HandleGLTFImported(const EMaterialShadingModel& UEMaterialShadingModel)
{
	if (!Builder.ExportOptions->bExportUnlitMaterials)
	{
		return false;
	}

	FGLTFImportMaterialMatchMakingHelper GLTFImportedProcesser(Builder, Material, *JsonMaterial);

	if (!GLTFImportedProcesser.bIsGLTFImportedMaterial)
	{
		return false;
	}

	ApplyExportOptionsToShadingModel(JsonMaterial->ShadingModel, UEMaterialShadingModel);

	GLTFImportedProcesser.Process();

	return true;
}

