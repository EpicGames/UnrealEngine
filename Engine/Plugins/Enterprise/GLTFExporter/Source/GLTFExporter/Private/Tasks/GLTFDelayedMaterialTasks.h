// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/GLTFDelayedTask.h"
#include "Builders/GLTFConvertBuilder.h"
#include "Converters/GLTFUVOverlapChecker.h"
#include "Converters/GLTFUVBoundsCalculator.h"
#include "Converters/GLTFMeshData.h"
#include "Materials/GLTFProxyMaterialParameterInfo.h"
#include "Materials/Material.h"
#if WITH_EDITOR
#include "MaterialPropertyEx.h"
#endif

struct FGLTFPropertyBakeOutput;

class FGLTFDelayedMaterialTask : public FGLTFDelayedTask
{
public:

	FGLTFDelayedMaterialTask(FGLTFConvertBuilder& Builder, FGLTFUVOverlapChecker& UVOverlapChecker, FGLTFUVBoundsCalculator& UVBoundsCalculator, const UMaterialInterface* Material, const FGLTFMeshData* MeshData, FGLTFIndexArray& SectionIndices, FGLTFJsonMaterial* JsonMaterial)
		: FGLTFDelayedTask(EGLTFTaskPriority::Material)
		, Builder(Builder)
		, UVOverlapChecker(UVOverlapChecker)
		, UVBoundsCalculator(UVBoundsCalculator)
		, Material(Material)
		, MeshData(MeshData)
		, SectionIndices(SectionIndices)
		, JsonMaterial(JsonMaterial)
	{
	}

	virtual FString GetName() override
	{
		return Material->GetName();
	}

	virtual void Process() override;

private:

	FGLTFConvertBuilder& Builder;
	FGLTFUVOverlapChecker& UVOverlapChecker;
	FGLTFUVBoundsCalculator& UVBoundsCalculator;
	const UMaterialInterface* Material;
	const FGLTFMeshData* MeshData;
	const FGLTFIndexArray SectionIndices; //We copy because the received value can be temporary value.
	FGLTFJsonMaterial* JsonMaterial;

	FString GetMaterialName() const;
	FString GetBakedTextureName(const FString& PropertyName) const;

	void GetProxyParameters(FGLTFJsonMaterial& OutMaterial) const;
	void GetProxyParameter(const TGLTFProxyMaterialParameterInfo<float>& ParameterInfo, float& OutValue) const;
	void GetProxyParameter(const TGLTFProxyMaterialParameterInfo<FLinearColor>& ParameterInfo, FGLTFJsonColor3& OutValue) const;
	void GetProxyParameter(const TGLTFProxyMaterialParameterInfo<FLinearColor>& ParameterInfo, FGLTFJsonColor4& OutValue) const;
	void GetProxyParameter(const FGLTFProxyMaterialTextureParameterInfo& ParameterInfo, FGLTFJsonTextureInfo& OutValue) const;

	template <typename ParameterType>
	bool HasProxyParameter(const TGLTFProxyMaterialParameterInfo<ParameterType>& ParameterInfo) const
	{
		ParameterType Value;
		return ParameterInfo.Get(Material, Value, true);
	}

	void ConvertShadingModel(EMaterialShadingModel& UEShadingModel, EGLTFJsonShadingModel& OutGLTFShadingModel) const;
	void ApplyExportOptionsToShadingModel(EGLTFJsonShadingModel& ShadingModel, const EMaterialShadingModel& UEMaterialShadingModel) const;

	void ConvertAlphaMode(EGLTFJsonAlphaMode& OutAlphaMode) const;

#if WITH_EDITOR
	TSet<FMaterialPropertyEx> MeshDataBakedProperties;

	bool TryGetBaseColorAndOpacity(FGLTFJsonPBRMetallicRoughness& OutPBRParams, const FMaterialPropertyEx& BaseColorProperty, const FMaterialPropertyEx& OpacityProperty);
	bool TryGetMetallicAndRoughness(FGLTFJsonPBRMetallicRoughness& OutPBRParams, const FMaterialPropertyEx& MetallicProperty, const FMaterialPropertyEx& RoughnessProperty);
	bool TryGetClearCoatRoughness(FGLTFJsonClearCoatExtension& OutExtParams, const FMaterialPropertyEx& IntensityProperty, const FMaterialPropertyEx& RoughnessProperty);
	bool TryGetEmissive(FGLTFJsonMaterial& OutMaterial, const FMaterialPropertyEx& EmissiveProperty);
	bool TryGetSpecular(FGLTFJsonMaterial& OutMaterial, const FMaterialPropertyEx& SpecularProperty);
	bool TryGetRefraction(FGLTFJsonMaterial& OutMaterial, const FMaterialPropertyEx& RefractionProperty);
	bool TryGetFuzzColorAndCloth(FGLTFJsonMaterial& OutMaterial, const FMaterialPropertyEx& FuzzColorProperty, const FMaterialPropertyEx& ClothProperty);
	bool TryGetTransmissionBaseColorAndOpacity(FGLTFJsonMaterial& OutMaterial, const FMaterialPropertyEx& BaseColorProperty, const FMaterialPropertyEx& OpacityProperty);

	bool IsPropertyNonDefault(const FMaterialPropertyEx& Property) const;
	bool TryGetConstantColor(FGLTFJsonColor3& OutValue, const FMaterialPropertyEx& Property) const;
	bool TryGetConstantColor(FGLTFJsonColor4& OutValue, const FMaterialPropertyEx& Property) const;
	bool TryGetConstantColor(FLinearColor& OutValue, const FMaterialPropertyEx& Property) const;
	bool TryGetConstantScalar(float& OutValue, const FMaterialPropertyEx& Property) const;

	bool TryGetSourceTexture(FGLTFJsonTextureInfo& OutTexInfo, const FMaterialPropertyEx& Property, const TArray<FLinearColor>& AllowedMasks = {}) const;
	bool TryGetSourceTexture(const UTexture2D*& OutTexture, int32& OutTexCoord, FGLTFJsonTextureTransform& OutTransform, const FMaterialPropertyEx& Property, const TArray<FLinearColor>& AllowedMasks = {}) const;

	bool TryGetBakedMaterialProperty(FGLTFJsonTextureInfo& OutTexInfo, FGLTFJsonColor3& OutConstant, const FMaterialPropertyEx& Property, const FString& PropertyName);
	bool TryGetBakedMaterialProperty(FGLTFJsonTextureInfo& OutTexInfo, FGLTFJsonColor4& OutConstant, const FMaterialPropertyEx& Property, const FString& PropertyName);
	bool TryGetBakedMaterialProperty(FGLTFJsonTextureInfo& OutTexInfo, float& OutConstant, const FMaterialPropertyEx& Property, const FString& PropertyName);
	bool TryGetBakedMaterialProperty(FGLTFJsonTextureInfo& OutTexInfo, const FMaterialPropertyEx& Property, const FString& PropertyName);

	template <typename CallbackType>
	bool TryGetBakedMaterialProperty(FGLTFJsonTextureInfo& Texture, float& Factor, const float& DefaultFactorValue, const FMaterialPropertyEx& Property, CallbackType Callback /*ModifyPixel(Pixel&,ChannelValue)*/, const FString& TextureName = TEXT(""));

	FGLTFPropertyBakeOutput BakeMaterialProperty(const FMaterialPropertyEx& Property, int32& OutTexCoord, FGLTFJsonTextureTransform& OutTransform);
	FGLTFPropertyBakeOutput BakeMaterialProperty(const FMaterialPropertyEx& Property, int32& OutTexCoord, FGLTFJsonTextureTransform& OutTransform, const FIntPoint& TextureSize, bool bFillAlpha);

	bool StoreBakedPropertyTexture(const FMaterialPropertyEx& Property, FGLTFJsonTextureInfo& OutTexInfo, FGLTFPropertyBakeOutput& PropertyBakeOutput, const FString& PropertyName) const;

	FIntPoint GetBakeSize(const FMaterialPropertyEx& Property) const;
	FIntPoint GetBakeSize(const FMaterialPropertyEx& PropertyA,const FMaterialPropertyEx& PropertyB) const;

	static EGLTFMaterialPropertyGroup GetPropertyGroup(const FMaterialPropertyEx& Property);

	template <typename CallbackType>
	static void CombinePixels(const TArray<FColor>& FirstPixels, const TArray<FColor>& SecondPixels, TArray<FColor>& OutPixels, CallbackType Callback);
#endif

	bool HandleGLTFImported(const EMaterialShadingModel& UEMaterialShadingModel);
};
