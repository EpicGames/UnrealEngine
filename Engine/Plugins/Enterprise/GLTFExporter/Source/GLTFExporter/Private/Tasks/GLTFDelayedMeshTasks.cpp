// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/GLTFDelayedMeshTasks.h"
#include "Algo/AllOf.h"
#include "Converters/GLTFMeshUtilities.h"
#include "Converters/GLTFBufferAdapter.h"
#include "Builders/GLTFConvertBuilder.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshResources.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "SkeletalMeshAttributes.h"

#include "Converters/GLTFMaterialUtilities.h"
#include "Rendering/PositionVertexBuffer.h"
#include "Components/SplineMeshComponent.h"
#include "LandscapeProxy.h"
#include "LandscapeComponent.h"
#include "LandscapeLayerInfoObject.h"
#include "Converters/GLTFMeshAttributesArray.h"
#include "Utilities/GLTFLandscapeComponentDataInterface.h"

#if WITH_EDITORONLY_DATA
#include "Utilities/MeshParsingUtilities.h"
#endif

namespace
{
	template <typename VectorType>
	void CheckTangentVectors(const void* SourceData, uint32 VertexCount, bool& bOutZeroNormals, bool& bOutZeroTangents)
	{
		bool bZeroNormals = false;
		bool bZeroTangents = false;

		typedef TStaticMeshVertexTangentDatum<VectorType> VertexTangentType;
		const VertexTangentType* VertexTangents = static_cast<const VertexTangentType*>(SourceData);

		for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
		{
			const VertexTangentType& VertexTangent = VertexTangents[VertexIndex];
			bZeroNormals |= VertexTangent.TangentZ.ToFVector().IsNearlyZero();
			bZeroTangents |= VertexTangent.TangentX.ToFVector().IsNearlyZero();
		}

		bOutZeroNormals = bZeroNormals;
		bOutZeroTangents = bZeroTangents;
	}

	void ValidateVertexBuffer(FGLTFConvertBuilder& Builder, const FStaticMeshVertexBuffer* VertexBuffer, const TCHAR* MeshName)
	{
		if (VertexBuffer == nullptr)
		{
			return;
		}

		const TUniquePtr<IGLTFBufferAdapter> SourceBuffer = IGLTFBufferAdapter::GetTangents(VertexBuffer);
		const uint8* SourceData = SourceBuffer->GetData();

		if (SourceData == nullptr)
		{
			return;
		}

		const uint32 VertexCount = VertexBuffer->GetNumVertices();
		bool bZeroNormals;
		bool bZeroTangents;

		if (VertexBuffer->GetUseHighPrecisionTangentBasis())
		{
			CheckTangentVectors<FPackedRGBA16N>(SourceData, VertexCount, bZeroNormals, bZeroTangents);
		}
		else
		{
			CheckTangentVectors<FPackedNormal>(SourceData, VertexCount, bZeroNormals, bZeroTangents);
		}

		if (bZeroNormals)
		{
			Builder.LogSuggestion(FString::Printf(
				TEXT("Mesh %s has some nearly zero-length normals which may not be supported in some glTF applications. Consider checking 'Recompute Normals' in the asset settings"),
				MeshName));
		}

		if (bZeroTangents)
		{
			Builder.LogSuggestion(FString::Printf(
				TEXT("Mesh %s has some nearly zero-length tangents which may not be supported in some glTF applications. Consider checking 'Recompute Tangents' in the asset settings"),
				MeshName));
		}
	}

	bool HasVertexColors(const FColorVertexBuffer* VertexBuffer)
	{
		if (VertexBuffer == nullptr)
		{
			return false;
		}

		const TUniquePtr<IGLTFBufferAdapter> SourceBuffer = IGLTFBufferAdapter::GetColors(VertexBuffer);
		const uint8* SourceData = SourceBuffer->GetData();

		if (SourceData == nullptr)
		{
			return false;
		}

		const uint32 VertexCount = VertexBuffer->GetNumVertices();
		const uint32 Stride = VertexBuffer->GetStride();

		for (uint32 VertexIndex = 0; VertexIndex < VertexCount; VertexIndex++)
		{
			const FColor& Color = *reinterpret_cast<const FColor*>(SourceData + Stride * VertexIndex);
			if (Color != FColor::White)
			{
				return true;
			}
		}

		return false;
	}

	template <class T>
	bool DoesBufferHasZeroVector(TArray<T> Buffer, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		for (const T& Value : Buffer)
		{
			if (FMath::Abs(Value.X) <= Tolerance
				&& FMath::Abs(Value.Y) <= Tolerance
				&& FMath::Abs(Value.Z) <= Tolerance)
			{
				return true;
			}
		}
		return false;
	}
}

FString FGLTFDelayedStaticAndSplineMeshTask::GetName()
{
	return StaticMeshComponent != nullptr ? FGLTFNameUtilities::GetName(StaticMeshComponent) : (SplineMeshComponent != nullptr ? FGLTFNameUtilities::GetName(SplineMeshComponent) : StaticMesh->GetName());
}

void FGLTFDelayedStaticAndSplineMeshTask::Process()
{
	FGLTFMeshUtilities::FullyLoad(StaticMesh);

	const UMeshComponent* MeshComponent = StaticMeshComponent != nullptr ? StaticMeshComponent : (SplineMeshComponent != nullptr ? SplineMeshComponent : nullptr);

	JsonMesh->Name = MeshComponent != nullptr ? FGLTFNameUtilities::GetName(MeshComponent) : StaticMesh->GetName();

	const TArray<FStaticMaterial>& MaterialSlots = FGLTFMeshUtilities::GetMaterials(StaticMesh);
	
	const FGLTFMeshData* MeshData = Builder.ExportOptions->BakeMaterialInputs == EGLTFMaterialBakeMode::UseMeshData ?
		Builder.AddUniqueMeshData(StaticMesh, StaticMeshComponent, LODIndex) : nullptr;

#if WITH_EDITOR
	if (MeshData != nullptr)
	{
		if (MeshData->Description.IsEmpty())
		{
			// TODO: report warning in case the mesh actually has data, which means we failed to extract a mesh description.
			MeshData = nullptr;
		}
		else if (MeshData->BakeUsingTexCoord < 0)
		{
			// TODO: report warning (about missing texture coordinate for baking with mesh data).
			MeshData = nullptr;
		}
	}
#endif

#if WITH_EDITORONLY_DATA
	if (Builder.ExportOptions->bExportSourceModel)
	{
		ProcessMeshDescription(MaterialSlots, MeshData);
	}
	else
#endif
	{
		ProcessRenderData(MaterialSlots, MeshData);
	}
}

#if WITH_EDITORONLY_DATA

void FGLTFDelayedStaticAndSplineMeshTask::ProcessMeshDescription(const TArray<FStaticMaterial>& MaterialSlots, const FGLTFMeshData* MeshData)
{
	FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(LODIndex);

	using namespace UE::MeshParser;

	FMeshDescriptionParser MeshDescriptionParser(MeshDescription, MaterialSlots);

	constexpr bool bExportVertexSkinWeights_False = false;
	constexpr bool bExportMorphTargets_False = false;
	constexpr int32 SkeletonInfluenceCountPerGroup_4 = 4;
	FExportConfigs ExportConfigs(bExportVertexSkinWeights_False, Builder.ExportOptions->bExportVertexColors, SplineMeshComponent, SkeletonInfluenceCountPerGroup_4, bExportMorphTargets_False);

	TArray<FMeshPrimitiveDescription> MeshPrimitiveDescriptions;
	TArray<FString> TargetNames; //Dummy variable
	MeshDescriptionParser.Parse(MeshPrimitiveDescriptions, TargetNames, ExportConfigs);
	
	if (MeshPrimitiveDescriptions.Num() != JsonMesh->Primitives.Num())
	{
		return;
	}

	int32 PrimitiveIndex = 0;
	for (FMeshPrimitiveDescription& MeshPrimitiveDescription : MeshPrimitiveDescriptions)
	{
		if (MeshPrimitiveDescription.IsEmpty())
		{
			PrimitiveIndex++;
			continue;
		}

		FGLTFJsonPrimitive& JsonPrimitive = JsonMesh->Primitives[PrimitiveIndex];

		//Set glTF Primitive:
		JsonPrimitive.Indices = Builder.AddUniqueIndexAccessor(MeshPrimitiveDescription.Indices, StaticMesh->GetName() + (MeshDescriptionParser.MeshDetails.NumberOfPrimitives > 1 ? (TEXT("_") + FString::FromInt(PrimitiveIndex)) : TEXT("")));
		JsonPrimitive.Attributes.Position = Builder.AddUniquePositionAccessor(MeshPrimitiveDescription.Positions);
		if (!MeshPrimitiveDescription.VertexColors.IsEmpty()) JsonPrimitive.Attributes.Color0 = Builder.AddUniqueColorAccessor(MeshPrimitiveDescription.VertexColors);
		JsonPrimitive.Attributes.Normal = Builder.AddUniqueNormalAccessor(MeshPrimitiveDescription.Normals);
		JsonPrimitive.Attributes.Tangent = Builder.AddUniqueTangentAccessor(MeshPrimitiveDescription.Tangents);
		JsonPrimitive.Attributes.TexCoords.AddUninitialized(MeshDescriptionParser.MeshDetails.UVCount);
		for (size_t UVIndex = 0; UVIndex < MeshDescriptionParser.MeshDetails.UVCount; UVIndex++)
		{
			JsonPrimitive.Attributes.TexCoords[UVIndex] = Builder.AddUniqueUVAccessor(FGLTFUVArray(MeshPrimitiveDescription.UVs[UVIndex]));
		}
		//
		const UMaterialInterface* Material = Materials.IsValidIndex(MeshPrimitiveDescription.MaterialIndex) ? Materials[MeshPrimitiveDescription.MaterialIndex] : MaterialSlots[MeshPrimitiveDescription.MaterialIndex].MaterialInterface;
		JsonPrimitive.Material = Builder.AddUniqueMaterial(Material, MeshData, { MeshPrimitiveDescription.MaterialIndex });

		//Validations:
		if (JsonPrimitive.Attributes.Position == nullptr)
		{
			Builder.LogError(
				FString::Printf(TEXT("Failed to export vertex positions related to material slot %d (%s) in static mesh %s"),
					0,
					*JsonPrimitive.Material->Name,
					*JsonMesh->Name
				));
		}

		PrimitiveIndex++;
	}
}
#endif

void FGLTFDelayedStaticAndSplineMeshTask::ProcessRenderData(const TArray<FStaticMaterial>& MaterialSlots, const FGLTFMeshData* MeshData)
{
	const FStaticMeshLODResources& RenderData = FGLTFMeshUtilities::GetRenderData(StaticMesh, LODIndex);

	const FPositionVertexBuffer& PositionBuffer = RenderData.VertexBuffers.PositionVertexBuffer;
	const FStaticMeshVertexBuffer& VertexBuffer = RenderData.VertexBuffers.StaticMeshVertexBuffer;
	const FColorVertexBuffer* ColorBuffer = &RenderData.VertexBuffers.ColorVertexBuffer; // TODO: add support for overriding color buffer by component

	if (Builder.ExportOptions->bExportVertexColors && HasVertexColors(ColorBuffer))
	{
		Builder.LogSuggestion(FString::Printf(
			TEXT("Vertex colors in mesh %s will act as a multiplier for base color in glTF, regardless of material, which may produce undesirable results"),
			*StaticMesh->GetName()));
	}
	else
	{
		ColorBuffer = nullptr;
	}

	if (StaticMeshComponent != nullptr && StaticMeshComponent->LODData.IsValidIndex(LODIndex))
	{
		const FStaticMeshComponentLODInfo& LODInfo = StaticMeshComponent->LODData[LODIndex];
		ColorBuffer = LODInfo.OverrideVertexColors != nullptr ? LODInfo.OverrideVertexColors : ColorBuffer;
	}
	else if (SplineMeshComponent != nullptr && SplineMeshComponent->LODData.IsValidIndex(LODIndex))
	{
		const FStaticMeshComponentLODInfo& LODInfo = SplineMeshComponent->LODData[LODIndex];
		ColorBuffer = LODInfo.OverrideVertexColors != nullptr ? LODInfo.OverrideVertexColors : ColorBuffer;
	}

	ValidateVertexBuffer(Builder, &VertexBuffer, *StaticMesh->GetName());

	for (int32 MaterialIndex = 0; MaterialIndex < MaterialSlots.Num(); ++MaterialIndex)
	{
		const FGLTFIndexArray SectionIndices = FGLTFMeshUtilities::GetSectionIndices(RenderData, MaterialIndex);
		const FGLTFMeshSection* ConvertedSection = MeshSectionConverter.GetOrAdd(StaticMesh, LODIndex, SectionIndices);

		FGLTFJsonPrimitive& JsonPrimitive = JsonMesh->Primitives[MaterialIndex];
		JsonPrimitive.Indices = Builder.AddUniqueIndexAccessor(ConvertedSection);

		if (!JsonPrimitive.Indices || JsonPrimitive.Indices->Count == 0)
		{
			//Do not export empty primitives.
			continue;
		}

		if (SplineMeshComponent)
		{
			//fix for Splines:
			FPositionVertexBuffer* TransformedPositionBuffer = new FPositionVertexBuffer();

			TransformedPositionBuffer->Init(PositionBuffer.GetNumVertices(), true);

			const uint32 VertexCount = PositionBuffer.GetNumVertices();
			const uint32 Stride = PositionBuffer.GetStride();

			const TUniquePtr<IGLTFBufferAdapter> SourceBuffer = IGLTFBufferAdapter::GetPositions(&PositionBuffer);
			const uint8* SourceData = SourceBuffer->GetData();

			for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex, SourceData+=Stride)
			{
				FVector3f& VertexPosition = TransformedPositionBuffer->VertexPosition(VertexIndex);
				VertexPosition = *reinterpret_cast<const FVector3f*>(SourceData);

				const FTransform3f SliceTransform = FTransform3f(SplineMeshComponent->CalcSliceTransform(USplineMeshComponent::GetAxisValueRef(VertexPosition, SplineMeshComponent->ForwardAxis)));
				USplineMeshComponent::GetAxisValueRef(VertexPosition, SplineMeshComponent->ForwardAxis) = 0;
				VertexPosition = SliceTransform.TransformPosition(VertexPosition);
			}

			JsonPrimitive.Attributes.Position = Builder.AddUniquePositionAccessor(ConvertedSection, TransformedPositionBuffer);
		}
		else
		{
			JsonPrimitive.Attributes.Position = Builder.AddUniquePositionAccessor(ConvertedSection, &PositionBuffer);
		}

		if (JsonPrimitive.Attributes.Position == nullptr)
		{
			Builder.LogError(
				FString::Printf(TEXT("Failed to export vertex positions related to material slot %d (%s) in static mesh %s"),
					MaterialIndex,
					*MaterialSlots[MaterialIndex].MaterialSlotName.ToString(),
					*ConvertedSection->ToString()
				));
		}

		if (ColorBuffer != nullptr)
		{
			JsonPrimitive.Attributes.Color0 = Builder.AddUniqueColorAccessor(ConvertedSection, ColorBuffer);
		}

		// TODO: report warning if both Mesh Quantization (export options) and Use High Precision Tangent Basis (vertex buffer) are disabled
		JsonPrimitive.Attributes.Normal = Builder.AddUniqueNormalAccessor(ConvertedSection, &VertexBuffer);
		JsonPrimitive.Attributes.Tangent = Builder.AddUniqueTangentAccessor(ConvertedSection, &VertexBuffer);

		const uint32 UVCount = VertexBuffer.GetNumTexCoords();
		// TODO: report warning or option to limit UV channels since most viewers don't support more than 2?
		JsonPrimitive.Attributes.TexCoords.AddUninitialized(UVCount);

		for (uint32 UVIndex = 0; UVIndex < UVCount; ++UVIndex)
		{
			JsonPrimitive.Attributes.TexCoords[UVIndex] = Builder.AddUniqueUVAccessor(ConvertedSection, &VertexBuffer, UVIndex);
		}

		const UMaterialInterface* Material = Materials[MaterialIndex];
		JsonPrimitive.Material = Builder.AddUniqueMaterial(Material, MeshData, SectionIndices);
	}
}


FString FGLTFDelayedSkeletalMeshTask::GetName()
{
	return SkeletalMeshComponent != nullptr ? FGLTFNameUtilities::GetName(SkeletalMeshComponent) : SkeletalMesh->GetName();
}

void FGLTFDelayedSkeletalMeshTask::Process()
{
	FGLTFMeshUtilities::FullyLoad(SkeletalMesh);
	JsonMesh->Name = SkeletalMeshComponent != nullptr ? FGLTFNameUtilities::GetName(SkeletalMeshComponent) : SkeletalMesh->GetName();

	const FGLTFMeshData* MeshData = Builder.ExportOptions->BakeMaterialInputs == EGLTFMaterialBakeMode::UseMeshData ?
		Builder.AddUniqueMeshData(SkeletalMesh, SkeletalMeshComponent, LODIndex) : nullptr;

#if WITH_EDITOR
	if (MeshData != nullptr)
	{
		if (MeshData->Description.IsEmpty())
		{
			// TODO: report warning in case the mesh actually has data, which means we failed to extract a mesh description.
			MeshData = nullptr;
		}
		else if (MeshData->BakeUsingTexCoord < 0)
		{
			// TODO: report warning (about missing texture coordinate for baking with mesh data).
			MeshData = nullptr;
		}
	}
#endif

	const TArray<FSkeletalMaterial>& MaterialSlots = FGLTFMeshUtilities::GetMaterials(SkeletalMesh);

#if WITH_EDITORONLY_DATA
	if (Builder.ExportOptions->bExportSourceModel)
	{
		ProcessSourceModel(MaterialSlots, MeshData);
	}
	else
#endif
	{
		ProcessRenderData(MaterialSlots, MeshData);
	}
}

#if WITH_EDITORONLY_DATA
void FGLTFDelayedSkeletalMeshTask::ProcessSourceModel(const TArray<FSkeletalMaterial>& MaterialSlots, const FGLTFMeshData* MeshData)
{
	if (SkeletalMesh->GetNumSourceModels() < LODIndex)
	{
		return;
	}
	const FSkeletalMeshSourceModel& MeshSourceModel = SkeletalMesh->GetSourceModel(LODIndex);
	const FMeshDescription* MeshDescription = MeshSourceModel.GetMeshDescription();

	using namespace UE::MeshParser;
	
	FMeshDescriptionParser MeshDescriptionParser(MeshDescription, MaterialSlots);
	
	constexpr bool bExportVertexSkinWeights_True = true;
	constexpr int32 SkeletonInfluenceCountPerGroup_4 = 4;
	FExportConfigs ExportConfigs(bExportVertexSkinWeights_True, Builder.ExportOptions->bExportVertexColors, nullptr, SkeletonInfluenceCountPerGroup_4, Builder.ExportOptions->bExportMorphTargets);
	

	TArray<FMeshPrimitiveDescription> MeshPrimitiveDescriptions;
	MeshDescriptionParser.Parse(MeshPrimitiveDescriptions, JsonMesh->TargetNames, ExportConfigs);

	if (MeshPrimitiveDescriptions.Num() != JsonMesh->Primitives.Num())
	{
		return;
	}

	int32 PrimitiveIndex = 0;
	for (FMeshPrimitiveDescription& MeshPrimitiveDescription : MeshPrimitiveDescriptions)
	{
		if (MeshPrimitiveDescription.IsEmpty())
		{
			PrimitiveIndex++;
			continue;
		}

		FGLTFJsonPrimitive& JsonPrimitive = JsonMesh->Primitives[PrimitiveIndex];

		//Set  Primitive:
		JsonPrimitive.Indices = Builder.AddUniqueIndexAccessor(MeshPrimitiveDescription.Indices, SkeletalMesh->GetName() + (MeshDescriptionParser.MeshDetails.NumberOfPrimitives > 1 ? (TEXT("_") + FString::FromInt(PrimitiveIndex)) : TEXT("")));
		JsonPrimitive.Attributes.Position = Builder.AddUniquePositionAccessor(MeshPrimitiveDescription.Positions);
		if (!MeshPrimitiveDescription.VertexColors.IsEmpty()) JsonPrimitive.Attributes.Color0 = Builder.AddUniqueColorAccessor(MeshPrimitiveDescription.VertexColors);
		JsonPrimitive.Attributes.Normal = Builder.AddUniqueNormalAccessor(MeshPrimitiveDescription.Normals);
		JsonPrimitive.Attributes.Tangent = Builder.AddUniqueTangentAccessor(MeshPrimitiveDescription.Tangents);
		JsonPrimitive.Attributes.TexCoords.AddUninitialized(MeshDescriptionParser.MeshDetails.UVCount);
		for (size_t UVIndex = 0; UVIndex < MeshDescriptionParser.MeshDetails.UVCount; UVIndex++)
		{
			JsonPrimitive.Attributes.TexCoords[UVIndex] = Builder.AddUniqueUVAccessor(MeshPrimitiveDescription.UVs[UVIndex]);
		}

		for (FPrimitiveTargetDescription& PrimitiveTargetDescription : MeshPrimitiveDescription.Targets)
		{
			if (PrimitiveTargetDescription.bHasPositions || PrimitiveTargetDescription.bHasNormals)
			{
				FGLTFJsonTarget& JSONMorphTarget = JsonPrimitive.Targets.AddDefaulted_GetRef();
				if (PrimitiveTargetDescription.bHasPositions) JSONMorphTarget.Position = Builder.AddUniquePositionAccessor(PrimitiveTargetDescription.Positions);
				if (PrimitiveTargetDescription.bHasNormals) JSONMorphTarget.Normal = Builder.AddUniqueNormalAccessor(PrimitiveTargetDescription.Normals, false);
			}
		}

		if (Builder.ExportOptions->bExportVertexSkinWeights)
		{
			int32 GroupCount = MeshPrimitiveDescription.JointInfluences.Num();
			if (GroupCount == MeshPrimitiveDescription.JointWeights.Num())
			{
				JsonPrimitive.Attributes.Joints.AddUninitialized(GroupCount);
				JsonPrimitive.Attributes.Weights.AddUninitialized(GroupCount);
				for (size_t GroupCountIndex = 0; GroupCountIndex < GroupCount; GroupCountIndex++)
				{
					JsonPrimitive.Attributes.Joints[GroupCountIndex] = Builder.AddUniqueJointAccessor(MeshPrimitiveDescription.JointInfluences[GroupCountIndex]);
					JsonPrimitive.Attributes.Weights[GroupCountIndex] = Builder.AddUniqueWeightAccessor(MeshPrimitiveDescription.JointWeights[GroupCountIndex]);
				}
			}
		}

		//
		const UMaterialInterface* Material = Materials.IsValidIndex(MeshPrimitiveDescription.MaterialIndex) ? Materials[MeshPrimitiveDescription.MaterialIndex] : MaterialSlots[MeshPrimitiveDescription.MaterialIndex].MaterialInterface;
		JsonPrimitive.Material = Builder.AddUniqueMaterial(Material, MeshData, { MeshPrimitiveDescription.MaterialIndex });

		//Validations:
		if (JsonPrimitive.Attributes.Position == nullptr)
		{
			Builder.LogError(
				FString::Printf(TEXT("Failed to export vertex positions related to material slot %d (%s) in static mesh %s"),
					0,
					*JsonPrimitive.Material->Name,
					*JsonMesh->Name
				));
		}

		PrimitiveIndex++;
	}
}
#endif

void FGLTFDelayedSkeletalMeshTask::ProcessRenderData(const TArray<FSkeletalMaterial>& MaterialSlots, const FGLTFMeshData* MeshData)
{
	const FSkeletalMeshLODRenderData& RenderData = FGLTFMeshUtilities::GetRenderData(SkeletalMesh, LODIndex);
	const FPositionVertexBuffer& PositionBuffer = RenderData.StaticVertexBuffers.PositionVertexBuffer;
	const FStaticMeshVertexBuffer& VertexBuffer = RenderData.StaticVertexBuffers.StaticMeshVertexBuffer;
	const FColorVertexBuffer* ColorBuffer = &RenderData.StaticVertexBuffers.ColorVertexBuffer; // TODO: add support for overriding color buffer by component
	const FSkinWeightVertexBuffer* SkinWeightBuffer = RenderData.GetSkinWeightVertexBuffer(); // TODO: add support for overriding skin weight buffer by component
	// TODO: add support for skin weight profiles?
	// TODO: add support for morph targets

	if (Builder.ExportOptions->bExportVertexColors && HasVertexColors(ColorBuffer))
	{
		Builder.LogSuggestion(FString::Printf(
			TEXT("Vertex colors in mesh %s will act as a multiplier for base color in glTF, regardless of material, which may produce undesirable results"),
			*SkeletalMesh->GetName()));
	}
	else
	{
		ColorBuffer = nullptr;
	}

	if (SkeletalMeshComponent != nullptr && SkeletalMeshComponent->LODInfo.IsValidIndex(LODIndex))
	{
		const FSkelMeshComponentLODInfo& LODInfo = SkeletalMeshComponent->LODInfo[LODIndex];
		ColorBuffer = LODInfo.OverrideVertexColors != nullptr ? LODInfo.OverrideVertexColors : ColorBuffer;
		SkinWeightBuffer = LODInfo.OverrideSkinWeights != nullptr ? LODInfo.OverrideSkinWeights : SkinWeightBuffer;
	}

	ValidateVertexBuffer(Builder, &VertexBuffer, *SkeletalMesh->GetName());

	const int32 MaterialCount = MaterialSlots.Num();

	struct FTargetDeltas
	{
		TMap<uint32, FVector3f> Deltas;
		bool bIsValid = false;

		void Add(const uint32 VertexIndex, const FVector3f& Delta)
		{
			Deltas.Add(VertexIndex, Delta);
		}

		void GenerateIsValid()
		{
			bIsValid = Deltas.Num() && !Algo::AllOf(Deltas, [](const TPair<uint32, FVector3f>& Element) {
				return Element.Value.IsZero();
			});
		}
	};
	
	TArray<FTargetDeltas> MorphTargetsPositions;
	TArray<FTargetDeltas> MorphTargetsNormals;

	if (Builder.ExportOptions->bExportMorphTargets)
	{
		const TArray<TObjectPtr<UMorphTarget>>& MorphTargetObjectPtrs = SkeletalMesh->GetMorphTargets();
		for (const TObjectPtr<UMorphTarget>& MorphTargetObjectPtr : MorphTargetObjectPtrs)
		{
			const UMorphTarget* MorphTarget = MorphTargetObjectPtr;

			FTargetDeltas& PositionsDeltas = MorphTargetsPositions.AddDefaulted_GetRef();
			FTargetDeltas& NormalsDeltas = MorphTargetsNormals.AddDefaulted_GetRef();

			for (FMorphTargetDeltaIterator DeltaIt = MorphTarget->GetDeltaIteratorForLOD(LODIndex); !DeltaIt.AtEnd(); ++DeltaIt)
			{
				PositionsDeltas.Add(DeltaIt->SourceIdx, DeltaIt->PositionDelta);
				NormalsDeltas.Add(DeltaIt->SourceIdx, DeltaIt->TangentZDelta);
			}

			PositionsDeltas.GenerateIsValid();
			NormalsDeltas.GenerateIsValid();
		}
	}

	for (int32 MaterialIndex = 0; MaterialIndex < MaterialCount; ++MaterialIndex)
	{
		const FGLTFIndexArray SectionIndices = FGLTFMeshUtilities::GetSectionIndices(RenderData, MaterialIndex);
		const FGLTFMeshSection* ConvertedSection = MeshSectionConverter.GetOrAdd(SkeletalMesh, LODIndex, SectionIndices);

		FGLTFJsonPrimitive& JsonPrimitive = JsonMesh->Primitives[MaterialIndex];
		JsonPrimitive.Indices = Builder.AddUniqueIndexAccessor(ConvertedSection);

		JsonPrimitive.Attributes.Position = Builder.AddUniquePositionAccessor(ConvertedSection, &PositionBuffer);
		if (JsonPrimitive.Attributes.Position == nullptr)
		{
			Builder.LogError(
				FString::Printf(TEXT("Failed to export vertex positions related to material slot %d (%s) in skeletal mesh %s"),
					MaterialIndex,
					*MaterialSlots[MaterialIndex].MaterialSlotName.ToString(),
					*ConvertedSection->ToString()
				));
		}

		const bool bHighPrecision = VertexBuffer.GetUseHighPrecisionTangentBasis();

		if (MorphTargetsPositions.Num() > 0)
		{
			JsonPrimitive.Targets.Reserve(MorphTargetsPositions.Num());

			for (size_t TargetCounter = 0; TargetCounter < MorphTargetsPositions.Num(); TargetCounter++)
			{
				FTargetDeltas& PositionsDeltas = MorphTargetsPositions[TargetCounter];
				FTargetDeltas& NormalsDeltas = MorphTargetsNormals[TargetCounter];

				if (PositionsDeltas.bIsValid || NormalsDeltas.bIsValid)
				{
					FGLTFJsonTarget& JSONMorphTarget = JsonPrimitive.Targets.AddDefaulted_GetRef();
					if (PositionsDeltas.bIsValid) JSONMorphTarget.Position = Builder.AddUniquePositionDeltaAccessor(ConvertedSection, &PositionsDeltas.Deltas);
					if (NormalsDeltas.bIsValid) JSONMorphTarget.Normal = Builder.AddUniqueNormalDeltaAccessor(ConvertedSection, &NormalsDeltas.Deltas, bHighPrecision);
				}
			}
		}

		if (ColorBuffer != nullptr)
		{
			JsonPrimitive.Attributes.Color0 = Builder.AddUniqueColorAccessor(ConvertedSection, ColorBuffer);
		}

		// TODO: report warning if both Mesh Quantization (export options) and Use High Precision Tangent Basis (vertex buffer) are disabled
		JsonPrimitive.Attributes.Normal = Builder.AddUniqueNormalAccessor(ConvertedSection, &VertexBuffer);
		JsonPrimitive.Attributes.Tangent = Builder.AddUniqueTangentAccessor(ConvertedSection, &VertexBuffer);

		const uint32 UVCount = VertexBuffer.GetNumTexCoords();
		// TODO: report warning or option to limit UV channels since most viewers don't support more than 2?
		JsonPrimitive.Attributes.TexCoords.AddUninitialized(UVCount);

		for (uint32 UVIndex = 0; UVIndex < UVCount; ++UVIndex)
		{
			JsonPrimitive.Attributes.TexCoords[UVIndex] = Builder.AddUniqueUVAccessor(ConvertedSection, &VertexBuffer, UVIndex);
		}

		if (Builder.ExportOptions->bExportVertexSkinWeights)
		{
			const uint32 GroupCount = (SkinWeightBuffer->GetMaxBoneInfluences() + 3) / 4;
			// TODO: report warning or option to limit groups (of joints and weights) since most viewers don't support more than one?
			JsonPrimitive.Attributes.Joints.AddUninitialized(GroupCount);
			JsonPrimitive.Attributes.Weights.AddUninitialized(GroupCount);

			for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
			{
				JsonPrimitive.Attributes.Joints[GroupIndex] = Builder.AddUniqueJointAccessor(ConvertedSection, SkinWeightBuffer, GroupIndex * 4);
				JsonPrimitive.Attributes.Weights[GroupIndex] = Builder.AddUniqueWeightAccessor(ConvertedSection, SkinWeightBuffer, GroupIndex * 4);
			}
		}

		const UMaterialInterface* Material = Materials[MaterialIndex];
		JsonPrimitive.Material = Builder.AddUniqueMaterial(Material, MeshData, SectionIndices);
	}
}


FGLTFDelayedLandscapeTask::FGLTFDelayedLandscapeTask(FGLTFConvertBuilder& Builder, const ULandscapeComponent& LandscapeComponent, FGLTFJsonMesh* JsonMesh, const UMaterialInterface& LandscapeMaterial)
	: FGLTFDelayedTask(EGLTFTaskPriority::Mesh)
	, Builder(Builder)
	, LandscapeComponent(LandscapeComponent)
	, JsonMesh(JsonMesh)
	, LandscapeMaterial(LandscapeMaterial)
{
}

FString FGLTFDelayedLandscapeTask::GetName()
{
	return LandscapeComponent.GetName();
}

void FGLTFDelayedLandscapeTask::Process()
{
	const ALandscapeProxy* Landscape = Cast<ALandscapeProxy>(LandscapeComponent.GetOwner());
	JsonMesh->Name = LandscapeComponent.GetName();

	int32 MinX = MAX_int32, MinY = MAX_int32;
	int32 MaxX = MIN_int32, MaxY = MIN_int32;

	// Create and fill in the vertex position data source.
	int32 ExportLOD = 0;
#if WITH_EDITOR
	ExportLOD = Landscape->ExportLOD;
#endif
	const int32 ComponentSizeQuads = ((Landscape->ComponentSizeQuads + 1) >> ExportLOD) - 1;
	const float ScaleFactor = (float)Landscape->ComponentSizeQuads / (float)ComponentSizeQuads;
	const int32 VertexCount = FMath::Square(ComponentSizeQuads + 1);
	const int32 TriangleCount = FMath::Square(ComponentSizeQuads) * 2;

	FGLTFIndexArray Indices;
	FGLTFPositionArray PositionBuffer;
	FGLTFColorArray VertexColorBuffer;
	FGLTFNormalArray Normals;
	FGLTFTangentArray Tangents;
	FGLTFUVArray UV;

	Indices.Reserve(FMath::Square(ComponentSizeQuads) * 2 * 3);
	PositionBuffer.SetNumZeroed(VertexCount);
	VertexColorBuffer.SetNumZeroed(VertexCount);
	Normals.SetNumZeroed(VertexCount);
	Tangents.SetNumZeroed(VertexCount);
	UV.SetNumZeroed(VertexCount);

	FGLTFJsonPrimitive& JsonPrimitive = JsonMesh->Primitives[0];
	TArray<uint8> VisibilityData;
	VisibilityData.SetNumZeroed(VertexCount);

	int OffsetX = Landscape->GetSectionBase().X;
	int OffsetY = Landscape->GetSectionBase().Y;
	
	FGLTFLandscapeComponentDataInterface CDI(LandscapeComponent, ExportLOD);

	TArray<uint8> CompVisData;
	const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = LandscapeComponent.GetWeightmapLayerAllocations();

	for (int32 AllocIdx = 0; AllocIdx < ComponentWeightmapLayerAllocations.Num(); AllocIdx++)
	{
		const FWeightmapLayerAllocationInfo& AllocInfo = ComponentWeightmapLayerAllocations[AllocIdx];
		//Landscape Visibility Layer is named: __LANDSCAPE_VISIBILITY__
		//based on: Engine/Source/Runtime/Landscape/Private/Materials/MaterialExpressionLandscapeVisibilityMask.cpp
		//		FName UMaterialExpressionLandscapeVisibilityMask::ParameterName = FName("__LANDSCAPE_VISIBILITY__");
		FString LayerName = AllocInfo.LayerInfo->GetLayerName().ToString();
		if (LayerName == TEXT("__LANDSCAPE_VISIBILITY__"))
		{
			CDI.GetWeightmapTextureData(AllocInfo.LayerInfo, CompVisData);
		}
	}

	if (CompVisData.Num() > 0)
	{
		for (int32 i = 0; i < VertexCount; ++i)
		{
			VisibilityData[i] = CompVisData[CDI.VertexIndexToTexel(i)];
		}
	}

	for (int32 VertexIndex = 0; VertexIndex < VertexCount; VertexIndex++)
	{
		int32 VertX, VertY;
		CDI.VertexIndexToXY(VertexIndex, VertX, VertY);

		FVector3f Position;
		FVector3f Normal;
		FVector2f UVElement;
		CDI.GetPositionNormalUV(VertX, VertY, Position, Normal, UVElement);

		PositionBuffer[VertexIndex] = Position;
		Normals[VertexIndex] = Normal;
		UV[VertexIndex] = UVElement;
	}

	const int32 VisThreshold = 170;
	
	for (int32 Y = 0; Y < ComponentSizeQuads; Y++)
	{
		for (int32 X = 0; X < ComponentSizeQuads; X++)
		{
			if (VisibilityData[Y * (ComponentSizeQuads + 1) + X] < VisThreshold)
			{
				Indices.Push((X + 0) + (Y + 0) * (ComponentSizeQuads + 1));
				Indices.Push((X + 1) + (Y + 1) * (ComponentSizeQuads + 1));
				Indices.Push((X + 1) + (Y + 0) * (ComponentSizeQuads + 1));

				Indices.Push((X + 0) + (Y + 0) * (ComponentSizeQuads + 1));
				Indices.Push((X + 0) + (Y + 1) * (ComponentSizeQuads + 1));
				Indices.Push((X + 1) + (Y + 1) * (ComponentSizeQuads + 1));
			}
		}
	}

	if (Indices.Num())
	{
		JsonPrimitive.Attributes.Position = Builder.AddUniquePositionAccessor(PositionBuffer);
		JsonPrimitive.Attributes.Normal = Builder.AddUniqueNormalAccessor(Normals);
		JsonPrimitive.Attributes.TexCoords.AddUninitialized(1);
		JsonPrimitive.Attributes.TexCoords[0] = Builder.AddUniqueUVAccessor(UV);
		JsonPrimitive.Indices = Builder.AddUniqueIndexAccessor(Indices, JsonMesh->Name);
		JsonPrimitive.Material = Builder.AddUniqueMaterial(&LandscapeMaterial);
	}
}