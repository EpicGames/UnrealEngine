// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/GLTFDelayedTask.h"
#include "Builders/GLTFConvertBuilder.h"
#include "Converters/GLTFMeshSectionConverters.h"
#include "Converters/GLTFMaterialArray.h"
#include "Converters/GLTFNameUtilities.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"

class SkeletalMeshComponent;
class StaticMeshComponent;

class FGLTFDelayedStaticAndSplineMeshTask : public FGLTFDelayedTask
{
public:
	FGLTFDelayedStaticAndSplineMeshTask(FGLTFConvertBuilder& Builder, FGLTFStaticMeshSectionConverter& MeshSectionConverter, const UStaticMesh* StaticMesh, 
		const UStaticMeshComponent* StaticMeshComponent, const USplineMeshComponent* SplineMeshComponent,
		FGLTFMaterialArray& Materials, int32 LODIndex, FGLTFJsonMesh* JsonMesh)
		: FGLTFDelayedTask(EGLTFTaskPriority::Mesh)
		, Builder(Builder)
		, MeshSectionConverter(MeshSectionConverter)
		, StaticMesh(StaticMesh)
		, StaticMeshComponent(StaticMeshComponent)
		, SplineMeshComponent(SplineMeshComponent)
		, Materials(Materials)
		, LODIndex(LODIndex)
		, JsonMesh(JsonMesh)
	{
	}

	virtual FString GetName() override;

	virtual void Process() override;

private:
#if WITH_EDITORONLY_DATA
	void ProcessMeshDescription(const TArray<FStaticMaterial>& MaterialSlots, const FGLTFMeshData* MeshData);
#endif
	void ProcessRenderData(const TArray<FStaticMaterial>& MaterialSlots, const FGLTFMeshData* MeshData);

	FGLTFConvertBuilder& Builder;
	FGLTFStaticMeshSectionConverter& MeshSectionConverter;
	const UStaticMesh* StaticMesh;
	const UStaticMeshComponent* StaticMeshComponent;
	const USplineMeshComponent* SplineMeshComponent;
	const FGLTFMaterialArray Materials; //We copy because the received value can be temporary value.
	const int32 LODIndex;
	FGLTFJsonMesh* JsonMesh;
};

class FGLTFDelayedStaticMeshTask : public FGLTFDelayedStaticAndSplineMeshTask
{
public:

	FGLTFDelayedStaticMeshTask(FGLTFConvertBuilder& Builder, FGLTFStaticMeshSectionConverter& MeshSectionConverter, const UStaticMesh* StaticMesh, const UStaticMeshComponent* StaticMeshComponent, FGLTFMaterialArray& Materials, int32 LODIndex, FGLTFJsonMesh* JsonMesh)
		: FGLTFDelayedStaticAndSplineMeshTask(Builder, MeshSectionConverter, StaticMesh, StaticMeshComponent, nullptr, Materials, LODIndex, JsonMesh)
	{
	}
};

class FGLTFDelayedSplineMeshTask : public FGLTFDelayedStaticAndSplineMeshTask
{
public:

	FGLTFDelayedSplineMeshTask(FGLTFConvertBuilder& Builder, FGLTFStaticMeshSectionConverter& MeshSectionConverter, const UStaticMesh* StaticMesh, const USplineMeshComponent* SplineMeshComponent, FGLTFMaterialArray& Materials, int32 LODIndex, FGLTFJsonMesh* JsonMesh)
		: FGLTFDelayedStaticAndSplineMeshTask(Builder, MeshSectionConverter, StaticMesh, nullptr, SplineMeshComponent, Materials, LODIndex, JsonMesh)
	{
	}
};

class FGLTFDelayedSkeletalMeshTask : public FGLTFDelayedTask
{
public:

	FGLTFDelayedSkeletalMeshTask(FGLTFConvertBuilder& Builder, FGLTFSkeletalMeshSectionConverter& MeshSectionConverter, const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent, FGLTFMaterialArray& Materials, int32 LODIndex, FGLTFJsonMesh* JsonMesh)
		: FGLTFDelayedTask(EGLTFTaskPriority::Mesh)
		, Builder(Builder)
		, MeshSectionConverter(MeshSectionConverter)
		, SkeletalMesh(SkeletalMesh)
		, SkeletalMeshComponent(SkeletalMeshComponent)
		, Materials(Materials)
		, LODIndex(LODIndex)
		, JsonMesh(JsonMesh)
	{
	}

	virtual FString GetName() override;

	virtual void Process() override;

private:
#if WITH_EDITORONLY_DATA
	void ProcessSourceModel(const TArray<FSkeletalMaterial>& MaterialSlots, const FGLTFMeshData* MeshData);
#endif
	void ProcessRenderData(const TArray<FSkeletalMaterial>& MaterialSlots, const FGLTFMeshData* MeshData);

	FGLTFConvertBuilder& Builder;
	FGLTFSkeletalMeshSectionConverter& MeshSectionConverter;
	const USkeletalMesh* SkeletalMesh;
	const USkeletalMeshComponent* SkeletalMeshComponent;
	const FGLTFMaterialArray Materials; //We copy because the received value can be temporary value.
	const int32 LODIndex;
	FGLTFJsonMesh* JsonMesh;
};

class FGLTFDelayedLandscapeTask : public FGLTFDelayedTask
{
public:

	FGLTFDelayedLandscapeTask(FGLTFConvertBuilder& Builder, const ULandscapeComponent& LandscapeComponent, FGLTFJsonMesh* JsonMesh, const UMaterialInterface& LandscapeMaterial);

	virtual FString GetName() override;

	virtual void Process() override;

private:

	FGLTFConvertBuilder& Builder;
	const ULandscapeComponent& LandscapeComponent;
	FGLTFJsonMesh* JsonMesh;
	const UMaterialInterface& LandscapeMaterial;
};
