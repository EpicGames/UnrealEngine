// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/GLTFDelayedTextureTasks.h"
#include "Builders/GLTFContainerBuilder.h"
#include "Utilities/GLTFCoreUtilities.h"
#include "Converters/GLTFTextureUtilities.h"
#include "Converters/GLTFNameUtilities.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"

FString FGLTFDelayedTexture2DTask::GetName()
{
	if (!Texture2D)
	{
		return TEXT("");
	}
	return Texture2D->GetName();
}

void FGLTFDelayedTexture2DTask::Process()
{
	FGLTFTextureUtilities::FullyLoad(Texture2D);
	Texture2D->GetName(JsonTexture->Name);

	const bool bFromSRGB = Texture2D->SRGB;
	if (bFromSRGB != bToSRGB)
	{
		JsonTexture->Name += bToSRGB ? TEXT("_sRGB") : TEXT("_Linear");
	}

	const FIntPoint Size = FGLTFTextureUtilities::GetInGameSize(Texture2D);
	UTextureRenderTarget2D* RenderTarget = FGLTFTextureUtilities::CreateRenderTarget(Size, false);

	// TODO: preserve maximum image quality (avoid compression artifacts) by copying source data (and adjustments) to a temp texture
	FGLTFTextureUtilities::DrawTexture(RenderTarget, Texture2D);

	TGLTFSharedArray<FColor> Pixels;
	if (!FGLTFTextureUtilities::ReadPixels(RenderTarget, *Pixels))
	{
		Builder.LogWarning(FString::Printf(TEXT("Failed to read pixels for 2D texture %s"), *JsonTexture->Name));
		return;
	}

	if (Builder.ExportOptions->bAdjustNormalmaps && Texture2D->IsNormalMap())
	{
		// TODO: add support for adjusting normals in GLTFNormalMapPreview instead
		FGLTFTextureUtilities::FlipGreenChannel(*Pixels);
	}

	FGLTFTextureUtilities::TransformColorSpace(*Pixels, bFromSRGB, bToSRGB);

	const bool bIgnoreAlpha = FGLTFTextureUtilities::IsAlphaless(Texture2D->GetPixelFormat());

	JsonTexture->Source = Builder.AddUniqueImage(Pixels, Size, bIgnoreAlpha, JsonTexture->Name);

	if (TextureAddressX == TextureAddress::TA_MAX) TextureAddressX = Texture2D->GetTextureAddressX();
	if (TextureAddressY == TextureAddress::TA_MAX) TextureAddressY = Texture2D->GetTextureAddressX();

	JsonTexture->Sampler = Builder.AddUniqueSampler(TextureAddressX, TextureAddressY, Texture2D->Filter, Texture2D->LODGroup);
}

FString FGLTFDelayedTextureRenderTarget2DTask::GetName()
{
	if (!RenderTarget2D)
	{
		return TEXT("");
	}
	return RenderTarget2D->GetName();
}

void FGLTFDelayedTextureRenderTarget2DTask::Process()
{
	FGLTFTextureUtilities::FullyLoad(RenderTarget2D);
	RenderTarget2D->GetName(JsonTexture->Name);

	const bool bFromSRGB = RenderTarget2D->SRGB;
	if (bFromSRGB != bToSRGB)
	{
		JsonTexture->Name += bToSRGB ? TEXT("_sRGB") : TEXT("_Linear");
	}

	TGLTFSharedArray<FColor> Pixels;
	if (!FGLTFTextureUtilities::ReadPixels(RenderTarget2D, *Pixels))
	{
		Builder.LogWarning(FString::Printf(TEXT("Failed to read pixels for 2D render target %s"), *JsonTexture->Name));
		return;
	}

	FGLTFTextureUtilities::TransformColorSpace(*Pixels, bFromSRGB, bToSRGB);

	const FIntPoint Size = { RenderTarget2D->SizeX, RenderTarget2D->SizeY };
	const bool bIgnoreAlpha = FGLTFTextureUtilities::IsAlphaless(RenderTarget2D->GetFormat());

	JsonTexture->Source = Builder.AddUniqueImage(Pixels, Size, bIgnoreAlpha, JsonTexture->Name);
	JsonTexture->Sampler = Builder.AddUniqueSampler(RenderTarget2D);
}

#if WITH_EDITOR

FString FGLTFDelayedTextureLightMapTask::GetName()
{
	if (!LightMap)
	{
		return TEXT("");
	}
	return LightMap->GetName();
}

void FGLTFDelayedTextureLightMapTask::Process()
{
	FGLTFTextureUtilities::FullyLoad(LightMap);
	LightMap->GetName(JsonTexture->Name);

	const FIntPoint Size = FGLTFTextureUtilities::GetInGameSize(LightMap);
	UTextureRenderTarget2D* RenderTarget = FGLTFTextureUtilities::CreateRenderTarget(Size, false);

	// TODO: preserve maximum image quality (avoid compression artifacts) by copying source data (and adjustments) to a temp texture
	FGLTFTextureUtilities::DrawTexture(RenderTarget, LightMap);

	TGLTFSharedArray<FColor> Pixels;
	if (!FGLTFTextureUtilities::ReadPixels(RenderTarget, *Pixels))
	{
		Builder.LogWarning(FString::Printf(TEXT("Failed to read pixels for 2D texture %s"), *JsonTexture->Name));
		return;
	}

	if (Builder.ExportOptions->bAdjustNormalmaps && LightMap->IsNormalMap())
	{
		// TODO: add support for adjusting normals in GLTFNormalMapPreview instead
		FGLTFTextureUtilities::FlipGreenChannel(*Pixels);
	}

	const bool bIgnoreAlpha = FGLTFTextureUtilities::IsAlphaless(LightMap->GetPixelFormat());

	JsonTexture->Source = Builder.AddUniqueImage(Pixels, Size, bIgnoreAlpha, JsonTexture->Name);
	JsonTexture->Sampler = Builder.AddUniqueSampler(LightMap->GetTextureAddressX(), LightMap->GetTextureAddressX(), LightMap->Filter, LightMap->LODGroup);
}

#endif
