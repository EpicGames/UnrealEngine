// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/GLTFDelayedTask.h"
#include "Builders/GLTFConvertBuilder.h"

class FGLTFDelayedTexture2DTask : public FGLTFDelayedTask
{
public:

	FGLTFDelayedTexture2DTask(FGLTFConvertBuilder& Builder, const UTexture2D* Texture2D, bool bToSRGB, FGLTFJsonTexture* JsonTexture, TextureAddress InTextureAddressX, TextureAddress InTextureAddressY)
		: FGLTFDelayedTask(EGLTFTaskPriority::Texture)
		, Builder(Builder)
		, Texture2D(Texture2D)
		, bToSRGB(bToSRGB)
		, TextureAddressX(InTextureAddressX)
		, TextureAddressY(InTextureAddressY)
		, JsonTexture(JsonTexture)
	{
	}

	virtual FString GetName() override;

	virtual void Process() override;

private:

	FGLTFConvertBuilder& Builder;
	const UTexture2D* Texture2D = nullptr;
	bool bToSRGB;
	TextureAddress TextureAddressX;
	TextureAddress TextureAddressY;
	FGLTFJsonTexture* JsonTexture = nullptr;
};

class FGLTFDelayedTextureRenderTarget2DTask : public FGLTFDelayedTask
{
public:

	FGLTFDelayedTextureRenderTarget2DTask(FGLTFConvertBuilder& Builder, const UTextureRenderTarget2D* RenderTarget2D, bool bToSRGB, FGLTFJsonTexture* JsonTexture)
		: FGLTFDelayedTask(EGLTFTaskPriority::Texture)
		, Builder(Builder)
		, RenderTarget2D(RenderTarget2D)
		, bToSRGB(bToSRGB)
		, JsonTexture(JsonTexture)
	{
	}

	virtual FString GetName() override;

	virtual void Process() override;

private:

	FGLTFConvertBuilder& Builder;
	const UTextureRenderTarget2D* RenderTarget2D = nullptr;
	bool bToSRGB;
	FGLTFJsonTexture* JsonTexture = nullptr;
};

#if WITH_EDITOR

class FGLTFDelayedTextureLightMapTask : public FGLTFDelayedTask
{
public:

	FGLTFDelayedTextureLightMapTask(FGLTFConvertBuilder& Builder, const ULightMapTexture2D* LightMap, FGLTFJsonTexture* JsonTexture)
		: FGLTFDelayedTask(EGLTFTaskPriority::Texture)
		, Builder(Builder)
		, LightMap(LightMap)
		, JsonTexture(JsonTexture)
	{
	}

	virtual FString GetName() override;

	virtual void Process() override;

private:

	FGLTFConvertBuilder& Builder;
	const ULightMapTexture2D* LightMap = nullptr;
	FGLTFJsonTexture* JsonTexture = nullptr;
};

#endif
