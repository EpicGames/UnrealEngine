// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "UI/GLTFExportOptionsWindow.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"

#define LOCTEXT_NAMESPACE "GLTFExportOptionsWindow"

SGLTFExportOptionsWindow::SGLTFExportOptionsWindow()
	: ExportOptions(nullptr)
	, bShouldExport(false)
	, bShouldExportAll(false)
{
}

void SGLTFExportOptionsWindow::Construct(const FArguments& InArgs)
{
	ExportOptions = InArgs._ExportOptions;
	WidgetWindow = InArgs._WidgetWindow;

	const FText CancelText = InArgs._BatchMode ? LOCTEXT("CancelBatch", "Cancel All") : LOCTEXT("Cancel", "Cancel");
	const FText CancelTooltipText = InArgs._BatchMode ? LOCTEXT("CancelBatch_ToolTip", "Cancel the batch export.") : LOCTEXT("Cancel_ToolTip", "Cancel the current glTF export.");

	TSharedPtr<SBox> HeaderToolBox;
	TSharedPtr<SHorizontalBox> HeaderButtons;
	TSharedPtr<SBox> InspectorBox;
	this->ChildSlot
	[
		SNew(SBox)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SAssignNew(HeaderToolBox, SBox)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(SBorder)
				.Padding(FMargin(3.0f))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
						.Text(LOCTEXT("Export_CurrentFileTitle", "Current File: "))
					]
					+SHorizontalBox::Slot()
					.Padding(5.0f, 0.0f, 0.0f, 0.0f)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("CurveEd.InfoFont"))
						.Text(InArgs._FullPath)
					]
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(2.0f)
			[
				SAssignNew(InspectorBox, SBox)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(2.0f)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2.0f)
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("ExportAll", "Export All"))
					.ToolTipText(LOCTEXT("ExportAll_ToolTip", "Export all files with these same settings"))
					.Visibility(InArgs._BatchMode ? EVisibility::All : EVisibility::Hidden)
					.OnClicked(this, &SGLTFExportOptionsWindow::OnExportAll)
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SAssignNew(ExportButton, SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("Export", "Export"))
					.OnClicked(this, &SGLTFExportOptionsWindow::OnExport)
				]
				+ SUniformGridPanel::Slot(2, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(CancelText)
					.ToolTipText(CancelTooltipText)
					.OnClicked(this, &SGLTFExportOptionsWindow::OnCancel)
				]
			]
		]
	];

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	InspectorBox->SetContent(DetailsView->AsShared());

	HeaderToolBox->SetContent(
		SNew(SBorder)
		.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SBox)
				.HAlign(HAlign_Right)
				[
					SAssignNew(HeaderButtons, SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(2.0f, 0.0f))
					[
						SNew(SButton)
						.Text(LOCTEXT("Reset", "Reset to Default"))
						.OnClicked(this, &SGLTFExportOptionsWindow::OnReset)
					]
				]
			]
		]
	);

	DetailsView->SetObject(ExportOptions);
}

FReply SGLTFExportOptionsWindow::OnReset() const
{
	ExportOptions->ResetToDefault();
	//Refresh the view to make sure the custom UI are updating correctly
	DetailsView->SetObject(ExportOptions, true);
	return FReply::Handled();
}

FReply SGLTFExportOptionsWindow::OnExport()
{
	bShouldExport = true;
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SGLTFExportOptionsWindow::OnExportAll()
{
	bShouldExportAll = true;
	return OnExport();
}

FReply SGLTFExportOptionsWindow::OnCancel()
{
	bShouldExport = false;
	bShouldExportAll = false;
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SGLTFExportOptionsWindow::OnKeyDown(const FGeometry & MyGeometry, const FKeyEvent & InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return OnCancel();
	}

	return FReply::Unhandled();
}

bool SGLTFExportOptionsWindow::ShouldExport() const
{
	return bShouldExport;
}

bool SGLTFExportOptionsWindow::ShouldExportAll() const
{
	return bShouldExportAll;
}

void SGLTFExportOptionsWindow::ShowDialog(UGLTFExportOptions* ExportOptions, const FString& FullPath, bool bBatchMode, bool& bOutOperationCanceled, bool& bOutExportAll)
{
	bOutOperationCanceled = false;
	bOutExportAll = false;

	const bool bBinaryFile = FPaths::GetExtension(FullPath, false).Compare(TEXT("glb"), ESearchCase::IgnoreCase) == 0;
	const FText Title = bBinaryFile ?
		LOCTEXT("TitleBinary", "glTF (Binary) Export Options") :
		LOCTEXT("Title", "glTF Export Options");

	const TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(Title)
		.SizingRule(ESizingRule::UserSized)
		.AutoCenter(EAutoCenter::PrimaryWorkArea)
		.ClientSize(FVector2D(500, 500));

	TSharedPtr<SGLTFExportOptionsWindow> OptionsWindow;
	Window->SetContent
	(
		SAssignNew(OptionsWindow, SGLTFExportOptionsWindow)
		.ExportOptions(ExportOptions)
		.WidgetWindow(Window)
		.FullPath(FText::FromString(FullPath))
		.BatchMode(bBatchMode)
	);

	IMainFrameModule* MainFrame = FModuleManager::LoadModulePtr<IMainFrameModule>("MainFrame");
	const TSharedPtr<SWindow> ParentWindow = MainFrame != nullptr ? MainFrame->GetParentWindow() : nullptr;

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

	if (OptionsWindow->ShouldExport())
	{
		bOutExportAll = OptionsWindow->ShouldExportAll();
	}
	else
	{
		bOutOperationCanceled = true;
	}
}

#undef LOCTEXT_NAMESPACE

#endif
