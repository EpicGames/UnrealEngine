// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Options/GLTFExportOptions.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"

class SButton;

class SGLTFExportOptionsWindow : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SGLTFExportOptionsWindow)
		: _ExportOptions(nullptr)
		, _BatchMode()
		{}

		SLATE_ARGUMENT( UGLTFExportOptions*, ExportOptions )
		SLATE_ARGUMENT( TSharedPtr<SWindow>, WidgetWindow )
		SLATE_ARGUMENT( FText, FullPath )
		SLATE_ARGUMENT( bool, BatchMode )
	SLATE_END_ARGS()

	GLTFEXPORTER_API SGLTFExportOptionsWindow();

	GLTFEXPORTER_API void Construct(const FArguments& InArgs);

	GLTFEXPORTER_API FReply OnReset() const;
	GLTFEXPORTER_API FReply OnExport();
	GLTFEXPORTER_API FReply OnExportAll();
	GLTFEXPORTER_API FReply OnCancel();

	/* Begin SCompoundWidget overrides */
	virtual bool SupportsKeyboardFocus() const override { return true; }
	GLTFEXPORTER_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	/* End SCompoundWidget overrides */

	GLTFEXPORTER_API bool ShouldExport() const;
	GLTFEXPORTER_API bool ShouldExportAll() const;

	static GLTFEXPORTER_API void ShowDialog(UGLTFExportOptions* ExportOptions, const FString& FullPath, bool bBatchMode, bool& bOutOperationCanceled, bool& bOutExportAll);

private:

	UGLTFExportOptions* ExportOptions;
	TSharedPtr<class IDetailsView> DetailsView;
	TWeakPtr<SWindow> WidgetWindow;
	TSharedPtr<SButton> ExportButton;
	bool bShouldExport;
	bool bShouldExportAll;
};

#endif
