// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "UI/GLTFMaterialBakeSizeCustomization.h"
#include "UserData/GLTFMaterialUserData.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "DetailWidgetRow.h"
#include "IPropertyUtilities.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "GLTFMaterialBakeSettingsDetails"

void FGLTFMaterialBakeSizeCustomization::Register()
{
	static FName PropertyTypeName("GLTFMaterialBakeSize");
	static FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);
	PropertyModule.RegisterCustomPropertyTypeLayout(PropertyTypeName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FGLTFMaterialBakeSizeCustomization::MakeInstance));
}

void FGLTFMaterialBakeSizeCustomization::Unregister()
{
	static FName PropertyTypeName("GLTFMaterialBakeSize");
	static FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);
	PropertyModule.UnregisterCustomPropertyTypeLayout(PropertyTypeName);
}

TSharedRef<IPropertyTypeCustomization> FGLTFMaterialBakeSizeCustomization::MakeInstance()
{
	return MakeShared<FGLTFMaterialBakeSizeCustomization>();
}

void FGLTFMaterialBakeSizeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	UE_LOGF(LogTemp, Warning, "FGLTFMaterialBakeSettingsDetails: %ls", *FGLTFMaterialBakeSize::StaticStruct()->GetName());
	StructPropertyHandle = PropertyHandle;

	UStruct* Struct = FGLTFMaterialBakeSize::StaticStruct();
	
	FProperty* Property = Struct->FindPropertyByName(TEXT("bAutoDetect"));

	FText AutoDetectToolTip = FText::FromString(Property->GetMetaData(TEXT("ToolTip")));
	
	TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();

	int32 Size = 1;
	while (Size <= 8192)
	{		
		OptionsSource.Add(MakeShared<FString>(FString::Printf(TEXT("%d x %d"), Size, Size)));
		Size *= 2;
	}

	FSlateFontInfo SmallFont = FCoreStyle::Get().GetFontStyle("SmallFont");

	HeaderRow
	.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
	.ValueContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSearchableComboBox)
				//.ContentPadding(0)
				.OptionsSource(&OptionsSource)
				.OnGenerateWidget_Lambda([SmallFont](const TSharedPtr<FString>& Item)
				{
					return SNew(STextBlock)
						.Text(FText::FromString(*Item))
						.Font(SmallFont);
				})
				.OnSelectionChanged_Lambda([this](const TSharedPtr<FString>& SelectedItem, ESelectInfo::Type Type)
				{
					SetSizeString(*SelectedItem);
				})
				.InitiallySelectedItem(GetSelectedOption())
				.SearchVisibility(EVisibility::Collapsed)
				.Content()
				[
					SNew(SEditableText)
					.MinDesiredWidth(89.0f) // TODO: instead of hardcoded value, should calculate it based on something
					.RevertTextOnEscape(true)
					.SelectAllTextWhenFocused(true)
					.Text_Lambda([this]()
					{
						return FText::FromString(GetSizeString());
					})
					.OnTextCommitted_Lambda([this](const FText& NewText, ETextCommit::Type Type)
                    {
						SetSizeString(NewText.ToString());
                    })
					.Font(SmallFont)
				]
			]
			+SHorizontalBox::Slot()
			.Padding(10.0f, 0.0f, 0.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]()
				{
					bool bAutoDetect;
					return TryGetPropertyValue<bool>(bAutoDetect, [](const FGLTFMaterialBakeSize& Size){ return Size.bAutoDetect; })
						? (bAutoDetect ? ECheckBoxState::Checked : ECheckBoxState::Unchecked) : ECheckBoxState::Undetermined;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
				{
					bool bAutoDetect = NewState == ECheckBoxState::Checked;
					SetPropertyValue([bAutoDetect](FGLTFMaterialBakeSize& Size){ Size.bAutoDetect = bAutoDetect; });
				})
				.ToolTipText(AutoDetectToolTip)
				[
					SNew(STextBlock)
					.Margin(FMargin(3.0f, 0.0f, 0.0f, 0.0f))
					.Text(LOCTEXT("AutoDetectLabel", "Auto-detect"))
					.Font(SmallFont)
				]
			]
		].IsEnabled(MakeAttributeLambda([=] { return !PropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }));
}

void FGLTFMaterialBakeSizeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

FString FGLTFMaterialBakeSizeCustomization::GetSizeString() const
{
	FIntPoint Value;
	if (!TryGetPropertyValue<FIntPoint>(Value, [](const FGLTFMaterialBakeSize& Size) { return FIntPoint(Size.X, Size.Y); }))
	{
		return TEXT("");
	}

	return FString::Printf(TEXT("%d x %d"), Value.X, Value.Y);
}

void FGLTFMaterialBakeSizeCustomization::SetSizeString(const FString& String) const
{
	auto ParseInt = [](const FString& Text, int32& OutValue, int32 DefaultValue)
	{
		const TCHAR* TextStart = *Text;
		TCHAR* TextEnd;

		OutValue = FCString::Strtoi(TextStart, &TextEnd, 10);
		if (TextEnd == TextStart || OutValue <= 0)
		{
			OutValue = DefaultValue;
		}
	};

	FIntPoint Value;

	FString Left, Right;
	if (String.Split(TEXT("x"), &Left, &Right))
	{
		ParseInt(*Left, Value.X, FGLTFMaterialBakeSize().X);
		ParseInt(*Right, Value.Y, FGLTFMaterialBakeSize().Y);
	}
	else
	{
		ParseInt(*String, Value.X, FGLTFMaterialBakeSize().X);
		Value.Y = Value.X;
	}

	SetPropertyValue([Value](FGLTFMaterialBakeSize& Size)
	{
		Size.X = Value.X;
		Size.Y = Value.Y;
	});
}

TSharedPtr<FString> FGLTFMaterialBakeSizeCustomization::GetSelectedOption() const
{
	const FString CurrentValue = GetSizeString();
	const TSharedPtr<FString>* CurrentItem = OptionsSource.FindByPredicate([CurrentValue](const TSharedPtr<FString>& Item) { return *Item == CurrentValue; });
	return CurrentItem != nullptr ? *CurrentItem : nullptr;
}

template <typename SetterType>
void FGLTFMaterialBakeSizeCustomization::SetPropertyValue(SetterType Setter) const
{
	TArray<void*> RawPtrs;
	StructPropertyHandle->AccessRawData(RawPtrs);
	StructPropertyHandle->NotifyPreChange();

	for (void* RawPtr : RawPtrs)
	{
		if (FGLTFMaterialBakeSize* Ptr = static_cast<FGLTFMaterialBakeSize*>(RawPtr))
		{
			Setter(*Ptr);
		}
	}

	StructPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	StructPropertyHandle->NotifyFinishedChangingProperties();
}

template <typename ValueType, typename GetterType>
bool FGLTFMaterialBakeSizeCustomization::TryGetPropertyValue(ValueType& OutValue, GetterType Getter) const
{
	TArray<const void*> RawPtrs;
	StructPropertyHandle->AccessRawData(RawPtrs);

	if (RawPtrs.Num() == 0)
	{
		return false;
	}

	const FGLTFMaterialBakeSize* FirstPtr = static_cast<const FGLTFMaterialBakeSize*>(RawPtrs[0]);
	if (FirstPtr == nullptr)
	{
		return false;
	}

	ValueType FirstValue = Getter(*FirstPtr);

	for (int32 Index = 1; Index < RawPtrs.Num(); ++Index)
	{
		if (const FGLTFMaterialBakeSize* Ptr = static_cast<const FGLTFMaterialBakeSize*>(RawPtrs[Index]))
		{
			ValueType Value = Getter(*Ptr);
			if (Value != FirstValue)
			{
				return false;
			}
		}
	}

	OutValue = FirstValue;
	return true;
}

#undef LOCTEXT_NAMESPACE

#endif
