// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Containers/Map.h"
#include "HAL/PlatformCrt.h"
#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class FName;
class FString;
class IDetailLayoutBuilder;
class UEnum;

class FGLTFMaterialBakeSizeCustomization : public IPropertyTypeCustomization
{
public:

	static void Register();
	static void Unregister();

	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:

	FString GetSizeString() const;
	void SetSizeString(const FString& String) const;
	TSharedPtr<FString> GetSelectedOption() const;

	template <typename SetterType>
	void SetPropertyValue(SetterType Setter) const;

	template <typename ValueType, typename GetterType>
	bool TryGetPropertyValue(ValueType& OutValue, GetterType Getter) const;

	TSharedPtr<IPropertyHandle> StructPropertyHandle;
	TArray<TSharedPtr<FString>> OptionsSource;
};

#endif
