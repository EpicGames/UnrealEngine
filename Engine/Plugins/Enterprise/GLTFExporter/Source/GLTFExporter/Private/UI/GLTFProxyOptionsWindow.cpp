// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "UI/GLTFProxyOptionsWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "Interfaces/IMainFrameModule.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "GLTFProxyOptionsWindow"

SGLTFProxyOptionsWindow::SGLTFProxyOptionsWindow()
	: ProxyOptions(nullptr)
	, bUserCancelled(true)
{
}

void SGLTFProxyOptionsWindow::Construct(const FArguments& InArgs)
{
	ProxyOptions = InArgs._ProxyOptions;
	WidgetWindow = InArgs._WidgetWindow;

	TSharedPtr<SBox> InspectorBox;
	this->ChildSlot
	[
		SNew(SBox)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(2.0f)
			[
				SAssignNew(InspectorBox, SBox)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(2.0f)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2.0f)
				+ SUniformGridPanel::Slot(0, 0)
				[
					SAssignNew(ConfirmButton, SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("Confirm", "Confirm"))
					.OnClicked(this, &SGLTFProxyOptionsWindow::OnConfirm)
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("Cancel", "Cancel"))
					.OnClicked(this, &SGLTFProxyOptionsWindow::OnCancel)
				]
			]
		]
	];

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	InspectorBox->SetContent(DetailsView->AsShared());
	DetailsView->SetObject(ProxyOptions);
}

FReply SGLTFProxyOptionsWindow::OnConfirm()
{
	bUserCancelled = false;
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SGLTFProxyOptionsWindow::OnCancel()
{
	bUserCancelled = true;
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SGLTFProxyOptionsWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return OnCancel();
	}

	return FReply::Unhandled();
}

bool SGLTFProxyOptionsWindow::ShowDialog(UGLTFProxyOptions* ProxyOptions)
{
	const TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("Title", "Create glTF Proxy Material"))
		.SizingRule(ESizingRule::UserSized)
		.AutoCenter(EAutoCenter::PrimaryWorkArea)
		.ClientSize(FVector2D(500, 250));

	TSharedPtr<SGLTFProxyOptionsWindow> OptionsWindow;

	Window->SetContent
	(
		SAssignNew(OptionsWindow, SGLTFProxyOptionsWindow)
		.WidgetWindow(Window)
		.ProxyOptions(ProxyOptions)
	);

	IMainFrameModule* MainFrame = FModuleManager::LoadModulePtr<IMainFrameModule>("MainFrame");
	const TSharedPtr<SWindow> ParentWindow = MainFrame != nullptr ? MainFrame->GetParentWindow() : nullptr;

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);
	return !OptionsWindow->bUserCancelled;
}

#undef LOCTEXT_NAMESPACE

#endif
