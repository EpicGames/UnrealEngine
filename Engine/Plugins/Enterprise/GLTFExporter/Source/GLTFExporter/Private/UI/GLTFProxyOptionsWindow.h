// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Options/GLTFProxyOptions.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"

class SButton;

/** Options window used to populate provided settings objects */
class SGLTFProxyOptionsWindow : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SGLTFProxyOptionsWindow)
	{}
		SLATE_ARGUMENT( UGLTFProxyOptions*, ProxyOptions )
		SLATE_ARGUMENT( TSharedPtr<SWindow>, WidgetWindow )
	SLATE_END_ARGS()

	SGLTFProxyOptionsWindow();

	void Construct(const FArguments& InArgs);

	FReply OnConfirm();
	FReply OnCancel();

	/* Begin SCompoundWidget overrides */
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	/* End SCompoundWidget overrides */

	static bool ShowDialog(UGLTFProxyOptions* ProxyOptions);

private:

	UGLTFProxyOptions* ProxyOptions;
	TSharedPtr<class IDetailsView> DetailsView;
	TWeakPtr<SWindow> WidgetWindow;
	TSharedPtr<SButton> ConfirmButton;
	bool bUserCancelled;
};

#endif
