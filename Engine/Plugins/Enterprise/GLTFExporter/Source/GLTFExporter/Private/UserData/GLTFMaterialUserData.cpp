// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserData/GLTFMaterialUserData.h"
#include "Materials/MaterialInstance.h"
#include "Engine/Texture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GLTFMaterialUserData)

FGLTFMaterialBakeSize FGLTFMaterialBakeSize::Default = { 1024, 1024, true };

FGLTFOverrideMaterialBakeSettings::FGLTFOverrideMaterialBakeSettings()
	: bOverrideSize(false)
	, Size(FGLTFMaterialBakeSize::Default)
	, bOverrideFilter(false)
	, Filter(TF_Trilinear)
	, bOverrideTiling(false)
	, Tiling(TA_Wrap)
{
}

const UMaterialInterface* UGLTFMaterialExportOptions::ResolveProxy(const UMaterialInterface* Material)
{
	if (Material == nullptr)
	{
		return nullptr;
	}

	TSet<const UMaterialInterface*> PreviousProxies;

	for (;;)
	{
		const UGLTFMaterialExportOptions* UserData = const_cast<UMaterialInterface*>(Material)->GetAssetUserData<UGLTFMaterialExportOptions>();
		if (UserData == nullptr)
		{
			return Material;
		}

		const UMaterialInterface* Proxy = UserData->Proxy;
		if (Proxy == nullptr)
		{
			return Material;
		}

		if (PreviousProxies.Contains(Proxy))
		{
			// report warning about cyclic references
			return Material;
		}

		PreviousProxies.Add(Proxy);
		Material = Proxy;
	}
}

FGLTFMaterialBakeSize UGLTFMaterialExportOptions::GetBakeSizeForPropertyGroup(const UMaterialInterface* Material, EGLTFMaterialPropertyGroup PropertyGroup, FGLTFMaterialBakeSize DefaultValue)
{
	if (const FGLTFOverrideMaterialBakeSettings* BakeSettings = GetBakeSettingsByPredicate(Material, PropertyGroup, [](const FGLTFOverrideMaterialBakeSettings& Settings) { return Settings.bOverrideSize; }))
	{
		return BakeSettings->Size;
	}

	return DefaultValue;
}

TextureFilter UGLTFMaterialExportOptions::GetBakeFilterForPropertyGroup(const UMaterialInterface* Material, EGLTFMaterialPropertyGroup PropertyGroup, TextureFilter DefaultValue)
{
	if (const FGLTFOverrideMaterialBakeSettings* BakeSettings = GetBakeSettingsByPredicate(Material, PropertyGroup, [](const FGLTFOverrideMaterialBakeSettings& Settings) { return Settings.bOverrideFilter; }))
	{
		return BakeSettings->Filter;
	}

	return DefaultValue;
}

TextureAddress UGLTFMaterialExportOptions::GetBakeTilingForPropertyGroup(const UMaterialInterface* Material, EGLTFMaterialPropertyGroup PropertyGroup, TextureAddress DefaultValue)
{
	if (const FGLTFOverrideMaterialBakeSettings* BakeSettings = GetBakeSettingsByPredicate(Material, PropertyGroup, [](const FGLTFOverrideMaterialBakeSettings& Settings) { return Settings.bOverrideTiling; }))
	{
		return BakeSettings->Tiling;
	}

	return DefaultValue;
}

template <typename Predicate>
const FGLTFOverrideMaterialBakeSettings* UGLTFMaterialExportOptions::GetBakeSettingsByPredicate(const UMaterialInterface* Material, EGLTFMaterialPropertyGroup PropertyGroup, Predicate Pred)
{
	while (Material != nullptr)
	{
		if (const UGLTFMaterialExportOptions* UserData = const_cast<UMaterialInterface*>(Material)->GetAssetUserData<UGLTFMaterialExportOptions>())
		{
			if (const FGLTFOverrideMaterialBakeSettings* BakeSettings = UserData->Inputs.Find(PropertyGroup))
			{
				if (Pred(*BakeSettings))
				{
					return BakeSettings;
				}
			}

			if (Pred(UserData->Default))
			{
				return &UserData->Default;
			}
		}

		const UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Material);
		Material = MaterialInstance != nullptr ? MaterialInstance->Parent.Get() : nullptr;
	}

	return nullptr;
}
