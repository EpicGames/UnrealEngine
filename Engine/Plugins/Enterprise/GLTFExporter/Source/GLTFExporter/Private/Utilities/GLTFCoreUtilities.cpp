// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/GLTFCoreUtilities.h"
#include "Converters/GLTFTextureUtilities.h"

float FGLTFCoreUtilities::ConvertLength(const float Length, const float ConversionScale)
{
	return Length * ConversionScale;
}

FGLTFVector3 FGLTFCoreUtilities::ConvertVector(const FVector3f& Vector)
{
	// Unreal uses a left-handed coordinate system, with Z up.
	// glTF uses a right-handed coordinate system, with Y up.
	return { Vector.X, Vector.Z, Vector.Y };
}

FGLTFVector3 FGLTFCoreUtilities::ConvertPosition(const FVector3f& Position, const float ConversionScale)
{
	return ConvertVector(Position * ConversionScale);
}

FGLTFVector3 FGLTFCoreUtilities::ConvertScale(const FVector3f& Scale)
{
	return ConvertVector(Scale);
}

FGLTFVector3 FGLTFCoreUtilities::ConvertNormal(const FVector3f& Normal)
{
	const FVector3f SafeNormal = Normal.GetSafeNormal();
	return ConvertVector(SafeNormal);
}

FGLTFInt8Vector4 FGLTFCoreUtilities::ConvertNormal(const FPackedNormal& Normal)
{
	const FPackedNormal SafeNormal = Normal.ToFVector3f().GetSafeNormal();
	return { SafeNormal.Vector.X, SafeNormal.Vector.Z, SafeNormal.Vector.Y, 0 };
}

FGLTFInt16Vector4 FGLTFCoreUtilities::ConvertNormal(const FPackedRGBA16N& Normal)
{
	const FPackedRGBA16N SafeNormal = Normal.ToFVector3f().GetSafeNormal();
	return { SafeNormal.X, SafeNormal.Z, SafeNormal.Y, 0 };
}

FGLTFVector3 FGLTFCoreUtilities::ConvertNormalDelta(const FVector3f& NormalDelta)
{
	return ConvertVector(NormalDelta);
}

FGLTFInt8Vector4 FGLTFCoreUtilities::ConvertNormalDelta(const FPackedNormal& NormalDelta)
{
	return { NormalDelta.Vector.X, NormalDelta.Vector.Z, NormalDelta.Vector.Y, 0 };
}

FGLTFInt16Vector4 FGLTFCoreUtilities::ConvertNormalDelta(const FPackedRGBA16N& NormalDelta)
{
	return { NormalDelta.X, NormalDelta.Z, NormalDelta.Y, 0 };
}


FGLTFVector4 FGLTFCoreUtilities::ConvertTangent(const FVector3f& Tangent, const FVector4f& Normal)
{
	// Unreal keeps the binormal sign in the normal's w-component.
	// glTF keeps the binormal sign in the tangent's w-component.
	const FVector3f SafeTangent = Tangent.GetSafeNormal();
	return { SafeTangent.X, SafeTangent.Z, SafeTangent.Y, Normal.W >= 0 ? 1.0f : -1.0f };
}

FGLTFVector4 FGLTFCoreUtilities::ConvertTangent(const FVector4f& Tangent)
{
	// Unreal keeps the binormal sign in the normal's w-component.
	// glTF keeps the binormal sign in the tangent's w-component.
	const FVector3f SafeTangent = FVector3f(Tangent).GetSafeNormal();
	return { SafeTangent.X, SafeTangent.Z, SafeTangent.Y, Tangent.W >= 0 ? 1.0f : -1.0f };
}

FGLTFInt8Vector4 FGLTFCoreUtilities::ConvertTangent(const FPackedNormal& Tangent, const FPackedNormal& Normal)
{
	const FPackedNormal SafeTangent = Tangent.ToFVector3f().GetSafeNormal();
	return { SafeTangent.Vector.X, SafeTangent.Vector.Z, SafeTangent.Vector.Y, Normal.Vector.W >= 0 ? MAX_int8 : MIN_int8 };
}

FGLTFInt16Vector4 FGLTFCoreUtilities::ConvertTangent(const FPackedRGBA16N& Tangent, const FPackedRGBA16N& Normal)
{
	const FPackedRGBA16N SafeTangent = Tangent.ToFVector3f().GetSafeNormal();
	return { SafeTangent.X, SafeTangent.Z, SafeTangent.Y, Normal.W >= 0 ? MAX_int16 : MIN_int16 };
}

FGLTFVector2 FGLTFCoreUtilities::ConvertUV(const FVector2f& UV)
{
	// No conversion actually needed, this is primarily for type-safety.
	return { UV.X, UV.Y };
}

FGLTFVector2 FGLTFCoreUtilities::ConvertUV(const FVector2DHalf& UV)
{
	return ConvertUV(FVector2f(UV));
}

FGLTFColor4 FGLTFCoreUtilities::ConvertColor(const FLinearColor& Color)
{
	// glTF requires that color components are in the range 0.0 - 1.0
	return {
		FMath::Clamp(Color.R, 0.0f, 1.0f),
		FMath::Clamp(Color.G, 0.0f, 1.0f),
		FMath::Clamp(Color.B, 0.0f, 1.0f),
		FMath::Clamp(Color.A, 0.0f, 1.0f)
	};
}

FGLTFColor3 FGLTFCoreUtilities::ConvertColor3(const FLinearColor& Color)
{
	// glTF requires that color components are in the range 0.0 - 1.0
	return {
		FMath::Clamp(Color.R, 0.0f, 1.0f),
		FMath::Clamp(Color.G, 0.0f, 1.0f),
		FMath::Clamp(Color.B, 0.0f, 1.0f)
	};
}

FGLTFUInt8Color4 FGLTFCoreUtilities::ConvertColor(const FColor& Color)
{
	// Unreal uses ABGR or ARGB depending on endianness.
	// glTF always uses RGBA independent of endianness.
	return { Color.R, Color.G, Color.B, Color.A };
}

FGLTFQuaternion FGLTFCoreUtilities::ConvertRotation(const FRotator3f& Rotation)
{
	return ConvertRotation(Rotation.Quaternion());
}

FGLTFQuaternion FGLTFCoreUtilities::ConvertRotation(const FQuat4f& Rotation)
{
	// Unreal uses a left-handed coordinate system, with Z up.
	// glTF uses a right-handed coordinate system, with Y up.
	// Rotation = (qX, qY, qZ, qW) = (sin(angle/2) * aX, sin(angle/2) * aY, sin(angle/2) * aZ, cons(angle/2))
	// where (aX, aY, aZ) - rotation axis, angle - rotation angle
	// Y swapped with Z between these coordinate systems
	// also, as handedness is changed rotation is inversed - hence negation
	// therefore glTFRotation = (-qX, -qZ, -qY, qw)

	const FQuat4f Normalized = Rotation.GetNormalized();
	return { -Normalized.X, -Normalized.Z, -Normalized.Y, Normalized.W };
}

FGLTFMatrix4 FGLTFCoreUtilities::ConvertMatrix(const FMatrix44f& Matrix)
{
	// Unreal stores matrix elements in row major order.
	// glTF stores matrix elements in column major order.

	FGLTFMatrix4 Result;
	for (int32 Row = 0; Row < 4; ++Row)
	{
		for (int32 Col = 0; Col < 4; ++Col)
		{
			Result(Row, Col) = Matrix.M[Row][Col];
		}
	}
	return Result;
}

FGLTFMatrix4 FGLTFCoreUtilities::ConvertTransform(const FTransform3f& Transform, const float ConversionScale)
{
	const FQuat4f Rotation = Transform.GetRotation();
	const FVector3f Translation = Transform.GetTranslation();
	const FVector3f Scale = Transform.GetScale3D();

	const FQuat4f ConvertedRotation(-Rotation.X, -Rotation.Z, -Rotation.Y, Rotation.W);
	const FVector3f ConvertedTranslation = FVector3f(Translation.X, Translation.Z, Translation.Y) * ConversionScale;
	const FVector3f ConvertedScale(Scale.X, Scale.Z, Scale.Y);

	const FTransform3f ConvertedTransform(ConvertedRotation, ConvertedTranslation, ConvertedScale);
	const FMatrix44f Matrix = ConvertedTransform.ToMatrixWithScale(); // TODO: should it be ToMatrixNoScale()?
	return ConvertMatrix(Matrix);
}

float FGLTFCoreUtilities::ConvertFieldOfView(float FOVInDegrees, float AspectRatio)
{
	const float HorizontalFOV = FMath::DegreesToRadians(FOVInDegrees);
	const float VerticalFOV = 2 * FMath::Atan(FMath::Tan(HorizontalFOV / 2) / AspectRatio);
	return VerticalFOV;
}

float FGLTFCoreUtilities::ConvertLightAngle(const float Angle)
{
	// Unreal uses degrees.
	// glTF uses radians.
	return FMath::DegreesToRadians(Angle);
}

FGLTFQuaternion FGLTFCoreUtilities::GetLocalCameraRotation()
{
	// Unreal uses +X axis as camera direction in Unreal coordinates.
	// glTF uses +Y as camera direction in Unreal coordinates.

	static FGLTFQuaternion Rotation = ConvertRotation(FRotator3f(0, 90, 0));
	return Rotation;
}

FGLTFQuaternion FGLTFCoreUtilities::GetLocalLightRotation()
{
	// Unreal uses +X axis as light direction in Unreal coordinates.
	// glTF uses +Y as light direction in Unreal coordinates.

	static FGLTFQuaternion Rotation = ConvertRotation(FRotator3f(0, 90, 0));
	return Rotation;
}

EGLTFJsonCameraType FGLTFCoreUtilities::ConvertCameraType(ECameraProjectionMode::Type ProjectionMode)
{
	switch (ProjectionMode)
	{
		case ECameraProjectionMode::Perspective:  return EGLTFJsonCameraType::Perspective;
		case ECameraProjectionMode::Orthographic: return EGLTFJsonCameraType::Orthographic;
		default:                                  return EGLTFJsonCameraType::None;
	}
}

EGLTFJsonLightType FGLTFCoreUtilities::ConvertLightType(ELightComponentType ComponentType)
{
	switch (ComponentType)
	{
		case LightType_Directional: return EGLTFJsonLightType::Directional;
		case LightType_Point:       return EGLTFJsonLightType::Point;
		case LightType_Spot:        return EGLTFJsonLightType::Spot;
		default:                    return EGLTFJsonLightType::None;
	}
}

EGLTFJsonInterpolation FGLTFCoreUtilities::ConvertInterpolation(const EAnimInterpolationType Type)
{
	switch (Type)
	{
		case EAnimInterpolationType::Linear: return EGLTFJsonInterpolation::Linear;
		case EAnimInterpolationType::Step:   return EGLTFJsonInterpolation::Step;
		default:                             return EGLTFJsonInterpolation::None;
	}
}

EGLTFJsonShadingModel FGLTFCoreUtilities::ConvertShadingModel(EMaterialShadingModel ShadingModel)
{
	switch (ShadingModel)
	{
		case MSM_Unlit:                return EGLTFJsonShadingModel::Unlit;
		case MSM_DefaultLit:           return EGLTFJsonShadingModel::Default;
		case MSM_ClearCoat:            return EGLTFJsonShadingModel::ClearCoat;
		case MSM_Cloth:                return EGLTFJsonShadingModel::Sheen;
		case MSM_ThinTranslucent:      return EGLTFJsonShadingModel::Transmission;
		default:                       return EGLTFJsonShadingModel::None;
	}
}

FString FGLTFCoreUtilities::GetShadingModelString(EGLTFJsonShadingModel ShadingModel)
{
	switch (ShadingModel)
	{
		case EGLTFJsonShadingModel::None:               return TEXT("Unknown");
		case EGLTFJsonShadingModel::Default:            return TEXT("Default");
		case EGLTFJsonShadingModel::Unlit:              return TEXT("Unlit");
		case EGLTFJsonShadingModel::ClearCoat:          return TEXT("ClearCoat");
		case EGLTFJsonShadingModel::Sheen:              return TEXT("Cloth");
		case EGLTFJsonShadingModel::Transmission:       return TEXT("ThinTranslucent");
		case EGLTFJsonShadingModel::NumShadingModels:   return TEXT("Unknown");
		default:                                        return TEXT("Unknown");
	}
}

EGLTFJsonAlphaMode FGLTFCoreUtilities::ConvertAlphaMode(EBlendMode Mode)
{
	switch (Mode)
	{
		case BLEND_Opaque:         return EGLTFJsonAlphaMode::Opaque;
		case BLEND_Masked:         return EGLTFJsonAlphaMode::Mask;
		case BLEND_Translucent:    return EGLTFJsonAlphaMode::Blend;
		default:                   return EGLTFJsonAlphaMode::None;
	}
}

EGLTFJsonTextureWrap FGLTFCoreUtilities::ConvertWrap(TextureAddress Address)
{
	switch (Address)
	{
		case TA_Wrap:   return EGLTFJsonTextureWrap::Repeat;
		case TA_Mirror: return EGLTFJsonTextureWrap::MirroredRepeat;
		case TA_Clamp:  return EGLTFJsonTextureWrap::ClampToEdge;
		default:        return EGLTFJsonTextureWrap::None;
	}
}

EGLTFJsonTextureFilter FGLTFCoreUtilities::ConvertMinFilter(TextureFilter Filter)
{
	switch (Filter)
	{
		case TF_Nearest:   return EGLTFJsonTextureFilter::NearestMipmapNearest;
		case TF_Bilinear:  return EGLTFJsonTextureFilter::LinearMipmapNearest;
		case TF_Trilinear: return EGLTFJsonTextureFilter::LinearMipmapLinear;
		default:           return EGLTFJsonTextureFilter::None;
	}
}

EGLTFJsonTextureFilter FGLTFCoreUtilities::ConvertMagFilter(TextureFilter Filter)
{
	switch (Filter)
	{
		case TF_Nearest:   return EGLTFJsonTextureFilter::Nearest;
		case TF_Bilinear:  return EGLTFJsonTextureFilter::Linear;
		case TF_Trilinear: return EGLTFJsonTextureFilter::Linear;
		default:           return EGLTFJsonTextureFilter::None;
	}
}
