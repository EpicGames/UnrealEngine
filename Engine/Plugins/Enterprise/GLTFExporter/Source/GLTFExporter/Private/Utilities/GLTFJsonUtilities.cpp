// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/GLTFJsonUtilities.h"

const TCHAR* FGLTFJsonUtilities::GetValue(EGLTFJsonExtension Enum)
{
	switch (Enum)
	{
		case EGLTFJsonExtension::KHR_LightsPunctual:                  return TEXT("KHR_lights_punctual");
		case EGLTFJsonExtension::EXT_LightsIES:                       return TEXT("EXT_lights_ies");
		case EGLTFJsonExtension::KHR_MaterialsClearCoat:              return TEXT("KHR_materials_clearcoat");
		case EGLTFJsonExtension::KHR_MaterialsEmissiveStrength:       return TEXT("KHR_materials_emissive_strength");
		case EGLTFJsonExtension::KHR_MaterialsUnlit:                  return TEXT("KHR_materials_unlit");
		case EGLTFJsonExtension::KHR_MaterialsVariants:               return TEXT("KHR_materials_variants");
		case EGLTFJsonExtension::KHR_MaterialsIOR:                    return TEXT("KHR_materials_ior");
		case EGLTFJsonExtension::KHR_MaterialsSheen:                  return TEXT("KHR_materials_sheen");
		case EGLTFJsonExtension::KHR_MaterialsTransmission:           return TEXT("KHR_materials_transmission");
		case EGLTFJsonExtension::KHR_MaterialsSpecularGlossiness:     return TEXT("KHR_materials_pbrSpecularGlossiness");
		case EGLTFJsonExtension::KHR_MaterialsIridescence:            return TEXT("KHR_materials_iridescence");
		case EGLTFJsonExtension::KHR_MaterialsAnisotropy:             return TEXT("KHR_materials_anisotropy");
		case EGLTFJsonExtension::KHR_MeshQuantization:                return TEXT("KHR_mesh_quantization");
		case EGLTFJsonExtension::KHR_TextureTransform:                return TEXT("KHR_texture_transform");
		case EGLTFJsonExtension::KHR_MaterialsSpecular:	              return TEXT("KHR_materials_specular");
		case EGLTFJsonExtension::EPIC_LightmapTextures:	              return TEXT("EPIC_lightmap_textures");
		default:
			checkNoEntry();
			return TEXT("");
	}
}

const TCHAR* FGLTFJsonUtilities::GetValue(EGLTFJsonAlphaMode Enum)
{
	switch (Enum)
	{
		case EGLTFJsonAlphaMode::Opaque: return TEXT("OPAQUE");
		case EGLTFJsonAlphaMode::Blend:  return TEXT("BLEND");
		case EGLTFJsonAlphaMode::Mask:   return TEXT("MASK");
		default:
			checkNoEntry();
			return TEXT("");
	}
}

const TCHAR* FGLTFJsonUtilities::GetValue(EGLTFJsonMimeType Enum)
{
	switch (Enum)
	{
		case EGLTFJsonMimeType::PNG:  return TEXT("image/png");
		case EGLTFJsonMimeType::JPEG: return TEXT("image/jpeg");
		default:
			checkNoEntry();
			return TEXT("");
	}
}

const TCHAR* FGLTFJsonUtilities::GetValue(EGLTFJsonAccessorType Enum)
{
	switch (Enum)
	{
		case EGLTFJsonAccessorType::Scalar: return TEXT("SCALAR");
		case EGLTFJsonAccessorType::Vec2:   return TEXT("VEC2");
		case EGLTFJsonAccessorType::Vec3:   return TEXT("VEC3");
		case EGLTFJsonAccessorType::Vec4:   return TEXT("VEC4");
		case EGLTFJsonAccessorType::Mat2:   return TEXT("MAT2");
		case EGLTFJsonAccessorType::Mat3:   return TEXT("MAT3");
		case EGLTFJsonAccessorType::Mat4:   return TEXT("MAT4");
		default:
			checkNoEntry();
			return TEXT("");
	}
}

const TCHAR* FGLTFJsonUtilities::GetValue(EGLTFJsonCameraType Enum)
{
	switch (Enum)
	{
		case EGLTFJsonCameraType::Perspective:  return TEXT("perspective");
		case EGLTFJsonCameraType::Orthographic: return TEXT("orthographic");
		default:
			checkNoEntry();
			return TEXT("");
	}
}

const TCHAR* FGLTFJsonUtilities::GetValue(EGLTFJsonLightType Enum)
{
	switch (Enum)
	{
		case EGLTFJsonLightType::Directional: return TEXT("directional");
		case EGLTFJsonLightType::Point:       return TEXT("point");
		case EGLTFJsonLightType::Spot:        return TEXT("spot");
		default:
			checkNoEntry();
			return TEXT("");
	}
}

const TCHAR* FGLTFJsonUtilities::GetValue(EGLTFJsonInterpolation Enum)
{
	switch (Enum)
	{
		case EGLTFJsonInterpolation::Linear:      return TEXT("LINEAR");
		case EGLTFJsonInterpolation::Step:        return TEXT("STEP");
		case EGLTFJsonInterpolation::CubicSpline: return TEXT("CUBICSPLINE");
		default:
			checkNoEntry();
			return TEXT("");
	}
}

const TCHAR* FGLTFJsonUtilities::GetValue(EGLTFJsonTargetPath Enum)
{
	switch (Enum)
	{
		case EGLTFJsonTargetPath::Translation: return TEXT("translation");
		case EGLTFJsonTargetPath::Rotation:    return TEXT("rotation");
		case EGLTFJsonTargetPath::Scale:       return TEXT("scale");
		case EGLTFJsonTargetPath::Weights:     return TEXT("weights");
		default:
			checkNoEntry();
			return TEXT("");
	}
}

const TCHAR* FGLTFJsonUtilities::GetValue(EGLTFJsonShadingModel Enum)
{
	switch (Enum)
	{
		case EGLTFJsonShadingModel::Default:                return TEXT("Default");
		case EGLTFJsonShadingModel::Unlit:                  return TEXT("Unlit");
		case EGLTFJsonShadingModel::ClearCoat:              return TEXT("ClearCoat");
		case EGLTFJsonShadingModel::Sheen:                  return TEXT("Sheen");
		case EGLTFJsonShadingModel::Transmission:           return TEXT("Transmission");
		case EGLTFJsonShadingModel::SpecularGlossiness:     return TEXT("SpecularGlossiness");
		default:
			checkNoEntry();
			return TEXT("");
	}
}
