// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/GLTFLandscapeComponentDataInterface.h"

#include "LandscapeComponent.h"
#include "LandscapeDataAccess.h"
#include "LandscapeLayerInfoObject.h"
#include "TextureResource.h"

FGLTFLandscapeComponentDataInterface::FGLTFLandscapeComponentDataInterface(const ULandscapeComponent& Component, int32 ExportLOD)
	: Component(Component)
	, MipLevel(ExportLOD)
{

	UTexture2D* HeightMapTexture = Component.GetHeightmap(true);
#if WITH_EDITOR
	if (HeightMapTexture)
	{
		if (MipLevel < HeightMapTexture->Source.GetNumMips())
		{
			HeightMapTexture->Source.GetMipData(HeightMipDataSource, MipLevel);
			HeightMipData = (FColor*)HeightMipDataSource.GetData();
		}
	}
#else
	if (HeightMapTexture)
	{
		FTexturePlatformData* PlatformData = HeightMapTexture->GetPlatformData();
		if (MipLevel < PlatformData->Mips.Num())
		{
			void* MipData = nullptr;
			PlatformData->Mips[MipLevel].BulkData.GetCopy(&MipData, false);
			HeightMipData = (FColor*)MipData;
		}
	}
#endif
	ComponentSubsectionSizeQuads = Component.SubsectionSizeQuads;

	ComponentSizeQuads = Component.ComponentSizeQuads;
	ComponentSizeVerts = (Component.ComponentSizeQuads + 1) >> MipLevel;
	ComponentSubsectionSizeVerts = (ComponentSubsectionSizeQuads + 1) >> MipLevel;
	ComponentNumSubsections = Component.NumSubsections;

	if (HeightMapTexture)
	{
		HeightmapStride = HeightMapTexture->GetSizeX() >> MipLevel;
		HeightmapComponentOffsetX = FMath::RoundToInt((HeightMapTexture->GetSizeX() >> MipLevel) * Component.HeightmapScaleBias.Z);
		HeightmapComponentOffsetY = FMath::RoundToInt((HeightMapTexture->GetSizeY() >> MipLevel) * Component.HeightmapScaleBias.W);
	}

	HeightmapSubsectionOffset = (ComponentSubsectionSizeQuads + 1) >> MipLevel;
}

void FGLTFLandscapeComponentDataInterface::ComponentXYToSubsectionXY(int32 CompX, int32 CompY, int32& SubNumX, int32& SubNumY, int32& SubX, int32& SubY) const
{
	// We do the calculation as if we're looking for the previous vertex.
	// This allows us to pick up the last shared vertex of every subsection correctly.
	SubNumX = (CompX - 1) / (ComponentSubsectionSizeVerts - 1);
	SubNumY = (CompY - 1) / (ComponentSubsectionSizeVerts - 1);
	SubX = (CompX - 1) % (ComponentSubsectionSizeVerts - 1) + 1;
	SubY = (CompY - 1) % (ComponentSubsectionSizeVerts - 1) + 1;

	// If we're asking for the first vertex, the calculation above will lead
	// to a negative SubNumX/Y, so we need to fix that case up.
	if (SubNumX < 0)
	{
		SubNumX = 0;
		SubX = 0;
	}

	if (SubNumY < 0)
	{
		SubNumY = 0;
		SubY = 0;
	}
}

int32 FGLTFLandscapeComponentDataInterface::TexelXYToIndex(int32 TexelX, int32 TexelY) const
{
	return TexelY * ComponentNumSubsections * ComponentSubsectionSizeVerts + TexelX;
}

void FGLTFLandscapeComponentDataInterface::VertexXYToTexelXY(int32 VertX, int32 VertY, int32& OutX, int32& OutY) const
{
	int32 SubNumX, SubNumY, SubX, SubY;
	ComponentXYToSubsectionXY(VertX, VertY, SubNumX, SubNumY, SubX, SubY);

	OutX = SubNumX * ComponentSubsectionSizeVerts + SubX;
	OutY = SubNumY * ComponentSubsectionSizeVerts + SubY;
}

void FGLTFLandscapeComponentDataInterface::VertexIndexToXY(int32 VertexIndex, int32& OutX, int32& OutY) const
{
	OutX = VertexIndex % ComponentSizeVerts;
	OutY = VertexIndex / ComponentSizeVerts;
}

int32 FGLTFLandscapeComponentDataInterface::VertexIndexToTexel(int32 VertexIndex) const
{
	int32 VertX, VertY;
	VertexIndexToXY(VertexIndex, VertX, VertY);
	int32 TexelX, TexelY;
	VertexXYToTexelXY(VertX, VertY, TexelX, TexelY);
	return TexelXYToIndex(TexelX, TexelY);
}

bool FGLTFLandscapeComponentDataInterface::GetWeightmapTextureData(ULandscapeLayerInfoObject* InLayerInfo,
	TArray<uint8>& OutData,
	bool bInUseEditingWeightmap, bool bInRemoveSubsectionDuplicates)
{
	int32 LayerIdx = INDEX_NONE;
	const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = Component.GetWeightmapLayerAllocations(bInUseEditingWeightmap);
	const TArray<UTexture2D*>& ComponentWeightmapTextures = Component.GetWeightmapTextures(bInUseEditingWeightmap);

	for (int32 Idx = 0; Idx < ComponentWeightmapLayerAllocations.Num(); Idx++)
	{
		if (ComponentWeightmapLayerAllocations[Idx].LayerInfo == InLayerInfo)
		{
			LayerIdx = Idx;
			break;
		}
	}
	if (LayerIdx < 0)
	{
		return false;
	}
	if (ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex >= ComponentWeightmapTextures.Num())
	{
		return false;
	}
	if (ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel >= 4)
	{
		return false;
	}

	UTexture2D* WeightMapTexture = ComponentWeightmapTextures[ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex];

	if (!WeightMapTexture)
	{
		return false;
	}

	// If requested to skip the duplicate row/col of texture data
	int32 WeightmapSize = bInRemoveSubsectionDuplicates ?
		((ComponentSubsectionSizeQuads * Component.NumSubsections) + 1) >> MipLevel :
		((ComponentSubsectionSizeQuads + 1) * Component.NumSubsections) >> MipLevel;

	OutData.Empty(FMath::Square(WeightmapSize));
	OutData.AddUninitialized(FMath::Square(WeightmapSize));
	
#if WITH_EDITOR
	if (MipLevel >= WeightMapTexture->Source.GetNumMips())
	{
		return false;
	}
	TArray64<uint8> MipData;
	WeightMapTexture->Source.GetMipData(MipData, MipLevel);
	const FColor* WeightMipData = (FColor*)MipData.GetData();
#else
	FTexturePlatformData* PlatformData = WeightMapTexture->GetPlatformData();
	if (MipLevel >= PlatformData->Mips.Num())
	{
		return false;
	}
	void* MipData = nullptr;
	PlatformData->Mips[MipLevel].BulkData.GetCopy(&MipData, false);
	const FColor* WeightMipData = (FColor*)MipData;
#endif

	if (!WeightMipData)
	{
		return false;
	}

	// Channel remapping
	int32 ChannelOffsets[4] = { (int32)STRUCT_OFFSET(FColor, R), (int32)STRUCT_OFFSET(FColor, G), (int32)STRUCT_OFFSET(FColor, B), (int32)STRUCT_OFFSET(FColor, A) };

	const uint8* SrcTextureData = (const uint8*)WeightMipData + ChannelOffsets[ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel];

	for (int32 i = 0; i < FMath::Square(WeightmapSize); i++)
	{
		// If removing subsection duplicates, convert vertex to texel index
		OutData[i] = bInRemoveSubsectionDuplicates ? SrcTextureData[VertexIndexToTexel(i) * sizeof(FColor)] : SrcTextureData[i * sizeof(FColor)];
	}

	return true;
}

FColor FGLTFLandscapeComponentDataInterface::GetHeightData(int32 LocalX, int32 LocalY) const
{
	if (HeightMipData == nullptr)
	{
		return FColor(0, 0, 0, 0);
	}

	int32 TexelX, TexelY;
	VertexXYToTexelXY(LocalX, LocalY, TexelX, TexelY);

	return HeightMipData[TexelX + HeightmapComponentOffsetX + (TexelY + HeightmapComponentOffsetY) * HeightmapStride];
}

// Deprecated
FColor FGLTFLandscapeComponentDataInterface::GetXYOffsetData(int32 LocalX, int32 LocalY) const
{
	return FColor();
}

// Deprecated
void FGLTFLandscapeComponentDataInterface::GetXYOffset(int32 X, int32 Y, float& XOffset, float& YOffset) const
{
}

void FGLTFLandscapeComponentDataInterface::GetPositionNormalUV(int32 LocalX, int32 LocalY,
	FVector3f& Position,
	FVector3f& Normal,
	FVector2f& UV) const
{
	{
		FColor Data = GetHeightData(LocalX, LocalY);
		FVector TangentZ = LandscapeDataAccess::UnpackNormal(Data);
		FVector TangentX = FVector(TangentZ.Z, 0.f, -TangentZ.X);
		FVector TangentY = TangentZ ^ TangentX;

		float Height = LandscapeDataAccess::UnpackHeight(Data);

		const float ScaleFactor = (float)ComponentSizeQuads / (float)(ComponentSizeVerts - 1);
		Position = FVector3f(LocalX * ScaleFactor, LocalY * ScaleFactor, Height);

		Normal = FVector3f(TangentZ.X, TangentZ.Y, TangentZ.Z);
		UV = FVector2f(Component.GetSectionBase().X + LocalX * ScaleFactor, Component.GetSectionBase().Y + LocalY * ScaleFactor);
	}
}