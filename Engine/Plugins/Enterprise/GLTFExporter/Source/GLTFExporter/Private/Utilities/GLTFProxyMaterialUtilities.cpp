// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/GLTFProxyMaterialUtilities.h"
#include "Materials/Material.h"
#include "Utilities/GLTFJsonUtilities.h"
#include "UserData/GLTFMaterialUserData.h"
#include "Materials/MaterialInstanceConstant.h"

#define PROXY_MATERIAL_NAME_PREFIX TEXT("M_GLTF_")
#define PROXY_MATERIAL_ROOT_PATH   TEXT("/GLTFExporter/Materials/Proxy/")

#if WITH_EDITOR
UMaterialInstanceConstant* FGLTFProxyMaterialUtilities::CreateProxyMaterial(EGLTFJsonShadingModel ShadingModel,
	UObject* Outer, FName Name, EObjectFlags Flags)
{
	UMaterialInstanceConstant* MaterialInstance = NewObject<UMaterialInstanceConstant>(Outer, Name, Flags);
	MaterialInstance->Parent = GetBaseMaterial(ShadingModel);
	return MaterialInstance;
}
#endif

bool FGLTFProxyMaterialUtilities::IsProxyMaterial(const UMaterial* Material)
{
	return Material != nullptr && Material->GetPathName().StartsWith(PROXY_MATERIAL_ROOT_PATH PROXY_MATERIAL_NAME_PREFIX);
}

bool FGLTFProxyMaterialUtilities::IsProxyMaterial(const UMaterialInterface* Material)
{
	return Material != nullptr && IsProxyMaterial(Material->GetMaterial());
}

UMaterial* FGLTFProxyMaterialUtilities::GetBaseMaterial(EGLTFJsonShadingModel ShadingModel)
{
	constexpr int32 NumMaterials = static_cast<int32>(EGLTFJsonShadingModel::NumShadingModels);
	static UMaterial* Materials[NumMaterials] = { nullptr };

	const int32 Index = static_cast<int32>(ShadingModel);
	if (Index < 0 || NumMaterials <= Index)
	{
		checkNoEntry();
		return nullptr;
	}

	UMaterial* Material = Materials[Index];
	if (Material == nullptr)
	{
		const FString Name = FGLTFJsonUtilities::GetValue(ShadingModel);
		const FString Path = PROXY_MATERIAL_ROOT_PATH PROXY_MATERIAL_NAME_PREFIX + Name + TEXT(".") PROXY_MATERIAL_NAME_PREFIX + Name;

		Material = LoadObject<UMaterial>(nullptr, *Path);
		Materials[Index] = Material;
	}

	return Material;
}

UMaterialInterface* FGLTFProxyMaterialUtilities::GetProxyMaterial(UMaterialInterface* OriginalMaterial)
{
	UGLTFMaterialExportOptions* UserData = OriginalMaterial->GetAssetUserData<UGLTFMaterialExportOptions>();
	return UserData != nullptr ? UserData->Proxy : nullptr;
}

void FGLTFProxyMaterialUtilities::SetProxyMaterial(UMaterialInterface* OriginalMaterial, UMaterialInterface* ProxyMaterial)
{
	UGLTFMaterialExportOptions* UserData = OriginalMaterial->GetAssetUserData<UGLTFMaterialExportOptions>();
	if (UserData == nullptr)
	{
		UserData = NewObject<UGLTFMaterialExportOptions>(OriginalMaterial, NAME_None, RF_Public | RF_Transactional);
		OriginalMaterial->AddAssetUserData(UserData);
	}

	UserData->Proxy = ProxyMaterial;
	OriginalMaterial->Modify();
}

bool FGLTFProxyMaterialUtilities::GetParameterValue(const UMaterialInterface* Material, const FHashedMaterialParameterInfo& ParameterInfo, float& OutValue, bool NonDefaultOnly)
{
	float Value;
	if (!Material->GetScalarParameterValue(ParameterInfo, Value, NonDefaultOnly))
	{
		return false;
	}

	if (NonDefaultOnly)
	{
		float DefaultValue;
		if (!Material->GetScalarParameterDefaultValue(ParameterInfo, DefaultValue) || DefaultValue == Value)
		{
			return false;
		}
	}

	OutValue = Value;
	return true;
}

bool FGLTFProxyMaterialUtilities::GetParameterValue(const UMaterialInterface* Material, const FHashedMaterialParameterInfo& ParameterInfo, FLinearColor& OutValue, bool NonDefaultOnly)
{
	FLinearColor Value;
	if (!Material->GetVectorParameterValue(ParameterInfo, Value, NonDefaultOnly))
	{
		return false;
	}

	if (NonDefaultOnly)
	{
		FLinearColor DefaultValue;
		if (!Material->GetVectorParameterDefaultValue(ParameterInfo, DefaultValue) || DefaultValue == Value)
		{
			return false;
		}
	}

	OutValue = Value;
	return true;
}

bool FGLTFProxyMaterialUtilities::GetParameterValue(const UMaterialInterface* Material, const FHashedMaterialParameterInfo& ParameterInfo, UTexture*& OutValue, bool NonDefaultOnly)
{
	UTexture* Value;
	if (!Material->GetTextureParameterValue(ParameterInfo, Value, NonDefaultOnly))
	{
		return false;
	}

	if (NonDefaultOnly)
	{
		UTexture* DefaultValue;
		if (!Material->GetTextureParameterDefaultValue(ParameterInfo, DefaultValue) || DefaultValue == Value)
		{
			return false;
		}
	}

	OutValue = Value;
	return true;
}

#if WITH_EDITOR

void FGLTFProxyMaterialUtilities::SetParameterValue(UMaterialInstanceConstant* Material, const FHashedMaterialParameterInfo& ParameterInfo, float Value, bool NonDefaultOnly)
{
	if (NonDefaultOnly)
	{
		float DefaultValue;
		if (!Material->GetScalarParameterDefaultValue(ParameterInfo, DefaultValue) || DefaultValue == Value)
		{
			return;
		}
	}

	Material->SetScalarParameterValueEditorOnly(FMaterialParameterInfo(ParameterInfo), Value);
}

void FGLTFProxyMaterialUtilities::SetParameterValue(UMaterialInstanceConstant* Material, const FHashedMaterialParameterInfo& ParameterInfo, const FLinearColor& Value, bool NonDefaultOnly)
{
	if (NonDefaultOnly)
	{
		FLinearColor DefaultValue;
		if (!Material->GetVectorParameterDefaultValue(ParameterInfo, DefaultValue) || DefaultValue == Value)
		{
			return;
		}
	}

	Material->SetVectorParameterValueEditorOnly(FMaterialParameterInfo(ParameterInfo), Value);
}

void FGLTFProxyMaterialUtilities::SetParameterValue(UMaterialInstanceConstant* Material, const FHashedMaterialParameterInfo& ParameterInfo, UTexture* Value, bool NonDefaultOnly)
{
	if (NonDefaultOnly)
	{
		UTexture* DefaultValue;
		if (!Material->GetTextureParameterDefaultValue(ParameterInfo, DefaultValue) || DefaultValue == Value)
		{
			return;
		}
	}

	Material->SetTextureParameterValueEditorOnly(FMaterialParameterInfo(ParameterInfo), Value);
}

#endif

bool FGLTFProxyMaterialUtilities::GetTwoSided(const UMaterialInterface* Material, bool& OutValue, bool NonDefaultOnly)
{
	const bool DefaultValue = Material->GetMaterial()->IsTwoSided();
	const bool Value = Material->IsTwoSided();

	if (NonDefaultOnly && Value == DefaultValue)
	{
		return false;
	}

	OutValue = Value;
	return true;
}

bool FGLTFProxyMaterialUtilities::GetIsThinSurface(const UMaterialInterface* Material, bool& OutValue, bool NonDefaultOnly)
{
	const bool DefaultValue = Material->GetMaterial()->IsThinSurface();
	const bool Value = Material->IsThinSurface();

	if (NonDefaultOnly && Value == DefaultValue)
	{
		return false;
	}

	OutValue = Value;
	return true;
}

bool FGLTFProxyMaterialUtilities::GetBlendMode(const UMaterialInterface* Material, EBlendMode& OutValue, bool NonDefaultOnly)
{
	const EBlendMode DefaultValue = Material->GetMaterial()->GetBlendMode();
	const EBlendMode Value = Material->GetBlendMode();

	if (NonDefaultOnly && Value == DefaultValue)
	{
		return false;
	}

	OutValue = Value;
	return true;
}

bool FGLTFProxyMaterialUtilities::GetOpacityMaskClipValue(const UMaterialInterface* Material, float& OutValue, bool NonDefaultOnly)
{
	const float DefaultValue = Material->GetMaterial()->GetOpacityMaskClipValue();
	const float Value = Material->GetOpacityMaskClipValue();

	if (NonDefaultOnly && Value == DefaultValue)
	{
		return false;
	}

	OutValue = Value;
	return true;
}

#if WITH_EDITOR
void FGLTFProxyMaterialUtilities::SetTwoSided(UMaterialInstanceConstant* Material, bool Value, bool NonDefaultOnly)
{
	if (NonDefaultOnly)
	{
		const bool DefaultValue = Material->Parent->IsTwoSided();
		if (DefaultValue == Value)
		{
			return;
		}
	}

	Material->BasePropertyOverrides.bOverride_TwoSided = true;
	Material->BasePropertyOverrides.TwoSided = Value;
	Material->TwoSided = Value;
}

void FGLTFProxyMaterialUtilities::SetIsThinSurface(UMaterialInstanceConstant* Material, bool Value, bool NonDefaultOnly)
{
	if (NonDefaultOnly)
	{
		const bool DefaultValue = Material->Parent->IsThinSurface();
		if (DefaultValue == Value)
		{
			return;
		}
	}

	Material->BasePropertyOverrides.bOverride_bIsThinSurface = true;
	Material->BasePropertyOverrides.bIsThinSurface = Value;
	Material->bIsThinSurface = Value;
}

void FGLTFProxyMaterialUtilities::SetBlendMode(UMaterialInstanceConstant* Material, EBlendMode Value, bool NonDefaultOnly)
{
	if (NonDefaultOnly)
	{
		const EBlendMode DefaultValue = Material->Parent->GetBlendMode();
		if (DefaultValue == Value)
		{
			return;
		}
	}

	Material->BasePropertyOverrides.bOverride_BlendMode = true;
	Material->BasePropertyOverrides.BlendMode = Value;
	Material->BlendMode = Value;
}

void FGLTFProxyMaterialUtilities::SetOpacityMaskClipValue(UMaterialInstanceConstant* Material, float Value, bool NonDefaultOnly)
{
	if (NonDefaultOnly)
	{
		const float DefaultValue = Material->Parent->GetOpacityMaskClipValue();
		if (DefaultValue == Value)
		{
			return;
		}
	}

	Material->BasePropertyOverrides.bOverride_OpacityMaskClipValue = true;
	Material->BasePropertyOverrides.OpacityMaskClipValue = Value;
	Material->OpacityMaskClipValue = Value;
}
#endif
