// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/MeshParsingUtilities.h"

#if WITH_EDITORONLY_DATA

#include "Components/SplineMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "SkeletalMeshAttributes.h"
#include "Utilities/MeshAttributesArray.h"

namespace UE::MeshParser
{
	namespace Private
	{
		template <class T>
		bool Equals(const T& A, const T& B)
		{
			return A.Equals(B, UE_SMALL_NUMBER);
		}

		bool Equals(const FColor& A, const FColor& B)
		{
			return A == B;
		}

		template <class T>
		bool CheckCompareSetValue(bool& IsSet, T& StoredValue, const T& ToSetValue)
		{
			if (IsSet)
			{
				return Equals(StoredValue, ToSetValue);
			}

			IsSet = true;
			StoredValue = ToSetValue;

			return true;
		}

		template <class T>
		bool DoesBufferHasZeroVector(TArray<T> Buffer, float Tolerance = UE_KINDA_SMALL_NUMBER)
		{
			for (const T& Value : Buffer)
			{
				if (FMath::Abs(Value.X) <= Tolerance
					&& FMath::Abs(Value.Y) <= Tolerance
					&& FMath::Abs(Value.Z) <= Tolerance)
				{
					return true;
				}
			}
			return false;
		}

		struct FVertexIdTracker
		{
			int32 ExportedVertexId;
			FVertexID VertexId; //original
			FVertexInstanceID VertexInstanceId; //original

			FVertexIdTracker(const int32& InExportedVertexId, const FVertexID& InVertexId, const FVertexInstanceID& InVertexInstanceId)
				: ExportedVertexId(InExportedVertexId)
				, VertexId(InVertexId)
				, VertexInstanceId(InVertexInstanceId)
			{
			}
		};
	}

	//
	//FMeshPrimitiveDescription
	//

	bool FMeshPrimitiveDescription::IsEmpty()
	{
		return !(Indices.Num() && Positions.Num());
	}

	//Except for Joint related containers
	void FMeshPrimitiveDescription::PrepareContainers(const int32& IndexCount, const int32& AttributesCount, const int32& UVCount, bool bPrepareVertexColors, const int32& TargetCount)
	{
		Indices.Reserve(IndexCount);
		Positions.SetNumZeroed(AttributesCount);
		if (bPrepareVertexColors) VertexColors.SetNumZeroed(AttributesCount);
		Normals.SetNumZeroed(AttributesCount);
		Tangents.SetNumZeroed(AttributesCount);
		UVs.AddDefaulted(UVCount);
		for (size_t UVIndex = 0; UVIndex < UVCount; UVIndex++)
		{
			UVs[UVIndex].SetNumZeroed(AttributesCount);
		}

		Targets.SetNum(TargetCount);
		for (size_t TargetCounter = 0; TargetCounter < TargetCount; TargetCounter++)
		{
			FPrimitiveTargetDescription& TargetDescription = Targets[TargetCounter];
			TargetDescription.Positions.SetNumZeroed(AttributesCount);
			TargetDescription.Normals.SetNumZeroed(AttributesCount);
		}
	}

	void FMeshPrimitiveDescription::PrepareJointContainers(const uint32& JointGroupCount, const int32& AttributesCount)
	{
		JointInfluences.AddDefaulted(JointGroupCount);
		JointWeights.AddDefaulted(JointGroupCount);
		for (size_t GroupIndex = 0; GroupIndex < JointGroupCount; GroupIndex++)
		{
			JointInfluences[GroupIndex].SetNumZeroed(AttributesCount);
			JointWeights[GroupIndex].SetNumZeroed(AttributesCount);
		}
	}

	FExportConfigs::FExportConfigs(bool bInExportVertexSkinWeights,
		bool bInExportVertexColors,
		const USplineMeshComponent* InSplineMeshComponent,
		int32 InSkeletonInfluenceCountPerGroup,
		bool bInExportMorphTargets)
		: bExportVertexSkinWeights(bInExportVertexSkinWeights)
		, bExportVertexColors(bInExportVertexColors)
		, SplineMeshComponent(InSplineMeshComponent)
		, SkeletonInfluenceCountPerGroup(InSkeletonInfluenceCountPerGroup)
		, bExportMorphTargets(bInExportMorphTargets)
	{
	}


	//
	//FMeshDescriptionParser
	//
	template <class T>
	FMeshDescriptionParser<T>::FMeshDescriptionParser(const FMeshDescription* InMeshDescription, const TArray<T>& InMaterialSlots)
		: MeshDescription(InMeshDescription)
		, MaterialSlots(InMaterialSlots)
	{
		FStaticMeshConstAttributes MeshAttributes(*MeshDescription);

		//Set MeshAttrbiutes to be used later:
		VertexPositions = MeshAttributes.GetVertexPositions();
		VertexInstanceNormals = MeshAttributes.GetVertexInstanceNormals();
		VertexInstanceTangents = MeshAttributes.GetVertexInstanceTangents();
		VertexInstanceBinormalSigns = MeshAttributes.GetVertexInstanceBinormalSigns();
		VertexInstanceUVs = MeshAttributes.GetVertexInstanceUVs();
		VertexInstanceColors = MeshAttributes.GetVertexInstanceColors();

		VertexInstanceIdToVertexId = MeshAttributes.GetVertexInstanceVertexIndices().GetRawArray();
		PolygonGroupMaterialSlotNames = MeshAttributes.GetPolygonGroupMaterialSlotNames();

		//MorphTargets:
		if constexpr (std::is_same_v<T, FSkeletalMaterial>)
		{
			FSkeletalMeshConstAttributes SkeletalMeshAttributes(*MeshDescription);
			TArray<FName> MorphTargetNames = SkeletalMeshAttributes.GetMorphTargetNames();

			MorphTargetNames.Sort([](const FName& Left, const FName Right)
				{
					return Left.ToString() < Right.ToString();
				});

			TargetVertexPositionDeltas.Reserve(MorphTargetNames.Num());
			TargetVertexInstanceNormalDeltas.Reserve(MorphTargetNames.Num());
			TargetNames.Reserve(MorphTargetNames.Num());

			for (const FName& MorphTargetName : MorphTargetNames)
			{
				if (SkeletalMeshAttributes.HasMorphTargetPositionsAttribute(MorphTargetName) || SkeletalMeshAttributes.HasMorphTargetNormalsAttribute(MorphTargetName))
				{
					TargetNames.Add(MorphTargetName.ToString());
				}
				else
				{
					continue;
				}

				if (SkeletalMeshAttributes.HasMorphTargetPositionsAttribute(MorphTargetName))
				{
					TargetVertexPositionDeltas.Add(SkeletalMeshAttributes.GetVertexMorphPositionDelta(MorphTargetName));
				}
				else
				{
					TargetVertexPositionDeltas.AddDefaulted();
				}

				if (SkeletalMeshAttributes.HasMorphTargetNormalsAttribute(MorphTargetName))
				{
					TargetVertexInstanceNormalDeltas.Add(SkeletalMeshAttributes.GetVertexInstanceMorphNormalDelta(MorphTargetName));
				}
				else
				{
					TargetVertexInstanceNormalDeltas.AddDefaulted();
				}
			}

			NumOfTargets = TargetNames.Num();
		}
		else
		{
			NumOfTargets = 0;
		}
		
		//Mesh level details that are used for the parsing
		MeshDetails.UVCount = VertexInstanceUVs.GetNumChannels();
		MeshDetails.NumberOfPrimitives = MeshDescription->PolygonGroups().Num();
		MeshDetails.bHasVertexColors = MeshDescription->VertexInstanceAttributes().HasAttribute(MeshAttribute::VertexInstance::Color);
	}

	template <class T>
	void FMeshDescriptionParser<T>::Parse(TArray<FMeshPrimitiveDescription>& OutMeshPrimitiveDescriptions, TArray<FString>& OutTargetNames, const FExportConfigs& ExportConfigs)
	{
		using namespace Private;

		OutTargetNames = TargetNames;

		//Iterate through all PolygonsGroups:
		for (const FPolygonGroupID& PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
		{
			//Acquire the TriangleIDs for the current PolygonGroup:
			const TArrayView<const FTriangleID>& TriangleIDs = MeshDescription->GetPolygonGroupTriangles(PolygonGroupID);

			//We add the MeshPrimitiveDescription here, even if it is empty, in order to keep the same number of MeshPrimitiveDescriptions as the PolygonGroups.
			FMeshPrimitiveDescription& MeshPrimitiveDescription = OutMeshPrimitiveDescriptions.AddDefaulted_GetRef();

			if (TriangleIDs.Num() == 0)
			{
				//Do not export empty primitives.
				continue;
			}

			//VertexIDs of the TriangleIDs can be part of a bigger/unified container
			//For exporting purposes we want to have the Primitives to have their own containers for Attributes+Indices.
			//Which means we have still remap the Vertices to their own containers per Primitives.

			if (!ParseVertexBased(TriangleIDs, MeshPrimitiveDescription, ExportConfigs))
			{
				//To Emtpy all attribute storages we re-initialize the FMeshPrimitiveDescription.
				MeshPrimitiveDescription = FMeshPrimitiveDescription();

				ParseVertexInstanceBased(TriangleIDs, MeshPrimitiveDescription, ExportConfigs);
			}

			//Material Index has to be set here as MeshPrimitiveDescription can be re-initialized if ParseVertexBased fails.
			MeshPrimitiveDescription.MaterialIndex = GetMaterialIndex(PolygonGroupID);

			if (MeshPrimitiveDescription.Tangents.Num() > 0 && DoesBufferHasZeroVector(MeshPrimitiveDescription.Tangents))
			{
				//Do not Export Tangents list that is zeroed out.
				MeshPrimitiveDescription.Tangents.Reset();
			}

			if (MeshPrimitiveDescription.Normals.Num() > 0 && DoesBufferHasZeroVector(MeshPrimitiveDescription.Normals))
			{
				//Do not Export Normals list that is zeroed out.
				MeshPrimitiveDescription.Normals.Reset();
			}
		}
	}

	template <class T>
	bool FMeshDescriptionParser<T>::ParseVertexBased(const TArrayView<const FTriangleID>& TriangleIDs, FMeshPrimitiveDescription& MeshPrimitiveDescription, const FExportConfigs& ExportConfigs)
	{
		using namespace Private;

		const int32 TrianglesCount = TriangleIDs.Num();
		const int32 VertexInstanceCount = TrianglesCount * 3; //Max number of VertexInstances used in the PolygonGroup aka the Primitive.

		TSet<FVertexID> UniqueVertexIDs;
		UniqueVertexIDs.Reserve(VertexInstanceCount);
		for (const FTriangleID& TriangleID : TriangleIDs)
		{
			TArrayView<const FVertexInstanceID> TriangleVertexInstanceIDs = MeshDescription->GetTriangleVertexInstances(TriangleID);
			for (const FVertexInstanceID& VertexInstanceID : TriangleVertexInstanceIDs)
			{
				FVertexID VertexID = VertexInstanceIdToVertexId[VertexInstanceID];
				UniqueVertexIDs.Add(VertexID);
			}
		}
		const int32 VertexCount = UniqueVertexIDs.Num();
		TMap<int32, int32> OriginalToExportedVertexIdMap;

		auto GetOriginalAndExportedVertexIDs = [&OriginalToExportedVertexIdMap](TConstArrayView<FVertexID>& LocalVertexInstanceIdToVertexId, const FVertexID& VertexInstanceID, FVertexID& VertexID, int32& ExportedVertexID)
			{
				if (LocalVertexInstanceIdToVertexId.Num() < VertexInstanceID)
				{
					return false;
				}
				VertexID = LocalVertexInstanceIdToVertexId[VertexInstanceID];
				int32* ExportedVertexIDPtr = OriginalToExportedVertexIdMap.Find(VertexID);
				if (!ExportedVertexIDPtr)
				{
					return false;
				}
				ExportedVertexID = *ExportedVertexIDPtr;

				return true;
			};


		//This is an attribute set track which we use for identifying if an Attribute[VertexID] has been set already or not.
		//	Which is needed in order to identify if all VertexInstances of a given VertexID uses the same/identical/equal Values per instance.
		//	- In case all VertexInstances of given VertexIDs use the same Value we can use the packed version for export.
		//	- In case that's not true, we fall back onto VertexInstance exports instead.
		struct FVertexAttributesSetTracker
		{
			TArray<bool> VertexColors;
			TArray<bool> Normals;
			TArray<bool> Tangents;
			TArray<TArray<bool>> UVs;
			TArray<TArray<bool>> TargetNormals;

			FVertexAttributesSetTracker(int32 VertexCount, int32 UVCount, int32 TargetCount)
			{
				VertexColors.SetNumZeroed(VertexCount);
				Normals.SetNumZeroed(VertexCount);
				Tangents.SetNumZeroed(VertexCount);
				for (size_t UVIndex = 0; UVIndex < UVCount; UVIndex++)
				{
					UVs.AddDefaulted_GetRef().SetNumZeroed(VertexCount);
				}
				for (size_t TargetCounter = 0; TargetCounter < TargetCount; TargetCounter++)
				{
					TargetNormals.AddDefaulted_GetRef().SetNumZeroed(VertexCount);
				}
			}
		};
		FVertexAttributesSetTracker AttributesSetTracker(VertexCount, MeshDetails.UVCount, NumOfTargets);

		const bool bCanExportVertexColors = MeshDetails.bHasVertexColors && ExportConfigs.bExportVertexColors;

		//Set and Reserve the appropriate counts for the Attributes (and Indices)
		//SetNum/Reserve the containers:
		MeshPrimitiveDescription.PrepareContainers(VertexInstanceCount, VertexCount, MeshDetails.UVCount, bCanExportVertexColors, NumOfTargets);

		TArray<FVertexIdTracker> VertexIdTrackers; //Exported to Original(VertexId, VertexInstanceId)
		VertexIdTrackers.Reserve(TriangleIDs.Num() * 3);

		for (const FTriangleID& TriangleID : TriangleIDs)
		{
			TArrayView<const FVertexInstanceID> TriangleVertexInstanceIDs = MeshDescription->GetTriangleVertexInstances(TriangleID);
			for (const FVertexInstanceID& VertexInstanceID : TriangleVertexInstanceIDs)
			{
				FVertexID OriginalVertexID = VertexInstanceIdToVertexId[VertexInstanceID];
				int32* ExportedVertexIDPtr = OriginalToExportedVertexIdMap.Find(OriginalVertexID);
				int32 ExportedVertexID = ExportedVertexIDPtr ? *ExportedVertexIDPtr : OriginalToExportedVertexIdMap.Num();

				VertexIdTrackers.Add(FVertexIdTracker(ExportedVertexID, OriginalVertexID, VertexInstanceID));

				MeshPrimitiveDescription.Indices.Add(ExportedVertexID);

				//Positions are Vertex based so they can be directly set, no need to check them:
				if (!ExportedVertexIDPtr)
				{
					OriginalToExportedVertexIdMap.Add(OriginalVertexID, ExportedVertexID);

					MeshPrimitiveDescription.Positions[ExportedVertexID] = VertexPositions[OriginalVertexID];
				}

				//For the VertexInstance based attributes we have to check if all VertexInstances that use the same Vertex base are identical/equal or not.
				//If they are equal then we can use these packed value, if NOT then we have to use the VertexInstances based solution.
				if (!CheckCompareSetValue(AttributesSetTracker.Normals[ExportedVertexID], MeshPrimitiveDescription.Normals[ExportedVertexID], VertexInstanceNormals[VertexInstanceID]))
				{
					return false;
				}
				if (!CheckCompareSetValue(AttributesSetTracker.Tangents[ExportedVertexID], MeshPrimitiveDescription.Tangents[ExportedVertexID], FVector4f(VertexInstanceTangents[VertexInstanceID], VertexInstanceBinormalSigns[VertexInstanceID])))
				{
					return false;
				}

				for (size_t UVIndex = 0; UVIndex < MeshDetails.UVCount; UVIndex++)
				{
					if (!CheckCompareSetValue(AttributesSetTracker.UVs[UVIndex][ExportedVertexID], MeshPrimitiveDescription.UVs[UVIndex][ExportedVertexID], VertexInstanceUVs.Get(VertexInstanceID, UVIndex)))
					{
						return false;
					}
				}
			}
		}

		if (ExportConfigs.bExportMorphTargets)
		{
			//TargetNames and VertexPositionDeltas will have the exact same amount of Targets
			for (size_t TargetCounter = 0; TargetCounter < NumOfTargets; TargetCounter++)
			{
				MeshPrimitiveDescription.Targets[TargetCounter].bHasPositions = TargetVertexPositionDeltas[TargetCounter].IsValid();
				MeshPrimitiveDescription.Targets[TargetCounter].bHasNormals = TargetVertexInstanceNormalDeltas[TargetCounter].IsValid();

				if (MeshPrimitiveDescription.Targets[TargetCounter].bHasPositions)
				{
					if (MeshPrimitiveDescription.Targets[TargetCounter].bHasNormals)
					{
						//Has both
						for (const FVertexIdTracker& VertexIdTracker : VertexIdTrackers)
						{
							int32 ExportedVertexID = VertexIdTracker.ExportedVertexId;
							FVertexID OriginalVertexID = VertexIdTracker.VertexId;
							FVertexInstanceID VertexInstanceID = VertexIdTracker.VertexInstanceId;

							MeshPrimitiveDescription.Targets[TargetCounter].Positions[ExportedVertexID] = TargetVertexPositionDeltas[TargetCounter][OriginalVertexID];

							if (!CheckCompareSetValue(AttributesSetTracker.TargetNormals[TargetCounter][ExportedVertexID],
								MeshPrimitiveDescription.Targets[TargetCounter].Normals[ExportedVertexID],
								TargetVertexInstanceNormalDeltas[TargetCounter][VertexInstanceID]))
							{
								return false;
							}
						}
					}
					else
					{
						//Has only positions
						for (const FVertexIdTracker& VertexIdTracker : VertexIdTrackers)
						{
							int32 ExportedVertexID = VertexIdTracker.ExportedVertexId;
							FVertexID OriginalVertexID = VertexIdTracker.VertexId;
							FVertexInstanceID VertexInstanceID = VertexIdTracker.VertexInstanceId;

							MeshPrimitiveDescription.Targets[TargetCounter].Positions[ExportedVertexID] = TargetVertexPositionDeltas[TargetCounter][OriginalVertexID];
						}
					}
				}
				else if (MeshPrimitiveDescription.Targets[TargetCounter].bHasNormals)
				{
					//Has only normals
					//Has both
					for (const FVertexIdTracker& VertexIdTracker : VertexIdTrackers)
					{
						int32 ExportedVertexID = VertexIdTracker.ExportedVertexId;
						FVertexID OriginalVertexID = VertexIdTracker.VertexId;
						FVertexInstanceID VertexInstanceID = VertexIdTracker.VertexInstanceId;

						if (!CheckCompareSetValue(AttributesSetTracker.TargetNormals[TargetCounter][ExportedVertexID],
							MeshPrimitiveDescription.Targets[TargetCounter].Normals[ExportedVertexID],
							TargetVertexInstanceNormalDeltas[TargetCounter][VertexInstanceID]))
						{
							return false;
						}
					}
				}
			}
		}

		//Set VertexColors if MeshDescription has any:
		if (bCanExportVertexColors)
		{
			for (const FTriangleID& TriangleID : TriangleIDs)
			{
				TArrayView<const FVertexInstanceID> TriangleVertexInstanceIDs = MeshDescription->GetTriangleVertexInstances(TriangleID);
				for (const FVertexInstanceID& VertexInstanceID : TriangleVertexInstanceIDs)
				{
					FVertexID VertexID = INDEX_NONE; int32 ExportedVertexID = INDEX_NONE;
					if (!GetOriginalAndExportedVertexIDs(VertexInstanceIdToVertexId, VertexInstanceID, VertexID, ExportedVertexID))
					{
						return false;
					}

					const FVector4f& SourceVertexColor = VertexInstanceColors[VertexInstanceID];
					if (!CheckCompareSetValue(AttributesSetTracker.VertexColors[ExportedVertexID], MeshPrimitiveDescription.VertexColors[ExportedVertexID], FLinearColor(SourceVertexColor).ToFColor(true)))
					{
						return false;
					}
				}
			}
		}

		if (ExportConfigs.bExportVertexSkinWeights)
		{
			FSkeletalMeshConstAttributes SkeletalMeshAttributes(*MeshDescription);
			FSkinWeightsVertexAttributesConstRef VertexSkinWeights = SkeletalMeshAttributes.GetVertexSkinWeights();

			//Find MaxBoneOInfluence straight from VertexSkinWeights:
			int32 MaxBoneInfluences = 0;
			for (const FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
			{
				const FVertexBoneWeightsConst BoneWeights = VertexSkinWeights.Get(VertexID);
				if (MaxBoneInfluences < BoneWeights.Num())
				{
					MaxBoneInfluences = BoneWeights.Num();
				}
			}

			const uint32 JointGroupCount = (MaxBoneInfluences + 3) / 4;
			MeshPrimitiveDescription.PrepareJointContainers(JointGroupCount, VertexCount);

			for (const FTriangleID& TriangleID : TriangleIDs)
			{
				TArrayView<const FVertexInstanceID> TriangleVertexInstanceIDs = MeshDescription->GetTriangleVertexInstances(TriangleID);
				for (const FVertexInstanceID& VertexInstanceID : TriangleVertexInstanceIDs)
				{
					FVertexID VertexID = INDEX_NONE; int32 ExportedVertexID = INDEX_NONE;
					if (!GetOriginalAndExportedVertexIDs(VertexInstanceIdToVertexId, VertexInstanceID, VertexID, ExportedVertexID))
					{
						return false;
					}

					int32 GroupCounter = 0;
					int32 NumInfluences = 0;
					int32 InGroupInfluenceCounter = 0;
					const FVertexBoneWeightsConst BoneWeights = VertexSkinWeights.Get(VertexID);
					for (const UE::AnimationCore::FBoneWeight BoneWeight : BoneWeights)
					{
						MeshPrimitiveDescription.JointInfluences[GroupCounter][ExportedVertexID][InGroupInfluenceCounter] = BoneWeight.GetBoneIndex();
						MeshPrimitiveDescription.JointWeights[GroupCounter][ExportedVertexID][InGroupInfluenceCounter] = BoneWeight.GetRawWeight();

						NumInfluences++;
						GroupCounter = (NumInfluences / ExportConfigs.SkeletonInfluenceCountPerGroup);
						InGroupInfluenceCounter = NumInfluences % ExportConfigs.SkeletonInfluenceCountPerGroup;
					}
				}
			}
		}

		return true;
	}

	template <class T>
	void FMeshDescriptionParser<T>::ParseVertexInstanceBased(const TArrayView<const FTriangleID>& TriangleIDs, FMeshPrimitiveDescription& MeshPrimitiveDescription, const FExportConfigs& ExportConfigs)
	{
		//This approach should be used when the VertexBased fails.
		//		VertexBased mesh parsing can fail due to non idential VertexInstance attributes (per Vertex)

		using namespace Private;

		const int32 TrianglesCount = TriangleIDs.Num();
		const int32 MaxVertexInstanceCount = TrianglesCount * 3; //Max number of VertexInstances used in the PolygonGroup aka the Primitive.

		//Note: while for the VertexBasedMeshParsing we calculate the VERTEX Count at this stage, here we calculate the VERTEXINSTANCE count:
		TSet<FVertexID> UniqueVertexInstanceIDs;
		UniqueVertexInstanceIDs.Reserve(MaxVertexInstanceCount);
		for (const FTriangleID& TriangleID : TriangleIDs)
		{
			TArrayView<const FVertexInstanceID> TriangleVertexInstanceIDs = MeshDescription->GetTriangleVertexInstances(TriangleID);
			for (const FVertexInstanceID& VertexInstanceID : TriangleVertexInstanceIDs)
			{
				UniqueVertexInstanceIDs.Add(VertexInstanceID);
			}
		}
		const int32 VertexInstanceCount = UniqueVertexInstanceIDs.Num();
		TMap<int32, int32> OriginalToExportedVertexIdMap;
		auto GetOriginalAndExportedVertexIDs = [&OriginalToExportedVertexIdMap](const FVertexID& VertexInstanceID, int32& ExportedVertexID)
			{
				int32* ExportedVertexIDPtr = OriginalToExportedVertexIdMap.Find(VertexInstanceID);
				if (!ExportedVertexIDPtr)
				{
					return false;
				}
				ExportedVertexID = *ExportedVertexIDPtr;

				return true;
			};

		const bool bCanExportVertexColors = MeshDetails.bHasVertexColors && ExportConfigs.bExportVertexColors;

		//SetNum/Reserve the containers:
		MeshPrimitiveDescription.PrepareContainers(VertexInstanceCount, VertexInstanceCount, MeshDetails.UVCount, bCanExportVertexColors, TargetNames.Num());

		TArray<FVertexIdTracker> VertexIdTrackers; //Exported to Original(VertexId, VertexInstanceId), primarily used for Targets.
		VertexIdTrackers.Reserve(TriangleIDs.Num() * 3);

		//Based on Triangles acquire the VertexInstanceID, based on which acquire the VertexInstance Attributes:
		for (const FTriangleID& TriangleID : TriangleIDs)
		{
			TArrayView<const FVertexInstanceID> TriangleVertexInstanceIDs = MeshDescription->GetTriangleVertexInstances(TriangleID);
			for (const FVertexInstanceID& VertexInstanceID : TriangleVertexInstanceIDs)
			{
				FVertexID OriginalVertexID = VertexInstanceIdToVertexId[VertexInstanceID];
				int32* ExportedVertexIDPtr = OriginalToExportedVertexIdMap.Find(VertexInstanceID);
				int32 ExportedVertexID = ExportedVertexIDPtr ? *ExportedVertexIDPtr : OriginalToExportedVertexIdMap.Num();

				VertexIdTrackers.Add(FVertexIdTracker(ExportedVertexID, OriginalVertexID, VertexInstanceID));

				MeshPrimitiveDescription.Indices.Add(ExportedVertexID);

				//Positions and Indicies are Vertex based so they can be directly set, no need to check them:
				if (!ExportedVertexIDPtr)
				{
					OriginalToExportedVertexIdMap.Add(VertexInstanceID, ExportedVertexID);

					MeshPrimitiveDescription.Positions[ExportedVertexID] = VertexPositions[OriginalVertexID];

					MeshPrimitiveDescription.Normals[ExportedVertexID] = VertexInstanceNormals[VertexInstanceID];
					MeshPrimitiveDescription.Tangents[ExportedVertexID] = FVector4f(VertexInstanceTangents[VertexInstanceID], VertexInstanceBinormalSigns[VertexInstanceID]);

					for (size_t UVIndex = 0; UVIndex < MeshDetails.UVCount; UVIndex++)
					{
						MeshPrimitiveDescription.UVs[UVIndex][ExportedVertexID] = VertexInstanceUVs.Get(VertexInstanceID, UVIndex);
					}
				}

			}
		}

		if (ExportConfigs.bExportMorphTargets)
		{
			//TargetNames and VertexPositionDeltas will have the exact same amount of Targets
			for (size_t TargetCounter = 0; TargetCounter < NumOfTargets; TargetCounter++)
			{
				MeshPrimitiveDescription.Targets[TargetCounter].bHasPositions = TargetVertexPositionDeltas[TargetCounter].IsValid();
				MeshPrimitiveDescription.Targets[TargetCounter].bHasNormals = TargetVertexInstanceNormalDeltas[TargetCounter].IsValid();

				if (MeshPrimitiveDescription.Targets[TargetCounter].bHasPositions)
				{
					if (MeshPrimitiveDescription.Targets[TargetCounter].bHasNormals)
					{
						//Has both
						for (const FVertexIdTracker& VertexIdTracker : VertexIdTrackers)
						{
							int32 ExportedVertexID = VertexIdTracker.ExportedVertexId;
							FVertexID OriginalVertexID = VertexIdTracker.VertexId;
							FVertexInstanceID VertexInstanceID = VertexIdTracker.VertexInstanceId;

							MeshPrimitiveDescription.Targets[TargetCounter].Positions[ExportedVertexID] = TargetVertexPositionDeltas[TargetCounter][OriginalVertexID];
							MeshPrimitiveDescription.Targets[TargetCounter].Normals[ExportedVertexID] = TargetVertexInstanceNormalDeltas[TargetCounter][VertexInstanceID];
						}
					}
					else
					{
						//Has only positions
						for (const FVertexIdTracker& VertexIdTracker : VertexIdTrackers)
						{
							int32 ExportedVertexID = VertexIdTracker.ExportedVertexId;
							FVertexID OriginalVertexID = VertexIdTracker.VertexId;
							FVertexInstanceID VertexInstanceID = VertexIdTracker.VertexInstanceId;

							MeshPrimitiveDescription.Targets[TargetCounter].Positions[ExportedVertexID] = TargetVertexPositionDeltas[TargetCounter][OriginalVertexID];
						}
					}
				}
				else if (MeshPrimitiveDescription.Targets[TargetCounter].bHasNormals)
				{
					//Has only normals
					for (const FVertexIdTracker& VertexIdTracker : VertexIdTrackers)
					{
						int32 ExportedVertexID = VertexIdTracker.ExportedVertexId;
						FVertexID OriginalVertexID = VertexIdTracker.VertexId;
						FVertexInstanceID VertexInstanceID = VertexIdTracker.VertexInstanceId;

						MeshPrimitiveDescription.Targets[TargetCounter].Normals[ExportedVertexID] = TargetVertexInstanceNormalDeltas[TargetCounter][VertexInstanceID];
					}
				}
			}
		}

		//Set VertexColors if MeshDescription has any:
		if (bCanExportVertexColors)
		{
			for (const FTriangleID& TriangleID : TriangleIDs)
			{
				TArrayView<const FVertexInstanceID> TriangleVertexInstanceIDs = MeshDescription->GetTriangleVertexInstances(TriangleID);
				for (const FVertexInstanceID& VertexInstanceID : TriangleVertexInstanceIDs)
				{
					int32 ExportedVertexID = INDEX_NONE;
					if (!GetOriginalAndExportedVertexIDs(VertexInstanceID, ExportedVertexID)) return;

					const FVector4f& SourceVertexColor = VertexInstanceColors[VertexInstanceID];

					MeshPrimitiveDescription.VertexColors[ExportedVertexID] = FLinearColor(SourceVertexColor).ToFColor(true);
				}
			}
		}

		if (ExportConfigs.bExportVertexSkinWeights)
		{
			FSkeletalMeshConstAttributes SkeletalMeshAttributes(*MeshDescription);
			FSkinWeightsVertexAttributesConstRef VertexSkinWeights = SkeletalMeshAttributes.GetVertexSkinWeights();

			int32 MaxBoneInfluences = 0;
			for (const FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
			{
				const FVertexBoneWeightsConst BoneWeights = VertexSkinWeights.Get(VertexID);
				if (MaxBoneInfluences < BoneWeights.Num())
				{
					MaxBoneInfluences = BoneWeights.Num();
				}
			}
			const uint32 JointGroupCount = (MaxBoneInfluences + 3) / 4;

			MeshPrimitiveDescription.PrepareJointContainers(JointGroupCount, VertexInstanceCount);

			for (const FTriangleID& TriangleID : TriangleIDs)
			{
				TArrayView<const FVertexInstanceID> TriangleVertexInstanceIDs = MeshDescription->GetTriangleVertexInstances(TriangleID);
				for (const FVertexInstanceID& VertexInstanceID : TriangleVertexInstanceIDs)
				{
					int32 ExportedVertexID = INDEX_NONE;
					if (!GetOriginalAndExportedVertexIDs(VertexInstanceID, ExportedVertexID)) return;
					FVertexID OriginalVertexID = VertexInstanceIdToVertexId[VertexInstanceID];

					int32 GroupCounter = 0;
					int32 NumInfluences = 0;
					int32 InGroupInfluenceCounter = 0;
					const FVertexBoneWeightsConst BoneWeights = VertexSkinWeights.Get(OriginalVertexID);
					for (const UE::AnimationCore::FBoneWeight BoneWeight : BoneWeights)
					{
						MeshPrimitiveDescription.JointInfluences[GroupCounter][ExportedVertexID][InGroupInfluenceCounter] = BoneWeight.GetBoneIndex();
						MeshPrimitiveDescription.JointWeights[GroupCounter][ExportedVertexID][InGroupInfluenceCounter] = BoneWeight.GetRawWeight();

						NumInfluences++;
						GroupCounter = (NumInfluences / ExportConfigs.SkeletonInfluenceCountPerGroup);
						InGroupInfluenceCounter = NumInfluences % ExportConfigs.SkeletonInfluenceCountPerGroup;
					}
				}
			}
		}
	}

	template <class T>
	int32 FMeshDescriptionParser<T>::GetMaterialIndex(const FPolygonGroupID& PolygonGroupID)
	{
		FName MaterialSlotName = PolygonGroupMaterialSlotNames[PolygonGroupID];

		int32 MaterialIndex = INDEX_NONE;

		for (size_t MatIndex = 0; MatIndex < MaterialSlots.Num(); MatIndex++)
		{
			if (MaterialSlots[MatIndex].ImportedMaterialSlotName == MaterialSlotName)
			{
				MaterialIndex = MatIndex;
				break;
			}
		}

		if (MaterialIndex == INDEX_NONE)
		{
			MaterialIndex = PolygonGroupID.GetValue();
		}
		if (!MaterialSlots.IsValidIndex(MaterialIndex))
		{
			MaterialIndex = 0;
		}

		return MaterialIndex;
	}

	template struct FMeshDescriptionParser<FStaticMaterial>;
	template struct FMeshDescriptionParser<FSkeletalMaterial>;
}

template <class T>
int32 UE::MeshParser::Private::GetNumberOfValidPolygons(const T* Mesh/*either UStaticMesh or USkeletalMesh*/, const int32 LODIndex)
{
	if (!Mesh)
	{
		return 0;
	}

	FMeshDescription* MeshDescription = nullptr;
	if constexpr (std::is_same_v<T, UStaticMesh>)
	{
		MeshDescription = Mesh->GetMeshDescription(LODIndex);
	}
	else if constexpr (std::is_same_v<T, USkeletalMesh>)
	{
		if (Mesh->GetNumSourceModels() <= LODIndex || !Mesh->IsSourceModelValid(LODIndex))
		{
			return 0;
		}
		const FSkeletalMeshSourceModel& MeshSourceModel = Mesh->GetSourceModel(LODIndex);
		MeshDescription = MeshSourceModel.GetMeshDescription();
	}
	else
	{
		ensure(false);
		return 0;
	}

	if (!MeshDescription)
	{
		return 0;
	}

	int PolygonCounter = 0;

	//Iterate through all PolygonsGroups:
	for (const FPolygonGroupID& PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
	{
		//Acquire the TriangleIDs for the current PolygonGroup:
		const TArrayView<const FTriangleID>& TriangleIDs = MeshDescription->GetPolygonGroupTriangles(PolygonGroupID);

		if (TriangleIDs.Num() == 0)
		{
			//Do not export empty primitives.
			continue;
		}

		PolygonCounter++;
	}

	return PolygonCounter;
}

template int32 UE::MeshParser::Private::GetNumberOfValidPolygons<UStaticMesh>(const UStaticMesh*, const int32 LODIndex);
template int32 UE::MeshParser::Private::GetNumberOfValidPolygons<USkeletalMesh>(const USkeletalMesh*, const int32 LODIndex);

#endif