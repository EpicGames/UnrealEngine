// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Builders/GLTFBufferBuilder.h"
#include "CoreMinimal.h"

#define UE_API GLTFEXPORTER_API

struct FAnalyticsEventAttribute;

class AActor;
class UStaticMesh;
class USkeletalMesh;
class ULevelSequence;
class UAnimSequence;
class UMaterialInterface;
class UTexture;
class UCameraComponent;
class ULightComponent;
class ULandscapeComponent;

class FGLTFAnalyticsBuilder : public FGLTFBufferBuilder
{
public:

	UE_API FGLTFAnalyticsBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions);

	UE_API TArray<FAnalyticsEventAttribute> GenerateAnalytics() const;

protected:

	UE_API void RecordActor(const AActor* Object);
	UE_API void RecordStaticMesh(const UStaticMesh* Object);
	UE_API void RecordSkeletalMesh(const USkeletalMesh* Object);
	UE_API void RecordSplineStaticMesh(const UStaticMesh* Object);
	UE_API void RecordLandscapeComponent(const ULandscapeComponent* Object);
	UE_API void RecordLevelSequence(const ULevelSequence* Object);
	UE_API void RecordAnimSequence(const UAnimSequence* Object);
	UE_API void RecordMaterial(const UMaterialInterface* Object);
	UE_API void RecordTexture(const UTexture* Object);
	UE_API void RecordCamera(const UCameraComponent* Object);
	UE_API void RecordLight(const ULightComponent* Object);

private:

	TSet<const AActor*>					ActorsRecorded;
	TSet<const USceneComponent*>		ComponentsRecorded;
	TSet<const UStaticMesh*>			StaticMeshesRecorded;
	TSet<const USkeletalMesh*>			SkeletalMeshesRecorded;
	TSet<const UStaticMesh*>			SplineStaticMeshesRecorded;
	TSet<const ULandscapeComponent*>	LandscapeComponentsRecorded;
	TSet<const ULevelSequence*>			LevelSequencesRecorded;
	TSet<const UAnimSequence*>			AnimSequencesRecorded;
	TSet<const UMaterialInterface*>		MaterialsRecorded;
	TSet<const UTexture*>				TexturesRecorded;
	
	TSet<const UCameraComponent*>		CamerasRecorded;
	TSet<const ULightComponent*>		LightsRecorded;
};

#undef UE_API
