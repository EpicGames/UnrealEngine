// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Builders/GLTFJsonBuilder.h"

#define UE_API GLTFEXPORTER_API

class FGLTFBufferBuilder : public FGLTFJsonBuilder
{
protected:

	UE_API FGLTFBufferBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions = nullptr);
	UE_API ~FGLTFBufferBuilder();

	UE_API const FGLTFMemoryArchive* GetBufferData() const;

public:

	UE_API FGLTFJsonBufferView* AddBufferView(const void* RawData, uint64 ByteLength, EGLTFJsonBufferTarget BufferTarget = EGLTFJsonBufferTarget::None, uint8 DataAlignment = 4);

	template <class ElementType>
	FGLTFJsonBufferView* AddBufferView(const TArray<ElementType>& Array, EGLTFJsonBufferTarget BufferTarget = EGLTFJsonBufferTarget::None, uint8 DataAlignment = 4)
	{
		return AddBufferView(Array.GetData(), Array.Num() * sizeof(ElementType), BufferTarget, DataAlignment);
	}

private:

	UE_API void InitializeBuffer();

	FGLTFJsonBuffer* JsonBuffer;
	TSharedPtr<FGLTFMemoryArchive> BufferArchive;
};

#undef UE_API
