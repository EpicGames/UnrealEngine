// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Options/GLTFExportOptions.h"
#include "UObject/GCObjectScopeGuard.h"

#define UE_API GLTFEXPORTER_API

class UMaterialInterface;
class UStaticMesh;
class UStaticMeshComponent;
class USkeletalMesh;
class USkeletalMeshComponent;
class USplineMeshComponent;

class FGLTFBuilder
{
public:

	const FString FileName;
	const bool bIsGLB;

	const UGLTFExportOptions* ExportOptions; // TODO: make ExportOptions private and expose each option via getters to ease overriding settings in future

	UE_API FGLTFBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions = nullptr);

	UE_API const UMaterialInterface* ResolveProxy(const UMaterialInterface* Material) const;
	UE_API void ResolveProxies(TArray<const UMaterialInterface*>& Materials) const;

	UE_API FGLTFMaterialBakeSize GetBakeSizeForMaterialProperty(const UMaterialInterface* Material, EGLTFMaterialPropertyGroup PropertyGroup) const;
	UE_API TextureFilter GetBakeFilterForMaterialProperty(const UMaterialInterface* Material, EGLTFMaterialPropertyGroup PropertyGroup) const;
	UE_API TextureAddress GetBakeTilingForMaterialProperty(const UMaterialInterface* Material, EGLTFMaterialPropertyGroup PropertyGroup) const;

	UE_API int32 SanitizeLOD(const UStaticMesh* StaticMesh, const UStaticMeshComponent* StaticMeshComponent, int32 LODIndex) const;
	UE_API int32 SanitizeLOD(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent, int32 LODIndex) const;
	UE_API int32 SanitizeLOD(const UStaticMesh* StaticMesh, const USplineMeshComponent* SplineMeshComponent, int32 LODIndex) const;

private:

	FGCObjectScopeGuard ExportOptionsGuard;

	static const UGLTFExportOptions* SanitizeExportOptions(const UGLTFExportOptions* Options);
};

#undef UE_API
