// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Builders/GLTFConvertBuilder.h"

#define UE_API GLTFEXPORTER_API

class FGLTFContainerBuilder : public FGLTFConvertBuilder
{
public:

	UE_API FGLTFContainerBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions = nullptr, const TSet<AActor*>& SelectedActors = {});

	UE_API bool WriteInternalArchive(FArchive& Archive);

	UE_API bool WriteAllFiles(const FString& DirPath, uint32 WriteFlags = 0);
	UE_API bool WriteAllFiles(const FString& DirPath, TArray<FString>& OutFilePaths, uint32 WriteFlags = 0);

	UE_API void GetAllFiles(TArray<FString>& OutFilePaths, const FString& DirPath = TEXT("")) const;

protected:

	UE_API bool WriteGlbArchive(FArchive& Archive);

private:

	bool WriteGlb(FArchive& Archive, const TArray64<uint8>& JsonData, const TArray64<uint8>* BinaryData);

	static void WriteHeader(FArchive& Archive, uint32 FileSize);
	static void WriteChunk(FArchive& Archive, uint32 ChunkType, const TArray64<uint8>& ChunkData, uint8 PaddingValue);

	static void WriteInt(FArchive& Archive, uint32 Value);
	static void WriteData(FArchive& Archive, const TArray64<uint8>& Data);
	static void WriteFill(FArchive& Archive, uint32 Size, uint8 Value);

	static uint32 GetChunkPaddingLength(uint64 Size);
};

#undef UE_API
