// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Builders/GLTFAnalyticsBuilder.h"
#include "Converters/GLTFAccessorConverters.h"
#include "Converters/GLTFMeshConverters.h"
#include "Converters/GLTFMeshDataConverters.h"
#include "Converters/GLTFMaterialConverters.h"
#include "Converters/GLTFSamplerConverters.h"
#include "Converters/GLTFTextureConverters.h"
#include "Converters/GLTFImageConverters.h"
#include "Converters/GLTFNodeConverters.h"
#include "Converters/GLTFSkinConverters.h"
#include "Converters/GLTFAnimationConverters.h"
#include "Converters/GLTFSceneConverters.h"
#include "Converters/GLTFCameraConverters.h"
#include "Converters/GLTFLightConverters.h"
#include "Converters/GLTFMaterialVariantConverters.h"
#include "Converters/GLTFMeshAttributesArray.h"
#include "Converters/GLTFLightMapConverters.h"

#define UE_API GLTFEXPORTER_API

class UMeshComponent;
class UPropertyValue;

class FGLTFConvertBuilder : public FGLTFAnalyticsBuilder
{
public:

	const TSet<AActor*> SelectedActors;

	UE_API FGLTFConvertBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions = nullptr, const TSet<AActor*>& SelectedActors = {});

	UE_API bool IsSelectedActor(const AActor* Object) const;
	UE_API bool IsRootActor(const AActor* Actor) const;

	UE_API FGLTFJsonAccessor* AddUniquePositionAccessor(const FGLTFMeshSection* MeshSection, const FPositionVertexBuffer* VertexBuffer);
	UE_API FGLTFJsonAccessor* AddUniquePositionDeltaAccessor(const FGLTFMeshSection* MeshSection, const TMap<uint32, FVector3f>* TargetPositionDeltas);
	UE_API FGLTFJsonAccessor* AddUniquePositionAccessor(const FGLTFPositionArray& VertexBuffer);
	UE_API FGLTFJsonAccessor* AddUniqueColorAccessor(const FGLTFMeshSection* MeshSection, const FColorVertexBuffer* VertexBuffer);
	UE_API FGLTFJsonAccessor* AddUniqueColorAccessor(const FGLTFColorArray& VertexColorBuffer);
	UE_API FGLTFJsonAccessor* AddUniqueNormalAccessor(const FGLTFMeshSection* MeshSection, const FStaticMeshVertexBuffer* VertexBuffer);
	UE_API FGLTFJsonAccessor* AddUniqueNormalDeltaAccessor(const FGLTFMeshSection* MeshSection, const TMap<uint32, FVector3f>* TargetNormalDeltas, const bool& bHighPrecision);
	UE_API FGLTFJsonAccessor* AddUniqueNormalAccessor(const FGLTFNormalArray& Normals, const bool& bNormalize = true);
	UE_API FGLTFJsonAccessor* AddUniqueTangentAccessor(const FGLTFMeshSection* MeshSection, const FStaticMeshVertexBuffer* VertexBuffer);
	UE_API FGLTFJsonAccessor* AddUniqueTangentAccessor(const FGLTFTangentArray& Tangents);
	UE_API FGLTFJsonAccessor* AddUniqueUVAccessor(const FGLTFMeshSection* MeshSection, const FStaticMeshVertexBuffer* VertexBuffer, int32 UVIndex);
	UE_API FGLTFJsonAccessor* AddUniqueUVAccessor(const FGLTFUVArray& UVs); /* Supports single UV channel */
	UE_API FGLTFJsonAccessor* AddUniqueJointAccessor(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, int32 InfluenceOffset);
	UE_API FGLTFJsonAccessor* AddUniqueWeightAccessor(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, int32 InfluenceOffset);
	UE_API FGLTFJsonAccessor* AddUniqueJointAccessor(const FGLTFJointInfluenceArray& BoneIndices);
	UE_API FGLTFJsonAccessor* AddUniqueWeightAccessor(const FGLTFJointWeightArray& Weights);
	UE_API FGLTFJsonAccessor* AddUniqueIndexAccessor(const FGLTFMeshSection* MeshSection);
	UE_API FGLTFJsonAccessor* AddUniqueIndexAccessor(const FGLTFIndexArray& IndexBuffer, const FString& MeshName);

	UE_API FGLTFJsonMesh* AddUniqueMesh(const UStaticMesh* StaticMesh, const FGLTFMaterialArray& Materials = {}, int32 LODIndex = INDEX_NONE);
	UE_API FGLTFJsonMesh* AddUniqueMesh(const USkeletalMesh* SkeletalMesh, const FGLTFMaterialArray& Materials = {}, int32 LODIndex = INDEX_NONE);
	UE_API FGLTFJsonMesh* AddUniqueMesh(const UMeshComponent* MeshComponent, const FGLTFMaterialArray& Materials = {}, int32 LODIndex = INDEX_NONE);
	UE_API FGLTFJsonMesh* AddUniqueMesh(const UStaticMeshComponent* StaticMeshComponent, const FGLTFMaterialArray& Materials = {}, int32 LODIndex = INDEX_NONE);
	UE_API FGLTFJsonMesh* AddUniqueMesh(const USkeletalMeshComponent* SkeletalMeshComponent, const FGLTFMaterialArray& Materials = {}, int32 LODIndex = INDEX_NONE);
	UE_API FGLTFJsonMesh* AddUniqueMesh(const USplineMeshComponent* SplineMeshComponent, const FGLTFMaterialArray& Materials = {}, int32 LODIndex = INDEX_NONE);
	UE_API FGLTFJsonMesh* AddUniqueMesh(const ULandscapeComponent* LandscapeComponent, const UMaterialInterface* LandscapeMaterial = nullptr);

	UE_API const FGLTFMeshData* AddUniqueMeshData(const UStaticMesh* StaticMesh, const UStaticMeshComponent* StaticMeshComponent = nullptr, int32 LODIndex = INDEX_NONE);
	UE_API const FGLTFMeshData* AddUniqueMeshData(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent = nullptr, int32 LODIndex = INDEX_NONE);

	UE_API FGLTFJsonMaterial* AddUniqueMaterial(const UMaterialInterface* Material, const UStaticMesh* StaticMesh, int32 LODIndex = INDEX_NONE, int32 MaterialIndex = INDEX_NONE);
	UE_API FGLTFJsonMaterial* AddUniqueMaterial(const UMaterialInterface* Material, const USkeletalMesh* SkeletalMesh, int32 LODIndex = INDEX_NONE, int32 MaterialIndex = INDEX_NONE);
	UE_API FGLTFJsonMaterial* AddUniqueMaterial(const UMaterialInterface* Material, const UMeshComponent* MeshComponent, int32 LODIndex = INDEX_NONE, int32 MaterialIndex = INDEX_NONE);
	UE_API FGLTFJsonMaterial* AddUniqueMaterial(const UMaterialInterface* Material, const UStaticMeshComponent* StaticMeshComponent, int32 LODIndex = INDEX_NONE, int32 MaterialIndex = INDEX_NONE);
	UE_API FGLTFJsonMaterial* AddUniqueMaterial(const UMaterialInterface* Material, const USkeletalMeshComponent* SkeletalMeshComponent, int32 LODIndex = INDEX_NONE, int32 MaterialIndex = INDEX_NONE);
	UE_API FGLTFJsonMaterial* AddUniqueMaterial(const UMaterialInterface* Material, const FGLTFMeshData* MeshData = nullptr, const FGLTFIndexArray& SectionIndices = {});

	UE_API FGLTFJsonSampler* AddUniqueSampler(const UTexture* Texture);
	UE_API FGLTFJsonSampler* AddUniqueSampler(TextureAddress Address, TextureFilter Filter, TextureGroup LODGroup = TEXTUREGROUP_MAX);
	UE_API FGLTFJsonSampler* AddUniqueSampler(TextureAddress AddressX, TextureAddress AddressY, TextureFilter Filter, TextureGroup LODGroup = TEXTUREGROUP_MAX);

	UE_API FGLTFJsonTexture* AddUniqueTexture(const UTexture* Texture);
	UE_API FGLTFJsonTexture* AddUniqueTexture(const UTexture2D* Texture);
	UE_API FGLTFJsonTexture* AddUniqueTexture(const UTextureRenderTarget2D* Texture);
	UE_API FGLTFJsonTexture* AddUniqueTexture(const UTexture* Texture, bool bToSRGB, TextureAddress TextureAddressX = TextureAddress::TA_MAX, TextureAddress TextureAddressY = TextureAddress::TA_MAX);
	UE_API FGLTFJsonTexture* AddUniqueTexture(const UTexture2D* Texture, bool bToSRGB, TextureAddress TextureAddressX, TextureAddress TextureAddressY);
	UE_API FGLTFJsonTexture* AddUniqueTexture(const UTextureRenderTarget2D* Texture, bool bToSRGB);
	UE_API FGLTFJsonTexture* AddUniqueTexture(const ULightMapTexture2D* Texture);

	UE_API FGLTFJsonImage* AddUniqueImage(TGLTFSharedArray<FColor>& Pixels, FIntPoint Size, bool bIgnoreAlpha, const FString& Name);

	UE_API FGLTFJsonSkin* AddUniqueSkin(FGLTFJsonNode* RootNode, const USkeletalMesh* SkeletalMesh);
	UE_API FGLTFJsonSkin* AddUniqueSkin(FGLTFJsonNode* RootNode, const USkeletalMeshComponent* SkeletalMeshComponent);
	UE_API FGLTFJsonAnimation* AddUniqueAnimation(FGLTFJsonNode* RootNode, const USkeletalMesh* SkeletalMesh, const UAnimSequence* AnimSequence);
	UE_API FGLTFJsonAnimation* AddUniqueAnimation(FGLTFJsonNode* RootNode, const USkeletalMeshComponent* SkeletalMeshComponent);
	UE_API FGLTFJsonAnimation* AddUniqueAnimation(const ULevel* Level, const ULevelSequence* LevelSequence);
	UE_API FGLTFJsonAnimation* AddUniqueAnimation(const ALevelSequenceActor* LevelSequenceActor);

	UE_API FGLTFJsonNode* AddUniqueNode(const AActor* Actor);
	UE_API FGLTFJsonNode* AddUniqueNode(const USceneComponent* SceneComponent);
	UE_API FGLTFJsonNode* AddUniqueNode(const USceneComponent* SceneComponent, FName SocketName);
	UE_API FGLTFJsonNode* AddUniqueNode(FGLTFJsonNode* RootNode, const UStaticMesh* StaticMesh, FName SocketName);
	UE_API FGLTFJsonNode* AddUniqueNode(FGLTFJsonNode* RootNode, const USkeletalMesh* SkeletalMesh, FName SocketName);
	UE_API FGLTFJsonNode* AddUniqueNode(FGLTFJsonNode* RootNode, const USkeletalMesh* SkeletalMesh, int32 BoneIndex);
	UE_API FGLTFJsonScene* AddUniqueScene(const UWorld* World);

	UE_API FGLTFJsonCamera* AddUniqueCamera(const UCameraComponent* CameraComponent);
	UE_API FGLTFJsonLight* AddUniqueLight(const ULightComponent* LightComponent);
	UE_API FGLTFJsonLightIES* AddUniqueLightIES(const ULightComponent* LightComponent);
	UE_API FGLTFJsonLightIESInstance* AddUniqueLightIESInstance(const ULightComponent* LightComponent);
	UE_API FGLTFJsonMaterialVariant* AddUniqueMaterialVariant(const UVariant* Variant);
	UE_API FGLTFJsonLightMap* AddUniqueLightMap(const UStaticMeshComponent* StaticMeshComponent);

	UE_API void RegisterObjectVariant(const UObject* Object, const UPropertyValue* Property);
	UE_API const TArray<const UPropertyValue*>* GetObjectVariants(const UObject* Object) const;

	//'Original' converters:
	TUniquePtr<IGLTFPositionBufferConverter> PositionBufferConverter = MakeUnique<FGLTFPositionBufferConverter>(*this);
	TUniquePtr<IGLTFPositionDeltaBufferConverter> PositionDeltaBufferConverter = MakeUnique<FGLTFPositionDeltaBufferConverter>(*this);
	TUniquePtr<IGLTFColorBufferConverter> ColorBufferConverter = MakeUnique<FGLTFColorBufferConverter>(*this);
	TUniquePtr<IGLTFNormalBufferConverter> NormalBufferConverter = MakeUnique<FGLTFNormalBufferConverter>(*this);
	TUniquePtr<IGLTFNormalDeltaBufferConverter> NormalDeltaBufferConverter = MakeUnique<FGLTFNormalDeltaBufferConverter>(*this);
	TUniquePtr<IGLTFTangentBufferConverter> TangentBufferConverter = MakeUnique<FGLTFTangentBufferConverter>(*this);
	TUniquePtr<IGLTFUVBufferConverter> UVBufferConverter = MakeUnique<FGLTFUVBufferConverter>(*this);
	TUniquePtr<IGLTFBoneIndexBufferConverter> BoneIndexBufferConverter = MakeUnique<FGLTFBoneIndexBufferConverter>(*this);
	TUniquePtr<IGLTFBoneWeightBufferConverter> BoneWeightBufferConverter = MakeUnique<FGLTFBoneWeightBufferConverter>(*this);
	TUniquePtr<IGLTFIndexBufferConverter> IndexBufferConverter = MakeUnique<FGLTFIndexBufferConverter>(*this);

	//Raw converters:
	TUniquePtr<IGLTFPositionBufferConverterRaw> PositionBufferConverterRaw = MakeUnique<FGLTFPositionBufferConverterRaw>(*this);
	TUniquePtr<IGLTFColorBufferConverterRaw> ColorBufferConverterRaw = MakeUnique<FGLTFColorBufferConverterRaw>(*this);
	TUniquePtr<IGLTFNormalBufferConverterRaw> NormalBufferConverterRaw = MakeUnique<FGLTFNormalBufferConverterRaw>(*this);
	TUniquePtr<IGLTFTangentBufferConverterRaw> TangentBufferConverterRaw = MakeUnique<FGLTFTangentBufferConverterRaw>(*this);
	TUniquePtr<IGLTFUVBufferConverterRaw> UVBufferConverterRaw = MakeUnique<FGLTFUVBufferConverterRaw>(*this);
	TUniquePtr<IGLTFIndexBufferConverterRaw> IndexBufferConverterRaw = MakeUnique<FGLTFIndexBufferConverterRaw>(*this);
	TUniquePtr<IGLTFBoneIndexBufferConverterRaw> BoneIndexBufferConverterRaw = MakeUnique<FGLTFBoneIndexBufferConverterRaw>(*this);
	TUniquePtr<IGLTFBoneWeightBufferConverterRaw> BoneWeightBufferConverterRaw = MakeUnique<FGLTFBoneWeightBufferConverterRaw>(*this);


	TUniquePtr<IGLTFSplineMeshConverter> SplineMeshConverter = MakeUnique<FGLTFSplineMeshConverter>(*this);
	TUniquePtr<IGLTFLandscapeMeshConverter> LandscapeConverter = MakeUnique<FGLTFLandscapeMeshConverter>(*this);

	TUniquePtr<IGLTFStaticMeshConverter> StaticMeshConverter = MakeUnique<FGLTFStaticMeshConverter>(*this);
	TUniquePtr<IGLTFSkeletalMeshConverter> SkeletalMeshConverter = MakeUnique<FGLTFSkeletalMeshConverter>(*this);

	TUniquePtr<IGLTFMaterialConverter> MaterialConverter = MakeUnique<FGLTFMaterialConverter>(*this);
	TUniquePtr<IGLTFStaticMeshDataConverter> StaticMeshDataConverter = MakeUnique<FGLTFStaticMeshDataConverter>(*this);
	TUniquePtr<IGLTFSkeletalMeshDataConverter> SkeletalMeshDataConverter = MakeUnique<FGLTFSkeletalMeshDataConverter>(*this);

	TUniquePtr<IGLTFSamplerConverter> SamplerConverter = MakeUnique<FGLTFSamplerConverter>(*this);

	TUniquePtr<IGLTFTexture2DConverter> Texture2DConverter = MakeUnique<FGLTFTexture2DConverter>(*this);
	TUniquePtr<IGLTFTextureRenderTarget2DConverter> TextureRenderTarget2DConverter = MakeUnique<FGLTFTextureRenderTarget2DConverter>(*this);
	TUniquePtr<IGLTFImageConverter> ImageConverter = MakeUnique<FGLTFImageConverter>(*this);

	TUniquePtr<IGLTFSkinConverter> SkinConverter = MakeUnique<FGLTFSkinConverter>(*this);
	TUniquePtr<IGLTFAnimationConverter> AnimationConverter = MakeUnique<FGLTFAnimationConverter>(*this);
	TUniquePtr<IGLTFAnimationDataConverter> AnimationDataConverter = MakeUnique<FGLTFAnimationDataConverter>(*this);
	TUniquePtr<IGLTFLevelSequenceConverter> LevelSequenceConverter = MakeUnique<FGLTFLevelSequenceConverter>(*this);
	TUniquePtr<IGLTFLevelSequenceDataConverter> LevelSequenceDataConverter = MakeUnique<FGLTFLevelSequenceDataConverter>(*this);

	TUniquePtr<IGLTFActorConverter> ActorConverter = MakeUnique<FGLTFActorConverter>(*this);
	TUniquePtr<IGLTFComponentConverter> ComponentConverter = MakeUnique<FGLTFComponentConverter>(*this);
	TUniquePtr<IGLTFComponentSocketConverter> ComponentSocketConverter = MakeUnique<FGLTFComponentSocketConverter>(*this);
	TUniquePtr<IGLTFStaticSocketConverter> StaticSocketConverter = MakeUnique<FGLTFStaticSocketConverter>(*this);
	TUniquePtr<IGLTFSkeletalSocketConverter> SkeletalSocketConverter = MakeUnique<FGLTFSkeletalSocketConverter>(*this);
	TUniquePtr<IGLTFSkeletalBoneConverter> SkeletalBoneConverter = MakeUnique<FGLTFSkeletalBoneConverter>(*this);
	TUniquePtr<IGLTFSceneConverter> SceneConverter = MakeUnique<FGLTFSceneConverter>(*this);

	TUniquePtr<IGLTFCameraConverter> CameraConverter = MakeUnique<FGLTFCameraConverter>(*this);
	TUniquePtr<IGLTFLightConverter> LightConverter = MakeUnique<FGLTFLightConverter>(*this);
	TUniquePtr<IGLTFLightIESConverter> LightIESConverter = MakeUnique<FGLTFLightIESConverter>(*this);
	TUniquePtr<FGLTFLightIESInstanceConverter> LightIESInstanceConverter = MakeUnique<FGLTFLightIESInstanceConverter>(*this);
	TUniquePtr<IGLTFMaterialVariantConverter> MaterialVariantConverter = MakeUnique<FGLTFMaterialVariantConverter>(*this);

	//LightMap related:
	TUniquePtr<IGLTFTextureLightMapConverter> TextureLightMapConverter = MakeUnique<FGLTFTextureLightMapConverter>(*this);
	TUniquePtr<IGLTFLightMapConverter> LightMapConverter = MakeUnique<FGLTFLightMapConverter>(*this);

private:

	TMap<const UObject*, TArray<const UPropertyValue*>> ObjectVariants;
};

#undef UE_API
