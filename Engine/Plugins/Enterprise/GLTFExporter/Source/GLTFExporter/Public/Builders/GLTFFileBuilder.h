// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Builders/GLTFTaskBuilder.h"
#include "Builders/GLTFMemoryArchive.h"

#define UE_API GLTFEXPORTER_API

class FGLTFFileBuilder : public FGLTFTaskBuilder
{
public:

	UE_API FGLTFFileBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions = nullptr);

	UE_API FString AddExternalFile(const FString& DesiredURI, const TSharedPtr<FGLTFMemoryArchive>& Archive);

	UE_API void GetExternalFiles(TArray<FString>& OutFilePaths, const FString& DirPath = TEXT("")) const;

	UE_API const TMap<FString, TSharedPtr<FGLTFMemoryArchive>>& GetExternalArchives() const;

	UE_API bool WriteExternalFiles(const FString& DirPath, uint32 WriteFlags = 0);

private:

	TMap<FString, TSharedPtr<FGLTFMemoryArchive>> ExternalArchives;

	FString GetUniqueFileName(const FString& InFileName) const;

	static FString SanitizeFileName(const FString& InFileName);

	static FString EncodeURI(const FString& InFileName);

protected:

	UE_API bool SaveToFile(const FString& FilePath, const TArray64<uint8>& FileData, uint32 WriteFlags);
};

#undef UE_API
