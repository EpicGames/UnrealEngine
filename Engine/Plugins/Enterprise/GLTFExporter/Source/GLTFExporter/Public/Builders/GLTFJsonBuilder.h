// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Builders/GLTFFileBuilder.h"
#include "Json/GLTFJsonRoot.h"

#define UE_API GLTFEXPORTER_API

class FGLTFJsonBuilder : public FGLTFFileBuilder
{
public:

	FGLTFJsonScene*& DefaultScene;

	UE_API FGLTFJsonBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions = nullptr);

	UE_API void AddExtension(EGLTFJsonExtension Extension, bool bIsRequired = false);

	UE_API FGLTFJsonAccessor* AddAccessor();
	UE_API FGLTFJsonAnimation* AddAnimation();
	UE_API FGLTFJsonBuffer* AddBuffer();
	UE_API FGLTFJsonBufferView* AddBufferView();
	UE_API FGLTFJsonCamera* AddCamera();
	UE_API FGLTFJsonImage* AddImage();
	UE_API FGLTFJsonMaterial* AddMaterial();
	UE_API FGLTFJsonMesh* AddMesh();
	UE_API FGLTFJsonNode* AddNode();
	UE_API FGLTFJsonSampler* AddSampler();
	UE_API FGLTFJsonScene* AddScene();
	UE_API FGLTFJsonSkin* AddSkin();
	UE_API FGLTFJsonTexture* AddTexture();
	UE_API FGLTFJsonLight* AddLight();
	UE_API FGLTFJsonLightMap* AddLightMap();
	UE_API FGLTFJsonLightIES* AddLightIES();
	UE_API FGLTFJsonLightIESInstance* AddLightIESInstance();
	UE_API FGLTFJsonMaterialVariant* AddMaterialVariant();

	UE_API const FGLTFJsonRoot& GetRoot() const;

protected:

	UE_API bool WriteJsonArchive(FArchive& Archive);

private:

	static FString GetGeneratorString();
	static FString GetCopyrightString();

	FGLTFJsonRoot JsonRoot;

	void ValidateAndFixGLTFJson();
};

#undef UE_API
