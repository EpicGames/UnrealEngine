// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Builders/GLTFBuilder.h"

#define UE_API GLTFEXPORTER_API

#if WITH_EDITOR
class IMessageLogListing;
#endif

class FGLTFLogBuilder : public FGLTFBuilder
{
public:

	UE_API FGLTFLogBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions = nullptr);

	UE_API void LogSuggestion(const FString& Message);

	UE_API void LogWarning(const FString& Message);

	UE_API void LogError(const FString& Message);

	UE_API const TArray<FString>& GetLoggedSuggestions() const;

	UE_API const TArray<FString>& GetLoggedWarnings() const;

	UE_API const TArray<FString>& GetLoggedErrors() const;

	UE_API bool HasLoggedMessages() const;

	UE_API void OpenLog() const;

	UE_API void ClearLog();

private:

	enum class ELogLevel
	{
		Suggestion,
		Warning,
		Error,
	};

	void PrintToLog(ELogLevel Level, const FString& Message) const;

	TArray<FString> Suggestions;
	TArray<FString> Warnings;
	TArray<FString> Errors;

#if WITH_EDITOR
	TSharedPtr<IMessageLogListing> LogListing;
#endif
};

#undef UE_API
