// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/MemoryArchive.h"

class FGLTFMemoryArchive final : public FMemoryArchive, public TArray64<uint8>
{
public:

	FGLTFMemoryArchive()
	{
		FMemoryArchive::SetIsSaving(true);
	}

	virtual void Serialize(void* Data, int64 Length) override
	{
		check(Length >= 0);

		const int64 NumBytesToAdd = Offset + Length - Num();
		AddUninitialized(NumBytesToAdd);

		FMemory::Memcpy(GetData() + Offset, Data, Length);
		Offset += Length;
	}

	virtual int64 TotalSize() override
	{
		return Num();
	}
};
