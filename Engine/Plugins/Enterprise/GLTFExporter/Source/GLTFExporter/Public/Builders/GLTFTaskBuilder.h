// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Builders/GLTFLogBuilder.h"
#include "Tasks/GLTFDelayedTask.h"

#define UE_API GLTFEXPORTER_API

class FGLTFTaskBuilder : public FGLTFLogBuilder
{
public:

	UE_API FGLTFTaskBuilder(const FString& FileName, const UGLTFExportOptions* ExportOptions = nullptr);

	template <typename TaskType, typename... TaskArgTypes, typename = typename TEnableIf<TIsDerivedFrom<TaskType, FGLTFDelayedTask>::Value>::Type>
	bool ScheduleSlowTask(TaskArgTypes&&... Args)
	{
		return ScheduleSlowTask(MakeUnique<TaskType>(Forward<TaskArgTypes>(Args)...));
	}

	template <typename TaskType, typename = typename TEnableIf<TIsDerivedFrom<TaskType, FGLTFDelayedTask>::Value>::Type>
	bool ScheduleSlowTask(TUniquePtr<TaskType> Task)
	{
		return ScheduleSlowTask(TUniquePtr<FGLTFDelayedTask>(Task.Release()));
	}

	UE_API bool ScheduleSlowTask(TUniquePtr<FGLTFDelayedTask> Task);

	UE_API void ProcessSlowTasks(FFeedbackContext* Context = nullptr);

private:

	static UE_API FText GetPriorityMessageFormat(EGLTFTaskPriority Priority);

	int32 PriorityIndexLock;
	TMap<EGLTFTaskPriority, TArray<TUniquePtr<FGLTFDelayedTask>>> TasksByPriority;
	TMap<EGLTFTaskPriority, TArray<TUniquePtr<FGLTFDelayedTask>>> TasksByPrioritySecondary; //Adding to this if ProcessSlowTasks is in progress.
};

#undef UE_API
