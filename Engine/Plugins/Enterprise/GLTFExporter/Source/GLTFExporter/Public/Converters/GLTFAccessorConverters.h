// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Converters/GLTFConverter.h"
#include "Converters/GLTFBuilderContext.h"
#include "Converters/GLTFMeshSection.h"
#include "Converters/GLTFMeshAttributesArray.h"

#define UE_API GLTFEXPORTER_API

class FPositionVertexBuffer;
class FColorVertexBuffer;
class FStaticMeshVertexBuffer;
class FSkinWeightVertexBuffer;

typedef TGLTFConverter<FGLTFJsonAccessor*, const FGLTFMeshSection*, const FPositionVertexBuffer*> IGLTFPositionBufferConverter;
typedef TGLTFConverter<FGLTFJsonAccessor*, const FGLTFMeshSection*, const TMap<uint32, FVector3f>*> IGLTFPositionDeltaBufferConverter;
typedef TGLTFConverter<FGLTFJsonAccessor*, const FGLTFMeshSection*, const FColorVertexBuffer*> IGLTFColorBufferConverter;
typedef TGLTFConverter<FGLTFJsonAccessor*, const FGLTFMeshSection*, const FStaticMeshVertexBuffer*> IGLTFNormalBufferConverter;
typedef TGLTFConverter<FGLTFJsonAccessor*, const FGLTFMeshSection*, const TMap<uint32, FVector3f>*, bool> IGLTFNormalDeltaBufferConverter;
typedef TGLTFConverter<FGLTFJsonAccessor*, const FGLTFMeshSection*, const FStaticMeshVertexBuffer*> IGLTFTangentBufferConverter;
typedef TGLTFConverter<FGLTFJsonAccessor*, const FGLTFMeshSection*, const FStaticMeshVertexBuffer*, uint32> IGLTFUVBufferConverter;
typedef TGLTFConverter<FGLTFJsonAccessor*, const FGLTFMeshSection*, const FSkinWeightVertexBuffer*, uint32> IGLTFBoneIndexBufferConverter;
typedef TGLTFConverter<FGLTFJsonAccessor*, const FGLTFMeshSection*, const FSkinWeightVertexBuffer*, uint32> IGLTFBoneWeightBufferConverter;
typedef TGLTFConverter<FGLTFJsonAccessor*, const FGLTFMeshSection*> IGLTFIndexBufferConverter;

typedef TGLTFConverter<FGLTFJsonAccessor*, FGLTFPositionArray> IGLTFPositionBufferConverterRaw;
typedef TGLTFConverter<FGLTFJsonAccessor*, FGLTFIndexArray, FString> IGLTFIndexBufferConverterRaw;
typedef TGLTFConverter<FGLTFJsonAccessor*, FGLTFNormalArray, bool> IGLTFNormalBufferConverterRaw;
typedef TGLTFConverter<FGLTFJsonAccessor*, FGLTFUVArray> IGLTFUVBufferConverterRaw;
typedef TGLTFConverter<FGLTFJsonAccessor*, FGLTFColorArray> IGLTFColorBufferConverterRaw;
typedef TGLTFConverter<FGLTFJsonAccessor*, FGLTFTangentArray> IGLTFTangentBufferConverterRaw;
typedef TGLTFConverter<FGLTFJsonAccessor*, FGLTFJointInfluenceArray> IGLTFBoneIndexBufferConverterRaw;
typedef TGLTFConverter<FGLTFJsonAccessor*, FGLTFJointWeightArray> IGLTFBoneWeightBufferConverterRaw;


class FGLTFPositionBufferConverter : public FGLTFBuilderContext, public IGLTFPositionBufferConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonAccessor* Convert(const FGLTFMeshSection* MeshSection, const FPositionVertexBuffer* VertexBuffer) override;
};

class FGLTFPositionDeltaBufferConverter : public FGLTFBuilderContext, public IGLTFPositionDeltaBufferConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonAccessor* Convert(const FGLTFMeshSection* MeshSection, const TMap<uint32, FVector3f>* TargetPositionDeltas) override;
};

class FGLTFColorBufferConverter : public FGLTFBuilderContext, public IGLTFColorBufferConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonAccessor* Convert(const FGLTFMeshSection* MeshSection, const FColorVertexBuffer* VertexBuffer) override;
};

class FGLTFNormalBufferConverter : public FGLTFBuilderContext, public IGLTFNormalBufferConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonAccessor* Convert(const FGLTFMeshSection* MeshSection, const FStaticMeshVertexBuffer* VertexBuffer) override;

private:

	template <typename DestinationType, typename SourceType>
	FGLTFJsonBufferView* ConvertBufferView(const FGLTFMeshSection* MeshSection, const void* TangentData);
};

class FGLTFNormalDeltaBufferConverter : public FGLTFBuilderContext, public IGLTFNormalDeltaBufferConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonAccessor* Convert(const FGLTFMeshSection* MeshSection, const TMap<uint32, FVector3f>* TargetNormalDeltas, bool bHighPrecision) override;

private:

	template <typename DestinationType, typename SourceType>
	FGLTFJsonBufferView* ConvertBufferView(const FGLTFMeshSection* MeshSection, const TMap<uint32, FVector3f>* TargetNormalDeltas);
};

class FGLTFTangentBufferConverter : public FGLTFBuilderContext, public IGLTFTangentBufferConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonAccessor* Convert(const FGLTFMeshSection* MeshSection, const FStaticMeshVertexBuffer* VertexBuffer) override;

private:

	template <typename DestinationType, typename SourceType>
	FGLTFJsonBufferView* ConvertBufferView(const FGLTFMeshSection* MeshSection, const void* TangentData);
};

class FGLTFUVBufferConverter : public FGLTFBuilderContext, public IGLTFUVBufferConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonAccessor* Convert(const FGLTFMeshSection* MeshSection, const FStaticMeshVertexBuffer* VertexBuffer, uint32 UVIndex) override;

private:

	template <typename SourceType>
	FGLTFJsonAccessor* Convert(const FGLTFMeshSection* MeshSection, const FStaticMeshVertexBuffer* VertexBuffer, uint32 UVIndex, const uint8* SourceData) const;
};

class FGLTFBoneIndexBufferConverter : public FGLTFBuilderContext, public IGLTFBoneIndexBufferConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonAccessor* Convert(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, uint32 InfluenceOffset) override;

private:

	template <typename DestinationType>
	FGLTFJsonAccessor* Convert(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, uint32 InfluenceOffset, const uint8* SourceData) const;

	template <typename DestinationType, typename SourceType>
	FGLTFJsonAccessor* Convert(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, uint32 InfluenceOffset, const uint8* SourceData) const;

	template <typename DestinationType, typename SourceType, typename CallbackType>
	FGLTFJsonAccessor* Convert(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, uint32 InfluenceOffset, const uint8* SourceData, CallbackType GetVertexInfluenceOffsetCount) const;
};

class FGLTFBoneWeightBufferConverter : public FGLTFBuilderContext, public IGLTFBoneWeightBufferConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonAccessor* Convert(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, uint32 InfluenceOffset) override;

private:

	template <typename BoneIndexType>
	FGLTFJsonAccessor* Convert(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, uint32 InfluenceOffset, const uint8* SourceData) const;

	template <typename BoneIndexType, typename CallbackType>
	FGLTFJsonAccessor* Convert(const FGLTFMeshSection* MeshSection, const FSkinWeightVertexBuffer* VertexBuffer, uint32 InfluenceOffset, const uint8* SourceData, CallbackType GetVertexInfluenceOffsetCount) const;
};

class FGLTFIndexBufferConverter : public FGLTFBuilderContext, public IGLTFIndexBufferConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonAccessor* Convert(const FGLTFMeshSection* MeshSection) override;
};


class FGLTFPositionBufferConverterRaw : public FGLTFBuilderContext, public IGLTFPositionBufferConverterRaw
{
public:
	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:
	UE_API virtual FGLTFJsonAccessor* Convert(FGLTFPositionArray VertexBuffer) override;
};

class FGLTFIndexBufferConverterRaw : public FGLTFBuilderContext, public IGLTFIndexBufferConverterRaw
{
public:
	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:
	UE_API virtual FGLTFJsonAccessor* Convert(FGLTFIndexArray IndexBuffer, FString MeshName) override;
};

class FGLTFNormalBufferConverterRaw : public FGLTFBuilderContext, public IGLTFNormalBufferConverterRaw
{
public:
	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:
	UE_API virtual FGLTFJsonAccessor* Convert(FGLTFNormalArray NormalsSource, bool bNormalize) override;
};

class FGLTFUVBufferConverterRaw : public FGLTFBuilderContext, public IGLTFUVBufferConverterRaw
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:
	UE_API virtual FGLTFJsonAccessor* Convert(FGLTFUVArray UVSource) override;
};

class FGLTFColorBufferConverterRaw : public FGLTFBuilderContext, public IGLTFColorBufferConverterRaw
{
public:
	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:
	UE_API virtual FGLTFJsonAccessor* Convert(FGLTFColorArray VertexColorBuffer) override;
};

class FGLTFTangentBufferConverterRaw : public FGLTFBuilderContext, public IGLTFTangentBufferConverterRaw
{
public:
	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:
	UE_API virtual FGLTFJsonAccessor* Convert(FGLTFTangentArray TangentSource) override;
};

class FGLTFBoneIndexBufferConverterRaw : public FGLTFBuilderContext, public IGLTFBoneIndexBufferConverterRaw
{
public:
	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:
	UE_API virtual FGLTFJsonAccessor* Convert(FGLTFJointInfluenceArray Joints) override;
};

class FGLTFBoneWeightBufferConverterRaw : public FGLTFBuilderContext, public IGLTFBoneWeightBufferConverterRaw
{
public:
	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:
	UE_API virtual FGLTFJsonAccessor* Convert(FGLTFJointWeightArray Weights) override;
};

#undef UE_API
