// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Converters/GLTFConverter.h"
#include "Converters/GLTFBuilderContext.h"

#define UE_API GLTFEXPORTER_API

class ALevelSequenceActor;
class UAnimSequence;
class ULevel;
class ULevelSequence;
class USkeletalMesh;
class USkeletalMeshComponent;

typedef TGLTFConverter<FGLTFJsonAnimation*, FGLTFJsonNode*, const USkeletalMesh*, const UAnimSequence*> IGLTFAnimationConverter;
typedef TGLTFConverter<FGLTFJsonAnimation*, FGLTFJsonNode*, const USkeletalMeshComponent*> IGLTFAnimationDataConverter;
typedef TGLTFConverter<FGLTFJsonAnimation*, const ULevel*, const ULevelSequence*> IGLTFLevelSequenceConverter;
typedef TGLTFConverter<FGLTFJsonAnimation*, const ALevelSequenceActor*> IGLTFLevelSequenceDataConverter;

class FGLTFAnimationConverter : public FGLTFBuilderContext, public IGLTFAnimationConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonAnimation* Convert(FGLTFJsonNode* RootNode, const USkeletalMesh* SkeletalMesh, const UAnimSequence* AnimSequence) override;
};

class FGLTFAnimationDataConverter : public FGLTFBuilderContext, public IGLTFAnimationDataConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonAnimation* Convert(FGLTFJsonNode* RootNode, const USkeletalMeshComponent* SkeletalMeshComponent) override;
};

class FGLTFLevelSequenceConverter : public FGLTFBuilderContext, public IGLTFLevelSequenceConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonAnimation* Convert(const ULevel* Level, const ULevelSequence*) override;
};

class FGLTFLevelSequenceDataConverter : public FGLTFBuilderContext, public IGLTFLevelSequenceDataConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonAnimation* Convert(const ALevelSequenceActor* LevelSequenceActor) override;
};

#undef UE_API
