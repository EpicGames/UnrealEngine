// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Converters/GLTFConverter.h"
#include "Converters/GLTFBuilderContext.h"

#define UE_API GLTFEXPORTER_API

class UCameraComponent;

typedef TGLTFConverter<FGLTFJsonCamera*, const UCameraComponent*> IGLTFCameraConverter;

class FGLTFCameraConverter : public FGLTFBuilderContext, public IGLTFCameraConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonCamera* Convert(const UCameraComponent* CameraComponent) override;
};

#undef UE_API
