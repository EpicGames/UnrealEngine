// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

template <typename OutputType, typename... InputTypes>
class TGLTFConverter
{
	typedef TTuple<InputTypes...> InputTupleType;

public:

	virtual ~TGLTFConverter() = default;

	OutputType Get(InputTypes... Inputs)
	{
		Sanitize(Inputs...);
		InputTupleType InputTuple(MoveTemp(Inputs)...);
		const uint32 InputHash = GetTypeHash(InputTuple);

		if (OutputType* CachedOutput = CachedOutputs.FindByHash(InputHash, InputTuple))
		{
			return *CachedOutput;
		}

		return {};
	}

	OutputType GetOrAdd(InputTypes... Inputs)
	{
		Sanitize(Inputs...);
		InputTupleType InputTuple(MoveTemp(Inputs)...);
		const uint32 InputHash = GetTypeHash(InputTuple);

		if (OutputType* CachedOutput = CachedOutputs.FindByHash(InputHash, InputTuple))
		{
			return *CachedOutput;
		}

		OutputType NewOutput = InputTuple.ApplyAfter(&TGLTFConverter::Convert, this);
		CachedOutputs.AddByHash(InputHash, MoveTemp(InputTuple), NewOutput);
		return NewOutput;
	}

	const TMap<InputTupleType, OutputType>& GetOutputs() const
	{
		return CachedOutputs;
	}

protected:

	virtual void Sanitize(InputTypes&... Inputs) { }

	virtual OutputType Convert(InputTypes... Inputs) = 0;

private:

	TMap<InputTupleType, OutputType> CachedOutputs;
};
