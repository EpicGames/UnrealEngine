// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Converters/GLTFConverter.h"
#include "Converters/GLTFBuilderContext.h"
#include "Converters/GLTFSuperfluous.h"
#include "Converters/GLTFSharedArray.h"

#define UE_API GLTFEXPORTER_API

typedef TGLTFConverter<FGLTFJsonImage*, TGLTFSuperfluous<FString>, bool, FIntPoint, TGLTFSharedArray<FColor>> IGLTFImageConverter;

class FGLTFImageConverter : public FGLTFBuilderContext, public IGLTFImageConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonImage* Convert(TGLTFSuperfluous<FString> Name, bool bIgnoreAlpha, FIntPoint Size, TGLTFSharedArray<FColor> Pixels) override;

private:

	EGLTFJsonMimeType GetMimeType(const FColor* Pixels, FIntPoint Size, bool bIgnoreAlpha) const;
};

#undef UE_API
