// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Converters/GLTFConverter.h"
#include "Converters/GLTFBuilderContext.h"

#define UE_API GLTFEXPORTER_API

class ULightComponent;

typedef TGLTFConverter<FGLTFJsonLight*, const ULightComponent*> IGLTFLightConverter;
typedef TGLTFConverter<FGLTFJsonLightIES*, const ULightComponent*> IGLTFLightIESConverter;
typedef TGLTFConverter<FGLTFJsonLightIESInstance*, const ULightComponent*> IGLTFLightIESInstanceConverter;


class FGLTFLightConverter : public FGLTFBuilderContext, public IGLTFLightConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonLight* Convert(const ULightComponent* LightComponent) override;
};

//IES export is not supported runtime, as we don't have access to the AssetImportData (which stores the SourceFileContent for the IES file).
class FGLTFLightIESConverter : public FGLTFBuilderContext, public IGLTFLightIESConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonLightIES* Convert(const ULightComponent* LightComponent) override;
};

//IES export is not supported runtime, as we don't have access to the AssetImportData (which stores the SourceFileContent for the IES file).
class FGLTFLightIESInstanceConverter : public FGLTFBuilderContext, public IGLTFLightIESInstanceConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonLightIESInstance* Convert(const ULightComponent* LightComponent) override;
};

#undef UE_API
