// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Converters/GLTFConverter.h"
#include "Converters/GLTFBuilderContext.h"

#define UE_API GLTFEXPORTER_API

class UStaticMeshComponent;

typedef TGLTFConverter<FGLTFJsonLightMap*, const UStaticMeshComponent*> IGLTFLightMapConverter;

class FGLTFLightMapConverter : public FGLTFBuilderContext, public IGLTFLightMapConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonLightMap* Convert(const UStaticMeshComponent* StaticMeshComponent) override;
};

#undef UE_API
