// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UMaterialInterface;

// TODO: should we add GLTFEXPORTER_API ?
class FGLTFMaterialArray : public TArray<const UMaterialInterface*>
{
public:

	using TArray::TArray;
	using TArray::operator=;

	friend uint32 GetTypeHash(const FGLTFMaterialArray& MaterialArray)
	{
		uint32 Hash = GetTypeHash(MaterialArray.Num());
		for (const UMaterialInterface* Material : MaterialArray)
		{
			Hash = HashCombine(Hash, GetTypeHash(Material));
		}
		return Hash;
	}
};
