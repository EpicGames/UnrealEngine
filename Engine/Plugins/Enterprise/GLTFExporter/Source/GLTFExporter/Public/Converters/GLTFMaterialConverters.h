// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Converters/GLTFConverter.h"
#include "Converters/GLTFBuilderContext.h"
#include "Converters/GLTFMeshData.h"
#include "Converters/GLTFMeshAttributesArray.h"
#include "Converters/GLTFUVOverlapChecker.h"
#include "Converters/GLTFUVBoundsCalculator.h"
#include "Converters/GLTFUVDegenerateChecker.h"

#define UE_API GLTFEXPORTER_API

typedef TGLTFConverter<FGLTFJsonMaterial*, const UMaterialInterface*, const FGLTFMeshData*, FGLTFIndexArray> IGLTFMaterialConverter;

class FGLTFMaterialConverter : public FGLTFBuilderContext, public IGLTFMaterialConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual void Sanitize(const UMaterialInterface*& Material, const FGLTFMeshData*& MeshData, FGLTFIndexArray& SectionIndices) override;

	UE_API virtual FGLTFJsonMaterial* Convert(const UMaterialInterface* Material, const FGLTFMeshData* MeshData, FGLTFIndexArray SectionIndices) override;

private:

	FGLTFUVOverlapChecker UVOverlapChecker;
	FGLTFUVBoundsCalculator UVBoundsCalculator;
	FGLTFUVDegenerateChecker UVDegenerateChecker;
};

#undef UE_API
