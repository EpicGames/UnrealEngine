// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Json/GLTFJsonMesh.h"
#include "Converters/GLTFConverter.h"
#include "Converters/GLTFBuilderContext.h"

#define UE_API GLTFEXPORTER_API

class UVariant;
class UPropertyValueMaterial;

typedef TGLTFConverter<FGLTFJsonMaterialVariant*, const UVariant*> IGLTFMaterialVariantConverter;

class FGLTFMaterialVariantConverter : public FGLTFBuilderContext, public IGLTFMaterialVariantConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonMaterialVariant* Convert(const UVariant* Variant) override;

private:

	bool TryParseMaterialProperty(FGLTFJsonPrimitive*& OutPrimitive, FGLTFJsonMaterial*& OutMaterial, const UPropertyValueMaterial* Property) const;
};

#undef UE_API
