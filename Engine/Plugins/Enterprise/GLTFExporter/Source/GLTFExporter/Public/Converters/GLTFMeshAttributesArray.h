// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BoneIndices.h"
#include "Utilities/MeshAttributesArray.h"

typedef FMeshAttributesArray<int32> FGLTFIndexArray;
typedef FMeshAttributesArray<FVector3f> FGLTFPositionArray;
typedef FMeshAttributesArray<FVector3f> FGLTFNormalArray;
typedef FMeshAttributesArray<FVector4f> FGLTFTangentArray;
typedef FMeshAttributesArray<FVector2f> FGLTFUVArray;
typedef FMeshAttributesArray<FColor> FGLTFColorArray;

typedef FMeshAttributesArray<UE::Math::TIntVector4<FBoneIndexType>> FGLTFJointInfluenceArray;
typedef FMeshAttributesArray<UE::Math::TIntVector4<uint16>> FGLTFJointWeightArray;