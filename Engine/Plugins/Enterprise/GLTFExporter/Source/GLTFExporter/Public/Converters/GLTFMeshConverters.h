// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Converters/GLTFConverter.h"
#include "Converters/GLTFBuilderContext.h"
#include "Converters/GLTFMeshSectionConverters.h"
#include "Converters/GLTFMaterialArray.h"

#define UE_API GLTFEXPORTER_API

class UStaticMesh;
class UStaticMeshComponent;
class USkeletalMesh;
class USkeletalMeshComponent;
class USplineMeshComponent;
class ULandscapeComponent;

typedef TGLTFConverter<FGLTFJsonMesh*, const UStaticMesh*, const UStaticMeshComponent*, FGLTFMaterialArray, int32> IGLTFStaticMeshConverter;
typedef TGLTFConverter<FGLTFJsonMesh*, const USkeletalMesh*, const USkeletalMeshComponent*, FGLTFMaterialArray, int32> IGLTFSkeletalMeshConverter;
typedef TGLTFConverter<FGLTFJsonMesh*, const UStaticMesh*, const USplineMeshComponent*, FGLTFMaterialArray, int32> IGLTFSplineMeshConverter;
typedef TGLTFConverter<FGLTFJsonMesh*, const ULandscapeComponent*, const UMaterialInterface*> IGLTFLandscapeMeshConverter;

class FGLTFStaticMeshConverter : public FGLTFBuilderContext, public IGLTFStaticMeshConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual void Sanitize(const UStaticMesh*& StaticMesh, const UStaticMeshComponent*& StaticMeshComponent, FGLTFMaterialArray& Materials, int32& LODIndex) override;

	UE_API virtual FGLTFJsonMesh* Convert(const UStaticMesh* StaticMesh, const UStaticMeshComponent* StaticMeshComponent, FGLTFMaterialArray Materials, int32 LODIndex) override;

private:

	FGLTFStaticMeshSectionConverter MeshSectionConverter;
};

class FGLTFSkeletalMeshConverter : public FGLTFBuilderContext, public IGLTFSkeletalMeshConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual void Sanitize(const USkeletalMesh*& SkeletalMesh, const USkeletalMeshComponent*& SkeletalMeshComponent, FGLTFMaterialArray& Materials, int32& LODIndex) override;

	UE_API virtual FGLTFJsonMesh* Convert(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent, FGLTFMaterialArray Materials, int32 LODIndex) override;

private:

	FGLTFSkeletalMeshSectionConverter MeshSectionConverter;
};

class FGLTFSplineMeshConverter : public FGLTFBuilderContext, public IGLTFSplineMeshConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual void Sanitize(const UStaticMesh*& StaticMesh, const USplineMeshComponent*& SplineMeshComponent, FGLTFMaterialArray& Materials, int32& LODIndex) override;

	UE_API virtual FGLTFJsonMesh* Convert(const UStaticMesh* StaticMesh, const USplineMeshComponent* SplineMeshComponent, FGLTFMaterialArray Materials, int32 LODIndex) override;

private:

	FGLTFStaticMeshSectionConverter MeshSectionConverter;
};

class FGLTFLandscapeMeshConverter : public FGLTFBuilderContext, public IGLTFLandscapeMeshConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual void Sanitize(const ULandscapeComponent*& LandscapeComponent, const UMaterialInterface*& LandscapeMaterial) override;

	UE_API virtual FGLTFJsonMesh* Convert(const ULandscapeComponent* LandscapeComponent, const UMaterialInterface* LandscapeMaterial) override;

private:
};

#undef UE_API
