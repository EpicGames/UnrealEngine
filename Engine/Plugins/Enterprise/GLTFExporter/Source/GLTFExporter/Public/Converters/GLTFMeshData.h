// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshDescription.h"
#include "SceneTypes.h"

#if WITH_EDITOR
// TODO: remove dependency to MaterialBaking in public header
#include "MaterialBakingStructures.h"
#endif

class FLightmapResourceCluster;
class UStaticMesh;
class UStaticMeshComponent;
class USkeletalMesh;
class USkeletalMeshComponent;

struct FGLTFMeshData
{
	GLTFEXPORTER_API FGLTFMeshData(const UStaticMesh* StaticMesh, const UStaticMeshComponent* StaticMeshComponent, int32 LODIndex);
	GLTFEXPORTER_API FGLTFMeshData(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent, int32 LODIndex);

	GLTFEXPORTER_API const FGLTFMeshData* GetParent() const;

	// TODO: find a better name for referencing the mesh-only data (no component)
	const FGLTFMeshData* Parent;

	FString Name;
	int32 LODIndex;

#if WITH_EDITOR
	FPrimitiveData PrimitiveData;
	FMeshDescription Description;

	FLightMapRef LightMap;
	const FLightmapResourceCluster* LightMapResourceCluster;
	int32 LightMapTexCoord;

	int32 BakeUsingTexCoord;
#endif
};
