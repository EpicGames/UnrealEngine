// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Converters/GLTFConverter.h"
#include "Converters/GLTFMeshData.h"
#include "Converters/GLTFBuilderContext.h"

typedef TGLTFConverter<const FGLTFMeshData*, const UStaticMesh*, const UStaticMeshComponent*, int32> IGLTFStaticMeshDataConverter;
typedef TGLTFConverter<const FGLTFMeshData*, const USkeletalMesh*, const USkeletalMeshComponent*, int32> IGLTFSkeletalMeshDataConverter;

template <typename MeshType, typename MeshComponentType>
class TGLTFMeshDataConverter : public FGLTFBuilderContext, public TGLTFConverter<const FGLTFMeshData*, const MeshType*, const MeshComponentType*, int32>
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

	TGLTFMeshDataConverter(TGLTFMeshDataConverter&&) = default;
	TGLTFMeshDataConverter& operator=(TGLTFMeshDataConverter&&) = default;

	TGLTFMeshDataConverter(const TGLTFMeshDataConverter&) = delete;
	TGLTFMeshDataConverter& operator=(const TGLTFMeshDataConverter&) = delete;

protected:

	virtual void Sanitize(const MeshType*& Mesh, const MeshComponentType*& MeshComponent, int32& LODIndex) override;

	virtual const FGLTFMeshData* Convert(const MeshType* Mesh, const MeshComponentType* MeshComponent, int32 LODIndex) override;

private:

	// TODO: standardize converters that are responsible for deleting their output
	TArray<TUniquePtr<FGLTFMeshData>> Outputs;
};

typedef TGLTFMeshDataConverter<UStaticMesh, UStaticMeshComponent> FGLTFStaticMeshDataConverter;
typedef TGLTFMeshDataConverter<USkeletalMesh, USkeletalMeshComponent> FGLTFSkeletalMeshDataConverter;
