// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BoneIndices.h"
#include "Converters/GLTFMeshAttributesArray.h"

#define UE_API GLTFEXPORTER_API

class UStaticMesh;
class USkeletalMesh;
struct FSkelMeshRenderSection;
struct FStaticMeshSection;

struct FGLTFMeshSection
{
	UE_API FGLTFMeshSection(const UStaticMesh* Mesh, int32 LODIndex, const FGLTFIndexArray& SectionIndices);
	UE_API FGLTFMeshSection(const USkeletalMesh* Mesh, int32 LODIndex, const FGLTFIndexArray& SectionIndices);

	FString Name;
	FGLTFIndexArray SectionIndices;

	TArray<uint32> IndexMap;
	TArray<uint32> IndexBuffer;

	TArray<TArray<FBoneIndexType>> BoneMaps;
	TArray<uint32> BoneMapLookup;
	FBoneIndexType MaxBoneIndex;

	UE_API FString ToString() const;

private:

	template <typename IndexType, typename SectionArrayType>
	void Init(const SectionArrayType& Sections, const IndexType* SourceData);

	static uint32 GetIndexOffset(const FStaticMeshSection& Section);
	static uint32 GetIndexOffset(const FSkelMeshRenderSection& Section);

	static const TArray<FBoneIndexType>& GetBoneMap(const FStaticMeshSection& Section);
	static const TArray<FBoneIndexType>& GetBoneMap(const FSkelMeshRenderSection& Section);
};

#undef UE_API
