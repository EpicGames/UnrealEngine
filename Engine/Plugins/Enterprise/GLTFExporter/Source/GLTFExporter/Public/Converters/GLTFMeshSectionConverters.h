// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Converters/GLTFConverter.h"
#include "Converters/GLTFMeshSection.h"
#include "Converters/GLTFMeshAttributesArray.h"

template <typename MeshType>
class TGLTFMeshSectionConverter : public TGLTFConverter<const FGLTFMeshSection*, const MeshType*, int32, FGLTFIndexArray>
{
public:

	TGLTFMeshSectionConverter() = default;
	TGLTFMeshSectionConverter(TGLTFMeshSectionConverter&&) = default;
	TGLTFMeshSectionConverter& operator=(TGLTFMeshSectionConverter&&) = default;

	TGLTFMeshSectionConverter(const TGLTFMeshSectionConverter&) = delete;
	TGLTFMeshSectionConverter& operator=(const TGLTFMeshSectionConverter&) = delete;

protected:

	virtual const FGLTFMeshSection* Convert(const MeshType* Mesh, int32 LODIndex, FGLTFIndexArray SectionIndices) override
	{
		return Outputs.Add_GetRef(MakeUnique<FGLTFMeshSection>(Mesh, LODIndex, SectionIndices)).Get();
	}

private:

	TArray<TUniquePtr<FGLTFMeshSection>> Outputs;
};

typedef TGLTFMeshSectionConverter<UStaticMesh> FGLTFStaticMeshSectionConverter;
typedef TGLTFMeshSectionConverter<USkeletalMesh> FGLTFSkeletalMeshSectionConverter;
