// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Converters/GLTFConverter.h"
#include "Converters/GLTFBuilderContext.h"

#define UE_API GLTFEXPORTER_API

class AActor;
class USceneComponent;
class UStaticMesh;
class USkeletalMesh;
//struct FGLTFJsonNode;

typedef TGLTFConverter<FGLTFJsonNode*, const AActor*> IGLTFActorConverter;
typedef TGLTFConverter<FGLTFJsonNode*, const USceneComponent*> IGLTFComponentConverter;
typedef TGLTFConverter<FGLTFJsonNode*, const USceneComponent*, FName> IGLTFComponentSocketConverter;
typedef TGLTFConverter<FGLTFJsonNode*, FGLTFJsonNode*, const UStaticMesh*, FName> IGLTFStaticSocketConverter;
typedef TGLTFConverter<FGLTFJsonNode*, FGLTFJsonNode*, const USkeletalMesh*, FName> IGLTFSkeletalSocketConverter;
typedef TGLTFConverter<FGLTFJsonNode*, FGLTFJsonNode*, const USkeletalMesh*, int32> IGLTFSkeletalBoneConverter;

class FGLTFActorConverter : public FGLTFBuilderContext, public IGLTFActorConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonNode* Convert(const AActor* Actor) override;
};

class FGLTFComponentConverter : public FGLTFBuilderContext, public IGLTFComponentConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonNode* Convert(const USceneComponent* SceneComponent) override;

	UE_API void ConvertComponentSpecialization(const USceneComponent* SceneComponent, const AActor* Owner, FGLTFJsonNode* Node);
};

class FGLTFComponentSocketConverter : public FGLTFBuilderContext, public IGLTFComponentSocketConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonNode* Convert(const USceneComponent* SceneComponent, FName SocketName) override;
};

class FGLTFStaticSocketConverter : public FGLTFBuilderContext, public IGLTFStaticSocketConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonNode* Convert(FGLTFJsonNode* RootNode, const UStaticMesh* StaticMesh, FName SocketName) override;
};

class FGLTFSkeletalSocketConverter : public FGLTFBuilderContext, public IGLTFSkeletalSocketConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonNode* Convert(FGLTFJsonNode* RootNode, const USkeletalMesh* SkeletalMesh, FName SocketName) override;
};

class FGLTFSkeletalBoneConverter : public FGLTFBuilderContext, public IGLTFSkeletalBoneConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonNode* Convert(FGLTFJsonNode* RootNode, const USkeletalMesh* SkeletalMesh, int32 BoneIndex) override;
};

#undef UE_API
