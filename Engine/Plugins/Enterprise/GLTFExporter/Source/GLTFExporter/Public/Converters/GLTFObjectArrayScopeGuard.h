// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"

template <typename ObjectType = UObject, typename = typename TEnableIf<TIsDerivedFrom<ObjectType, UObject>::Value>::Type>
class FGLTFObjectArrayScopeGuard : public FGCObject, public TArray<TObjectPtr<ObjectType>>
{
public:

	FGLTFObjectArrayScopeGuard()
	{
	}

	using TArray<TObjectPtr<ObjectType>>::TArray;

	/** Non-copyable */
	FGLTFObjectArrayScopeGuard(const FGLTFObjectArrayScopeGuard&) = delete;
	FGLTFObjectArrayScopeGuard& operator=(const FGLTFObjectArrayScopeGuard&) = delete;

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObjects<ObjectType>(*this);
	}

	virtual FString GetReferencerName() const override
	{
		return TEXT("FGLTFObjectArrayScopeGuard");
	}
};
