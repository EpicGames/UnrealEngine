// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Converters/GLTFConverter.h"
#include "Converters/GLTFBuilderContext.h"
#include "Engine/TextureDefines.h"

#define UE_API GLTFEXPORTER_API

typedef TGLTFConverter<FGLTFJsonSampler*, TextureAddress, TextureAddress, TextureFilter, TextureGroup> IGLTFSamplerConverter;

class FGLTFSamplerConverter : public FGLTFBuilderContext, public IGLTFSamplerConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual void Sanitize(TextureAddress& AddressX, TextureAddress& AddressY, TextureFilter& Filter, TextureGroup& LODGroup) override;

	UE_API virtual FGLTFJsonSampler* Convert(TextureAddress AddressX, TextureAddress AddressY, TextureFilter Filter, TextureGroup LODGroup) override;
};

#undef UE_API
