// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Converters/GLTFConverter.h"
#include "Converters/GLTFBuilderContext.h"

#define UE_API GLTFEXPORTER_API

typedef TGLTFConverter<FGLTFJsonScene*, const UWorld*> IGLTFSceneConverter;

class FGLTFSceneConverter : public FGLTFBuilderContext, public IGLTFSceneConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonScene* Convert(const UWorld* Level) override;

	UE_API void MakeSkinnedMeshesRoot(FGLTFJsonScene* Scene);
	UE_API void MakeSkinnedMeshesRoot(FGLTFJsonNode* Node, bool bIsRootNode, FGLTFJsonScene* Scene);
};

#undef UE_API
