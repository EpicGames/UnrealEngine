// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/Requires.h"

#include <type_traits>

template <typename ElementType, typename ArrayType = TArray<ElementType>, ESPMode Mode = ESPMode::ThreadSafe>
class TGLTFSharedArray : public TSharedRef<ArrayType, Mode>
{
public:

	TGLTFSharedArray()
		: TSharedRef<ArrayType, Mode>(MakeShared<ArrayType>())
	{
	}

	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ArrayType*>)
	>
	TGLTFSharedArray(const TSharedRef<OtherType, Mode>& SharedRef)
	: TSharedRef<ArrayType, Mode>(SharedRef)
	{
	}

	using TSharedRef<ArrayType, Mode>::TSharedRef;
	using TSharedRef<ArrayType, Mode>::operator=;

	bool operator==(const TGLTFSharedArray& Other) const
	{
		return this->Get() == Other.Get();
	}

	bool operator!=(const TGLTFSharedArray& Other) const
	{
		return this->Get() != Other.Get();
	}

	friend uint32 GetTypeHash(const TGLTFSharedArray& SharedArray)
	{
		const ArrayType& Array = SharedArray.Get();
		uint32 Hash = GetTypeHash(Array.Num());

		for (const auto& Element : Array)
		{
			Hash = HashCombine(Hash, GetTypeHash(Element));
		}

		return Hash;
	}
};
