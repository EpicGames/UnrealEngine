// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Converters/GLTFConverter.h"
#include "Converters/GLTFBuilderContext.h"

#define UE_API GLTFEXPORTER_API

class USkeletalMesh;

typedef TGLTFConverter<FGLTFJsonSkin*, FGLTFJsonNode*, const USkeletalMesh*> IGLTFSkinConverter;

class FGLTFSkinConverter : public FGLTFBuilderContext, public IGLTFSkinConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonSkin* Convert(FGLTFJsonNode* RootNode, const USkeletalMesh* SkeletalMesh) override;
};

#undef UE_API
