// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

template <typename Type>
class TGLTFSuperfluous : public Type
{
public:

	TGLTFSuperfluous(const Type& Other)
		: Type(Other)
	{
	}

	using Type::Type;
	using Type::operator=;

	bool operator==(const TGLTFSuperfluous& Other) const
	{
		return true;
	}

	bool operator!=(const TGLTFSuperfluous& Other) const
	{
		return false;
	}

	friend uint32 GetTypeHash(const TGLTFSuperfluous& Superfluous)
	{
		return 0;
	}
};
