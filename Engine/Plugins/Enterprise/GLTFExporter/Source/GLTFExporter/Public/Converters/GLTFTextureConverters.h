// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Converters/GLTFConverter.h"
#include "Converters/GLTFBuilderContext.h"

#include "Engine/TextureDefines.h"

#define UE_API GLTFEXPORTER_API

class UTexture2D;
class UTextureRenderTarget2D;
class ULightMapTexture2D;

// TODO: generalize parameter bToSRGB to SamplerType (including normalmap unpacking)

typedef TGLTFConverter<FGLTFJsonTexture*, const UTexture2D*, bool, TextureAddress, TextureAddress> IGLTFTexture2DConverter;
typedef TGLTFConverter<FGLTFJsonTexture*, const UTextureRenderTarget2D*, bool> IGLTFTextureRenderTarget2DConverter;
typedef TGLTFConverter<FGLTFJsonTexture*, const ULightMapTexture2D*> IGLTFTextureLightMapConverter;

class FGLTFTexture2DConverter : public FGLTFBuilderContext, public IGLTFTexture2DConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonTexture* Convert(const UTexture2D* Texture2D, bool bToSRGB, TextureAddress WrapS, TextureAddress WrapT) override;
};

class FGLTFTextureRenderTarget2DConverter : public FGLTFBuilderContext, public IGLTFTextureRenderTarget2DConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonTexture* Convert(const UTextureRenderTarget2D* RenderTarget2D, bool bToSRGB) override;
};

class FGLTFTextureLightMapConverter : public FGLTFBuilderContext, public IGLTFTextureLightMapConverter
{
public:

	using FGLTFBuilderContext::FGLTFBuilderContext;

protected:

	UE_API virtual FGLTFJsonTexture* Convert(const ULightMapTexture2D* LightMap) override;
};

#undef UE_API
