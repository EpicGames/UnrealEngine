// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Converters/GLTFConverter.h"
#include "Converters/GLTFMeshAttributesArray.h"

#define UE_API GLTFEXPORTER_API

struct FMeshDescription;

class FGLTFUVBoundsCalculator : public TGLTFConverter<FBox2f, const FMeshDescription*, FGLTFIndexArray, int32>
{
protected:

	UE_API virtual FBox2f Convert(const FMeshDescription* Description, FGLTFIndexArray SectionIndices, int32 TexCoord) override;
};

#undef UE_API
