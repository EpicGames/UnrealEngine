// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Converters/GLTFConverter.h"
#include "Converters/GLTFMeshAttributesArray.h"
#include "Containers/StaticArray.h"

#define UE_API GLTFEXPORTER_API

struct FMeshDescription;

class FGLTFUVDegenerateChecker : public TGLTFConverter<float, const FMeshDescription*, FGLTFIndexArray, int32>
{
protected:

	UE_API virtual float Convert(const FMeshDescription* Description, FGLTFIndexArray SectionIndices, int32 TexCoord) override;

private:

	static bool IsDegenerateTriangle(const TStaticArray<FVector2f, 3>& Points);
	static bool IsDegenerateTriangle(const TStaticArray<FVector3f, 3>& Points);
};

#undef UE_API
