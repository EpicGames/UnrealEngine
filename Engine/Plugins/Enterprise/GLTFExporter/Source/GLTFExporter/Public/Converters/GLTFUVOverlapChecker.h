// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Converters/GLTFConverter.h"
#include "Converters/GLTFMeshAttributesArray.h"

#define UE_API GLTFEXPORTER_API

class UMaterialInterface;
struct FMeshDescription;

class FGLTFUVOverlapChecker : public TGLTFConverter<float, const FMeshDescription*, FGLTFIndexArray, int32>
{
protected:

	UE_API virtual float Convert(const FMeshDescription* Description, FGLTFIndexArray SectionIndices, int32 TexCoord) override;

private:

	static UMaterialInterface* GetMaterial();
};

#undef UE_API
