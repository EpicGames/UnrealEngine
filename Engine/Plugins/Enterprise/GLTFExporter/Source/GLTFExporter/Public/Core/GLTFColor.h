// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if PLATFORM_SUPPORTS_PRAGMA_PACK
	#pragma pack(push, 1)
#endif

template <typename ComponentType>
struct TGLTFColor3
{
	union
	{
		struct
		{
			ComponentType R GCC_PACK(1);
			ComponentType G GCC_PACK(1);
			ComponentType B GCC_PACK(1);
		};

		ComponentType Components[3] GCC_PACK(1);
	};

	TGLTFColor3(
		ComponentType R,
		ComponentType G,
		ComponentType B
	)
		: R(R)
		, G(G)
		, B(B)
	{}
};

template <typename ComponentType>
struct TGLTFColor4
{
	union
	{
		struct
		{
			ComponentType R GCC_PACK(1);
			ComponentType G GCC_PACK(1);
			ComponentType B GCC_PACK(1);
			ComponentType A GCC_PACK(1);
		};

		ComponentType Components[4] GCC_PACK(1);
	};

	TGLTFColor4(
		ComponentType R,
		ComponentType G,
		ComponentType B,
		ComponentType A
	)
		: R(R)
		, G(G)
		, B(B)
		, A(A)
	{}
};

#if PLATFORM_SUPPORTS_PRAGMA_PACK
	#pragma pack(pop)
#endif

typedef TGLTFColor3<float> FGLTFColor3;
typedef TGLTFColor4<float> FGLTFColor4;

typedef TGLTFColor4<uint8> FGLTFUInt8Color4;
