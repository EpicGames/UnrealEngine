// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if PLATFORM_SUPPORTS_PRAGMA_PACK
	#pragma pack(push, 1)
#endif

template <typename ElementType, SIZE_T Dimensions>
struct TGLTFMatrix
{
	ElementType Elements[Dimensions * Dimensions] GCC_PACK(1);

	ElementType& operator()(uint32 Column, uint32 Row)
	{
		return Elements[Column * Dimensions + Row];
	}

	const ElementType& operator()(uint32 Column, uint32 Row) const
	{
		return Elements[Column * Dimensions + Row];
	}
};

#if PLATFORM_SUPPORTS_PRAGMA_PACK
	#pragma pack(pop)
#endif

typedef TGLTFMatrix<float, 2> FGLTFMatrix2;
typedef TGLTFMatrix<float, 3> FGLTFMatrix3;
typedef TGLTFMatrix<float, 4> FGLTFMatrix4;
