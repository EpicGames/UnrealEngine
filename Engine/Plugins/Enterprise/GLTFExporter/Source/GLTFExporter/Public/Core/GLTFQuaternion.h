// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if PLATFORM_SUPPORTS_PRAGMA_PACK
	#pragma pack(push, 1)
#endif

template <typename ComponentType>
struct TGLTFQuaternion
{
	union
	{
		struct
		{
			ComponentType X GCC_PACK(1);
			ComponentType Y GCC_PACK(1);
			ComponentType Z GCC_PACK(1);
			ComponentType W GCC_PACK(1);
		};

		ComponentType Components[4] GCC_PACK(1);
	};

	TGLTFQuaternion(
		ComponentType X,
		ComponentType Y,
		ComponentType Z,
		ComponentType W
	)
		: X(X)
		, Y(Y)
		, Z(Z)
		, W(W)
	{}
};

#if PLATFORM_SUPPORTS_PRAGMA_PACK
	#pragma pack(pop)
#endif

typedef TGLTFQuaternion<float> FGLTFQuaternion;
