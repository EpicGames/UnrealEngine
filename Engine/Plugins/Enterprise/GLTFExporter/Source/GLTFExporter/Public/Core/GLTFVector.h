// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if PLATFORM_SUPPORTS_PRAGMA_PACK
	#pragma pack(push, 1)
#endif

template <typename ComponentType>
struct TGLTFVector2
{
	union
	{
		struct
		{
			ComponentType X GCC_PACK(1);
			ComponentType Y GCC_PACK(1);
		};

		ComponentType Components[2] GCC_PACK(1);
	};

	TGLTFVector2(
		ComponentType X,
		ComponentType Y
	)
		: X(X)
		, Y(Y)
	{}
};

template <typename ComponentType>
struct TGLTFVector3
{
	union
	{
		struct
		{
			ComponentType X GCC_PACK(1);
			ComponentType Y GCC_PACK(1);
			ComponentType Z GCC_PACK(1);
		};

		ComponentType Components[3] GCC_PACK(1);
	};

	TGLTFVector3(
		ComponentType X,
		ComponentType Y,
		ComponentType Z
	)
		: X(X)
		, Y(Y)
		, Z(Z)
	{}
};

template <typename ComponentType>
struct TGLTFVector4
{
	union
	{
		struct
		{
			ComponentType X GCC_PACK(1);
			ComponentType Y GCC_PACK(1);
			ComponentType Z GCC_PACK(1);
			ComponentType W GCC_PACK(1);
		};

		ComponentType Components[4] GCC_PACK(1);
	};

	TGLTFVector4(
		ComponentType X,
		ComponentType Y,
		ComponentType Z,
		ComponentType W
	)
		: X(X)
		, Y(Y)
		, Z(Z)
		, W(W)
	{}
};

#if PLATFORM_SUPPORTS_PRAGMA_PACK
	#pragma pack(pop)
#endif

typedef TGLTFVector2<float> FGLTFVector2;
typedef TGLTFVector3<float> FGLTFVector3;
typedef TGLTFVector4<float> FGLTFVector4;

typedef TGLTFVector4<int8> FGLTFInt8Vector4;
typedef TGLTFVector4<int16> FGLTFInt16Vector4;
