// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Exporters/Exporter.h"
#include "GLTFExporter.generated.h"

#define UE_API GLTFEXPORTER_API

class FGLTFContainerBuilder;
class UGLTFExportOptions;

USTRUCT(BlueprintType)
struct FGLTFExportMessages
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General)
	TArray<FString> Suggestions;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General)
	TArray<FString> Warnings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General)
	TArray<FString> Errors;
};

UCLASS(MinimalAPI, Abstract)
class UGLTFExporter : public UExporter
{
public:

	GENERATED_BODY()

	UE_API explicit UGLTFExporter(const FObjectInitializer& ObjectInitializer);

	UE_API virtual bool ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Archive, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags) override;

	/**
	 * Export the specified object to a glTF file (.gltf or .glb)
	 *
	 * @param Object          The object to export (supported types are UMaterialInterface, UStaticMesh, USkeletalMesh, UWorld, UAnimSequence, ULevelSequence, ULevelVariantSets). Will default to the currently active world if null.
	 * @param FilePath        The filename on disk to save as. Associated textures and binary files will be saved in the same folder, unless file extension is .glb - which results in a self-contained binary file.
	 * @param Options         The various options to use during export. Will default to the project's user-specific editor settings if null.
	 * @param SelectedActors  The set of actors to export, only applicable if the object to export is a UWorld. An empty set results in the export of all actors.
	 * @param OutMessages     The resulting log messages from the export.
	 *
	 * @return true if the object was successfully exported
	 */
	UFUNCTION(BlueprintCallable, Category = "Miscellaneous", meta=(AutoCreateRefTerm="SelectedActors"))
	static UE_API bool ExportToGLTF(UObject* Object, const FString& FilePath, const UGLTFExportOptions* Options, const TSet<AActor*>& SelectedActors, FGLTFExportMessages& OutMessages);

	/**
	 * Export the specified object to a glTF file (.gltf or .glb)
	 *
	 * @param Object          The object to export (supported types are UMaterialInterface, UStaticMesh, USkeletalMesh, UWorld, UAnimSequence, ULevelSequence, ULevelVariantSets). Will default to the currently active world if null.
	 * @param FilePath        The filename on disk to save as. Associated textures and binary files will be saved in the same folder, unless file extension is .glb - which results in a self-contained binary file.
	 * @param Options         The various options to use during export. Will default to the project's user-specific editor settings if null.
	 * @param SelectedActors  The set of actors to export, only applicable if the object to export is a UWorld. An empty set results in the export of all actors.
	 *
	 * @return true if the object was successfully exported
	 */
	static UE_API bool ExportToGLTF(UObject* Object, const FString& FilePath, const UGLTFExportOptions* Options = nullptr, const TSet<AActor*>& SelectedActors = {});

protected:

	UE_API virtual bool AddObject(FGLTFContainerBuilder& Builder, const UObject* Object);

private:

	const UGLTFExportOptions* GetExportOptions();

	FString GetFilePath() const;
	bool IsAutomated() const;
};

#undef UE_API
