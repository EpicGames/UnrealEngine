// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Exporters/GLTFExporter.h"
#include "GLTFLevelExporter.generated.h"

#define UE_API GLTFEXPORTER_API

UCLASS(MinimalAPI)
class UGLTFLevelExporter : public UGLTFExporter
{
public:

	GENERATED_BODY()

	UE_API explicit UGLTFLevelExporter(const FObjectInitializer& ObjectInitializer = FObjectInitializer());

	UE_API virtual bool AddObject(FGLTFContainerBuilder& Builder, const UObject* Object) override;
};

#undef UE_API
