// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Exporters/GLTFExporter.h"
#include "GLTFLevelSequenceExporter.generated.h"

#define UE_API GLTFEXPORTER_API

UCLASS(MinimalAPI)
class UGLTFLevelSequenceExporter : public UGLTFExporter
{
public:

	GENERATED_BODY()

	UE_API explicit UGLTFLevelSequenceExporter(const FObjectInitializer& ObjectInitializer = FObjectInitializer());

	UE_API virtual bool AddObject(FGLTFContainerBuilder& Builder, const UObject* Object) override;
};

#undef UE_API
