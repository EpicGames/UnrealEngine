// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define GLTFEXPORTER_MODULE_NAME TEXT("GLTFExporter")

DECLARE_LOG_CATEGORY_EXTERN(LogGLTFExporter, Log, All);

class UGLTFProxyOptions;

/**
 * The public interface of the GLTFExporter module
 */
class IGLTFExporterModule : public IModuleInterface
{
public:

	/**
	 * Singleton-like access to IGLTFExporter
	 *
	 * @return Returns IGLTFExporter singleton instance, loading the module on demand if needed
	 */
	static IGLTFExporterModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IGLTFExporterModule>(GLTFEXPORTER_MODULE_NAME);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(GLTFEXPORTER_MODULE_NAME);
	}
};
