// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonAccessor : IGLTFJsonIndexedObject
{
	FString Name;

	FGLTFJsonBufferView*   BufferView;
	int64                  ByteOffset;
	int32                  Count;
	EGLTFJsonAccessorType  Type;
	EGLTFJsonComponentType ComponentType;
	bool                   bNormalized;

	int32 MinMaxLength;
	float Min[16];
	float Max[16];

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonAccessor, void>;

	FGLTFJsonAccessor(int32 Index)
		: IGLTFJsonIndexedObject(Index)
		, BufferView(nullptr)
		, ByteOffset(0)
		, Count(0)
		, Type(EGLTFJsonAccessorType::None)
		, ComponentType(EGLTFJsonComponentType::None)
		, bNormalized(false)
		, MinMaxLength(0)
		, Min{0}
	, Max{0}
	{
	}
};

#undef UE_API
