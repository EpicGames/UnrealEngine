// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonAnimationChannelTarget : IGLTFJsonObject
{
	FGLTFJsonNode* Node;
	EGLTFJsonTargetPath Path;

	FGLTFJsonAnimationChannelTarget()
		: Node(nullptr)
		, Path(EGLTFJsonTargetPath::None)
	{
	}

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

struct FGLTFJsonAnimationChannel : IGLTFJsonObject
{
	FGLTFJsonAnimationSampler* Sampler;
	FGLTFJsonAnimationChannelTarget Target;

	FGLTFJsonAnimationChannel()
		: Sampler(nullptr)
	{
	}

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

struct FGLTFJsonAnimationSampler : IGLTFJsonIndexedObject
{
	FGLTFJsonAccessor* Input;
	FGLTFJsonAccessor* Output;

	EGLTFJsonInterpolation Interpolation;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonAnimationSampler, void>;

	FGLTFJsonAnimationSampler(int32 Index)
		: IGLTFJsonIndexedObject(Index)
		, Input(nullptr)
		, Output(nullptr)
		, Interpolation(EGLTFJsonInterpolation::Linear)
	{
	}
};

struct FGLTFJsonAnimation : IGLTFJsonIndexedObject
{
	FString Name;

	TArray<FGLTFJsonAnimationChannel> Channels;
	TGLTFJsonIndexedObjectArray<FGLTFJsonAnimationSampler> Samplers;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonAnimation, void>;

	FGLTFJsonAnimation(int32 Index)
		: IGLTFJsonIndexedObject(Index)
	{
	}
};

#undef UE_API
