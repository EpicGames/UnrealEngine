// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonAsset : IGLTFJsonObject
{
	FString Version;
	FString Generator;
	FString Copyright;

	FGLTFJsonAsset()
		: Version(TEXT("2.0"))
	{
	}

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

#undef UE_API
