// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonBuffer : IGLTFJsonIndexedObject
{
	FString Name;

	FString URI;
	int64   ByteLength;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonBuffer, void>;

	FGLTFJsonBuffer(int32 Index)
		: IGLTFJsonIndexedObject(Index)
		, ByteLength(0)
	{
	}
};

#undef UE_API
