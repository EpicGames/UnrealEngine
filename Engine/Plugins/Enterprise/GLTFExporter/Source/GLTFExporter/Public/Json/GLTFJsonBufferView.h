// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonBufferView : IGLTFJsonIndexedObject
{
	FString Name;

	FGLTFJsonBuffer* Buffer;

	int64 ByteLength;
	int64 ByteOffset;
	int32 ByteStride;

	EGLTFJsonBufferTarget Target;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonBufferView, void>;

	FGLTFJsonBufferView(int32 Index)
		: IGLTFJsonIndexedObject(Index)
		, Buffer(nullptr)
		, ByteLength(0)
		, ByteOffset(0)
		, ByteStride(0)
		, Target(EGLTFJsonBufferTarget::None)
	{
	}
};

#undef UE_API
