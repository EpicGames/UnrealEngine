// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonOrthographic : IGLTFJsonObject
{
	float XMag; // horizontal magnification of the view
	float YMag; // vertical magnification of the view
	float ZFar;
	float ZNear;

	FGLTFJsonOrthographic()
		: XMag(0)
		, YMag(0)
		, ZFar(0)
		, ZNear(0)
	{
	}

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

struct FGLTFJsonPerspective : IGLTFJsonObject
{
	float AspectRatio; // aspect ratio of the field of view
	float YFov; // vertical field of view in radians
	float ZFar;
	float ZNear;

	FGLTFJsonPerspective()
		: AspectRatio(0)
		, YFov(0)
		, ZFar(0)
		, ZNear(0)
	{
	}

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

struct FGLTFJsonCamera : IGLTFJsonIndexedObject
{
	FString Name;

	EGLTFJsonCameraType   Type;
	FGLTFJsonOrthographic Orthographic;
	FGLTFJsonPerspective  Perspective;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonCamera, void>;

	FGLTFJsonCamera(int32 Index)
		: IGLTFJsonIndexedObject(Index)
		, Type(EGLTFJsonCameraType::None)
	{
	}
};

#undef UE_API
