// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/GLTFColor.h"
#include "Json/GLTFJsonArray.h"
#include "Json/GLTFJsonWriter.h"

#define UE_API GLTFEXPORTER_API

template <typename BaseType>
struct TGLTFJsonColor : BaseType, IGLTFJsonArray
{
	TGLTFJsonColor(const BaseType& Other)
		: BaseType(Other)
	{
	}

	TGLTFJsonColor& operator=(const BaseType& Other)
	{
		*static_cast<BaseType*>(this) = Other;
		return *this;
	}

	bool operator==(const BaseType& Other) const
	{
		for (SIZE_T i = 0; i < GetNum(BaseType::Components); ++i)
		{
			if (BaseType::Components[i] != Other.Components[i])
			{
				return false;
			}
		}

		return true;
	}

	bool operator!=(const BaseType& Other) const
	{
		return !(*this == Other);
	}

	virtual void WriteArray(IGLTFJsonWriter& Writer) const override
	{
		for (SIZE_T i = 0; i < GetNum(BaseType::Components); ++i)
		{
			Writer.Write(BaseType::Components[i]);
		}
	}

	bool IsNearlyEqual(const BaseType& Other, float Tolerance = KINDA_SMALL_NUMBER) const
	{
		for (SIZE_T i = 0; i < GetNum(BaseType::Components); ++i)
		{
			if (!FMath::IsNearlyEqual(BaseType::Components[i], Other.Components[i], Tolerance))
			{
				return false;
			}
		}

		return true;
	}
};

struct FGLTFJsonColor3 : TGLTFJsonColor<FGLTFColor3>
{
	static UE_API const FGLTFJsonColor3 Black;
	static UE_API const FGLTFJsonColor3 White;

	using TGLTFJsonColor::TGLTFJsonColor;
	using TGLTFJsonColor::operator=;
};

struct FGLTFJsonColor4 : TGLTFJsonColor<FGLTFColor4>
{
	static UE_API const FGLTFJsonColor4 Black;
	static UE_API const FGLTFJsonColor4 White;

	using TGLTFJsonColor::TGLTFJsonColor;
	using TGLTFJsonColor::operator=;
};

#undef UE_API
