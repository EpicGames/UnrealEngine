// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Json/GLTFJsonEnums.h"
#include "Json/GLTFJsonValue.h"
#include "Json/GLTFJsonArray.h"
#include "Json/GLTFJsonObject.h"
#include "Json/GLTFJsonIndexedObject.h"
#include "Json/GLTFJsonIndexedObjectArray.h"
#include "Json/GLTFJsonColor.h"
#include "Json/GLTFJsonMatrix.h"
#include "Json/GLTFJsonQuaternion.h"
#include "Json/GLTFJsonVector.h"
#include "Json/GLTFJsonWriter.h"

struct FGLTFJsonAccessor;
struct FGLTFJsonAnimation;
struct FGLTFJsonAnimationSampler;
struct FGLTFJsonBuffer;
struct FGLTFJsonBufferView;
struct FGLTFJsonCamera;
struct FGLTFJsonImage;
struct FGLTFJsonMaterial;
struct FGLTFJsonMesh;
struct FGLTFJsonNode;
struct FGLTFJsonSampler;
struct FGLTFJsonScene;
struct FGLTFJsonSkin;
struct FGLTFJsonTexture;
struct FGLTFJsonBackdrop;
struct FGLTFJsonLight;
struct FGLTFJsonLightIES;
struct FGLTFJsonLightIESInstance;
struct FGLTFJsonLightMap;
struct FGLTFJsonSkySphere;
struct FGLTFJsonEpicLevelVariantSets;
struct FGLTFJsonMaterialVariant;
