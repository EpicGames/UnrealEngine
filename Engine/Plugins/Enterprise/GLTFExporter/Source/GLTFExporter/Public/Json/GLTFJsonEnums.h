// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

enum class EGLTFJsonExtension
{
	KHR_LightsPunctual,
	EXT_LightsIES,
	KHR_MaterialsClearCoat,
	KHR_MaterialsEmissiveStrength,
	KHR_MaterialsUnlit,
	KHR_MaterialsVariants,
	KHR_MaterialsIOR,
	KHR_MaterialsSheen,
	KHR_MaterialsTransmission,
	KHR_MaterialsSpecularGlossiness,
	KHR_MaterialsIridescence,
	KHR_MaterialsAnisotropy,
	KHR_MeshQuantization,
	KHR_TextureTransform,
	KHR_MaterialsSpecular,
	EPIC_LightmapTextures
};

enum class EGLTFJsonShadingModel
{
	None = -1,
	Default,
	Unlit,
	ClearCoat,
	Sheen, /*UE Cloth*/
	Transmission, /*UE Thin Translucent*/
	SpecularGlossiness, /*Only for Importer material mapping.*/

	NumShadingModels
};

enum class EGLTFJsonAccessorType
{
	None = -1,
	Scalar,
	Vec2,
	Vec3,
	Vec4,
	Mat2,
	Mat3,
	Mat4
};

enum class EGLTFJsonComponentType
{
	None = -1,
	Int8 = 5120,
	UInt8 = 5121,
	Int16 = 5122,
	UInt16 = 5123,
	Int32 = 5124, // unused
	UInt32 = 5125,
	Float = 5126
};

enum class EGLTFJsonBufferTarget
{
	None = -1,
	ArrayBuffer = 34962,
	ElementArrayBuffer = 34963
};

enum class EGLTFJsonPrimitiveMode
{
	Points = 0,
	Lines = 1,
	LineLoop = 2,
	LineStrip = 3,
	Triangles = 4,
	TriangleStrip = 5,
	TriangleFan = 6
};

enum class EGLTFJsonAlphaMode
{
	None = -1,
	Opaque,
	Mask,
	Blend
};

enum class EGLTFJsonMimeType
{
	None = -1,
	PNG,
	JPEG
};

enum class EGLTFJsonTextureFilter
{
	None = -1,
	Nearest = 9728,
	Linear = 9729,
	NearestMipmapNearest = 9984,
	LinearMipmapNearest = 9985,
	NearestMipmapLinear = 9986,
	LinearMipmapLinear = 9987
};

enum class EGLTFJsonTextureWrap
{
	None = -1,
	Repeat = 10497,
	MirroredRepeat = 33648,
	ClampToEdge = 33071
};

enum class EGLTFJsonInterpolation
{
	None = -1,
	Linear,
	Step,
	CubicSpline,
};

enum class EGLTFJsonTargetPath
{
	None = -1,
	Translation,
	Rotation,
	Scale,
	Weights
};

enum class EGLTFJsonCameraType
{
	None = -1,
	Orthographic,
	Perspective
};

enum class EGLTFJsonLightType
{
	None = -1,
	Directional,
	Point,
	Spot
};
