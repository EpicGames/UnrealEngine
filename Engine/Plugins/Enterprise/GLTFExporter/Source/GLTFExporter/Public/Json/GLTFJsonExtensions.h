// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Json/GLTFJsonEnums.h"

struct FGLTFJsonExtensions
{
	TSet<EGLTFJsonExtension> Used;
	TSet<EGLTFJsonExtension> Required;
};
