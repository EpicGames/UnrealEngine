// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonImage : IGLTFJsonIndexedObject
{
	FString Name;
	FString URI;

	EGLTFJsonMimeType MimeType;

	FGLTFJsonBufferView* BufferView;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonImage, void>;

	FGLTFJsonImage(int32 Index)
		: IGLTFJsonIndexedObject(Index)
		, MimeType(EGLTFJsonMimeType::None)
		, BufferView(nullptr)
	{
	}
};

#undef UE_API
