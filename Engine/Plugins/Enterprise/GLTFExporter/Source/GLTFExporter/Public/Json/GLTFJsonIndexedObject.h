// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonObject.h"

struct IGLTFJsonIndexedObject : IGLTFJsonObject
{
	int32 Index;

protected:

	IGLTFJsonIndexedObject(int32 Index)
		: Index(Index)
	{
	}
};
