// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonArray.h"
#include "Json/GLTFJsonWriter.h"
#include "Json/GLTFJsonIndexedObject.h"

template <typename ElementType, typename = typename TEnableIf<TIsDerivedFrom<ElementType, IGLTFJsonIndexedObject>::Value>::Type>
struct TGLTFJsonIndexedObjectArray : IGLTFJsonArray
{
	using ArrayType = TArray<TUniquePtr<ElementType>>;
	using SizeType = typename ArrayType::SizeType;

	template <typename IteratorType, typename ValueType>
	struct TIterator
	{
		TIterator(const IteratorType& Iterator)
			: Iterator(Iterator)
		{
		}

		ValueType operator*() const
		{
			return (*Iterator).Get();
		}

		TIterator& operator++()
		{
			++Iterator;
			return *this;
		}

		TIterator& operator--()
		{
			--Iterator;
			return *this;
		}

		friend bool operator!=(const TIterator& A, const TIterator& B)
		{
			return A.Iterator != B.Iterator;
		};

	private:

		IteratorType Iterator;
	};

	typedef TIterator<typename ArrayType::RangedForIteratorType, ElementType*> IteratorType;
	typedef TIterator<typename ArrayType::RangedForConstIteratorType, const ElementType*> ConstIteratorType;

	TGLTFJsonIndexedObjectArray() = default;
	TGLTFJsonIndexedObjectArray(TGLTFJsonIndexedObjectArray&& Other) = default;
	TGLTFJsonIndexedObjectArray& operator=(TGLTFJsonIndexedObjectArray&& Other) = default;

	TGLTFJsonIndexedObjectArray(const TGLTFJsonIndexedObjectArray&) = delete;
	TGLTFJsonIndexedObjectArray& operator=(const TGLTFJsonIndexedObjectArray&) = delete;

	ElementType* Add()
	{
		ElementType* Element = new ElementType(Array.Num());
		Array.Add(TUniquePtr<ElementType>(Element));
		return Element;
	}

	//Do not forget to call FixElementIndices after.
	void Remove(int32 Index)
	{
		Array.RemoveAt(Index);
	}

	void FixElementIndices()
	{
		for (size_t ElementIndex = 0; ElementIndex < Array.Num(); ElementIndex++)
		{
			Array[ElementIndex]->Index = ElementIndex;
		}
	}

	bool IsValidIndex(SizeType Index) const
	{
		return Array.IsValidIndex(Index);
	}

	SizeType Num() const
	{
		return Array.Num();
	}

	bool Contains(const ElementType* Element) const
	{
		TUniquePtr<ElementType> TempPtr(Element);
		const bool Result = Array.Contains(TempPtr);
		(void)TempPtr.Release();
		return Result;
	}

	SizeType Find(const ElementType* Element) const
	{
		TUniquePtr<ElementType> TempPtr(Element);
		const SizeType Result = Array.Find(TempPtr);
		(void)TempPtr.Release();
		return Result;
	}

	ElementType* operator[](SizeType Index)
	{
		return Array[Index].Get();
	}

	const ElementType* operator[](SizeType Index) const
	{
		return Array[Index].Get();
	}

	IteratorType begin() { return IteratorType(Array.begin()); }
	IteratorType end() { return IteratorType(Array.end()); }
	ConstIteratorType begin() const { return ConstIteratorType(Array.begin()); }
	ConstIteratorType end() const { return ConstIteratorType(Array.end()); }

	virtual void WriteArray(IGLTFJsonWriter& Writer) const override
	{
		for (const ElementType* Element : *this)
		{
			Writer.Write(*Element);
		}
	}

private:

	ArrayType Array;
};
