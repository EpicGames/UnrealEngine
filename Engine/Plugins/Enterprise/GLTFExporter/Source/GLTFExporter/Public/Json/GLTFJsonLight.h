// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonSpotLight : IGLTFJsonObject
{
	float InnerConeAngle;
	float OuterConeAngle;

	FGLTFJsonSpotLight()
		: InnerConeAngle(0)
		, OuterConeAngle(HALF_PI)
	{
	}

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

struct FGLTFJsonLight : IGLTFJsonIndexedObject
{
	FString Name;

	EGLTFJsonLightType Type;

	FGLTFJsonColor3 Color;

	float Intensity;
	float Range;

	FGLTFJsonSpotLight Spot;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonLight, void>;

	FGLTFJsonLight(int32 Index)
		: IGLTFJsonIndexedObject(Index)
		, Type(EGLTFJsonLightType::None)
		, Color(FGLTFJsonColor3::White)
		, Intensity(1)
		, Range(0)
	{
	}
};


struct FGLTFJsonLightIES : IGLTFJsonIndexedObject
{
	FString Name;

	FString URI;

	FString MimeType;
	FGLTFJsonBufferView* BufferView;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonLightIES, void>;

	FGLTFJsonLightIES(int32 Index)
		: IGLTFJsonIndexedObject(Index)
		, MimeType(TEXT("application/x-ies-lm-63"))
		, BufferView(nullptr)
	{
	}
};

struct FGLTFJsonLightIESInstance : IGLTFJsonIndexedObject
{
	//FGLTFJsonColor3 Color; //Note: While IES extension can store the Color, the Export will save out the Color within the KHR_Lights_Punctual extension.
	float Multiplier;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

	FGLTFJsonLightIES* LightIES;

	bool HasValue() { return LightIES && (LightIES->URI.Len() || LightIES->BufferView); }

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonLightIESInstance, void>;

	FGLTFJsonLightIESInstance(int32 Index)
		: IGLTFJsonIndexedObject(Index)
		, Multiplier(1)
		, LightIES(nullptr)
	{
	}
};

#undef UE_API
