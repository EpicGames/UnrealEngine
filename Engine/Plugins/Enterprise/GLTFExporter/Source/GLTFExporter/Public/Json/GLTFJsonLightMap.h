// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Json/GLTFJsonMaterial.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonLightMap : IGLTFJsonIndexedObject
{
	FString              Name;
	FGLTFJsonTextureInfo Texture;
	FGLTFJsonVector4     LightMapScale;
	FGLTFJsonVector4     LightMapAdd;
	FGLTFJsonVector4     CoordinateScaleBias;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonLightMap, void>;

	FGLTFJsonLightMap(int32 Index)
		: IGLTFJsonIndexedObject(Index)
		, LightMapScale(FGLTFJsonVector4::One)
		, LightMapAdd(FGLTFJsonVector4::Zero)
		, CoordinateScaleBias({ 1, 1, 0, 0 })
	{
	}
};

#undef UE_API
