// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Json/GLTFJsonTextureTransform.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonTextureInfo : IGLTFJsonObject
{
	FGLTFJsonTexture* Index;
	int32 TexCoord;

	FGLTFJsonTextureTransform Transform;

	FGLTFJsonTextureInfo()
		: Index(nullptr)
		, TexCoord(0)
	{
	}

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

struct FGLTFJsonNormalTextureInfo : FGLTFJsonTextureInfo
{
	float Scale;

	FGLTFJsonNormalTextureInfo()
		: Scale(1)
	{
	}

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

struct FGLTFJsonOcclusionTextureInfo : FGLTFJsonTextureInfo
{
	float Strength;

	FGLTFJsonOcclusionTextureInfo()
		: Strength(1)
	{
	}

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

struct FGLTFJsonPBRMetallicRoughness : IGLTFJsonObject
{
	FGLTFJsonColor4 BaseColorFactor;
	FGLTFJsonTextureInfo BaseColorTexture;

	float MetallicFactor;
	float RoughnessFactor;
	FGLTFJsonTextureInfo MetallicRoughnessTexture;

	FGLTFJsonPBRMetallicRoughness()
		: BaseColorFactor(FGLTFJsonColor4::White)
		, MetallicFactor(1)
		, RoughnessFactor(1)
	{
	}

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

struct FGLTFJsonClearCoatExtension : IGLTFJsonObject
{
	float ClearCoatFactor;
	FGLTFJsonTextureInfo ClearCoatTexture;

	float ClearCoatRoughnessFactor;
	FGLTFJsonTextureInfo ClearCoatRoughnessTexture;

	FGLTFJsonNormalTextureInfo ClearCoatNormalTexture;

	FGLTFJsonClearCoatExtension()
		: ClearCoatFactor(0)
		, ClearCoatRoughnessFactor(0)
	{
	}

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

/*
* Only taking advantage of SpecularFactor and SpecularMaps
*/
struct FGLTFJsonSpecularExtension : IGLTFJsonObject
{
	float Factor; //SpecularFactor
	FGLTFJsonTextureInfo Texture; //SpecularMap

	FGLTFJsonSpecularExtension()
		: Factor(0.5)
	{
	}

	//UE's default Specular value is 0.5, while glTF's is 1.
	bool HasValue() const { return ((!FMath::IsNearlyEqual(Factor, 0.5f) && !FMath::IsNearlyEqual(Factor, 1.0f)) || Texture.Index != nullptr); }

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

struct FGLTFJsonIORExtension : IGLTFJsonObject
{
	float Value;

	FGLTFJsonIORExtension()
		: Value(1.5)
	{
	}

	bool HasValue() const { return !FMath::IsNearlyEqual(Value, 1.5f); }

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

struct FGLTFJsonSheenExtension : IGLTFJsonObject
{
	FGLTFJsonColor3         ColorFactor;
	FGLTFJsonTextureInfo    ColorTexture;

	float                   RoughnessFactor;
	FGLTFJsonTextureInfo    RoughnessTexture;

	FGLTFJsonSheenExtension()
		: ColorFactor(FGLTFJsonColor3::Black)
		, RoughnessFactor(0)
	{
	}

	bool HasValue() const { return !ColorFactor.IsNearlyEqual(FGLTFJsonColor3::Black) || ColorTexture.Index != nullptr || !FMath::IsNearlyEqual(RoughnessFactor, 0.f) || RoughnessTexture.Index != nullptr; }

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

struct FGLTFJsonTransmissionExtension : IGLTFJsonObject
{
	float                   Factor; //transmissionFactor
	FGLTFJsonTextureInfo    Texture; //transmissionTexture

	FGLTFJsonTransmissionExtension()
		: Factor(0.f)
	{
	}

	bool HasValue() const { return !FMath::IsNearlyEqual(Factor, 0.f) || Texture.Index != nullptr; }

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

struct FGLTFJsonSpecularGlossinessExtension : IGLTFJsonObject
{
	FGLTFJsonColor4         DiffuseFactor;
	FGLTFJsonTextureInfo    DiffuseTexture;

	FGLTFJsonColor3         SpecularFactor;
	float                   GlossinessFactor;
	FGLTFJsonTextureInfo    SpecularGlossinessTexture;

	FGLTFJsonSpecularGlossinessExtension()
		: DiffuseFactor(FGLTFJsonColor4::White)
		, SpecularFactor(FGLTFJsonColor3::White)
		, GlossinessFactor(1)
	{
	}

	bool HasValue() const { return !DiffuseFactor.IsNearlyEqual(FGLTFJsonColor4::White) || DiffuseTexture.Index != nullptr || !SpecularFactor.IsNearlyEqual(FGLTFJsonColor3::White) || !FMath::IsNearlyEqual(GlossinessFactor, 1.0f) || SpecularGlossinessTexture.Index != nullptr; }

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

struct FGLTFJsonIridescenceExtension : IGLTFJsonObject
{
	float                   IridescenceFactor;
	FGLTFJsonTextureInfo    IridescenceTexture;
	float                   IridescenceIOR;

	float                   IridescenceThicknessMinimum;
	float                   IridescenceThicknessMaximum;
	FGLTFJsonTextureInfo    IridescenceThicknessTexture;


	FGLTFJsonIridescenceExtension()
		: IridescenceFactor(0.f)
		, IridescenceIOR(1.3f)
		, IridescenceThicknessMinimum(100.f)
		, IridescenceThicknessMaximum(400.f)
	{
	}

	bool HasValue() const { return !FMath::IsNearlyEqual(IridescenceFactor, 0.f) || IridescenceTexture.Index != nullptr || !FMath::IsNearlyEqual(IridescenceIOR, 1.3f) || 
		!FMath::IsNearlyEqual(IridescenceThicknessMinimum, 100.f) || !FMath::IsNearlyEqual(IridescenceThicknessMaximum, 400.f) || IridescenceThicknessTexture.Index != nullptr; }

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

struct FGLTFJsonAnisotropyExtension : IGLTFJsonObject
{
	float                AnisotropyStrength = 0.0f;
	float                AnisotropyRotation = 0.0f;
	FGLTFJsonTextureInfo AnisotropyTexture;

	bool HasValue() const { return !FMath::IsNearlyEqual(AnisotropyStrength, 0.f) || !FMath::IsNearlyEqual(AnisotropyRotation, 0.f) || AnisotropyTexture.Index != nullptr; }

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

struct FGLTFJsonMaterial : IGLTFJsonIndexedObject
{
	FString Name;

	EGLTFJsonShadingModel ShadingModel;

	FGLTFJsonPBRMetallicRoughness PBRMetallicRoughness;
	FGLTFJsonSpecularGlossinessExtension PBRSpecularGlossiness;

	FGLTFJsonNormalTextureInfo NormalTexture;
	FGLTFJsonOcclusionTextureInfo OcclusionTexture;

	FGLTFJsonTextureInfo EmissiveTexture;
	FGLTFJsonColor3 EmissiveFactor;
	float EmissiveStrength;

	EGLTFJsonAlphaMode AlphaMode;
	float AlphaCutoff;

	bool DoubleSided;

	FGLTFJsonClearCoatExtension    ClearCoat;
	FGLTFJsonSpecularExtension     Specular;
	FGLTFJsonIORExtension          IOR;
	FGLTFJsonSheenExtension        Sheen;
	FGLTFJsonTransmissionExtension Transmission;
	FGLTFJsonIridescenceExtension  Iridescence;
	FGLTFJsonAnisotropyExtension   Anisotropy;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonMaterial, void>;

	FGLTFJsonMaterial(int32 Index)
		: IGLTFJsonIndexedObject(Index)
		, ShadingModel(EGLTFJsonShadingModel::Default)
		, EmissiveFactor(FGLTFJsonColor3::Black)
		, EmissiveStrength(1.0f)
		, AlphaMode(EGLTFJsonAlphaMode::Opaque)
		, AlphaCutoff(0.5f)
		, DoubleSided(false)
	{
	}
};

#undef UE_API
