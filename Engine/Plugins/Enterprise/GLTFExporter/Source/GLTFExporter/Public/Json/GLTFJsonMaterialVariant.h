// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonMaterialVariantMapping : IGLTFJsonObject
{
	FGLTFJsonMaterial* Material;
	TArray<FGLTFJsonMaterialVariant*> Variants;

	FGLTFJsonMaterialVariantMapping()
		: Material(nullptr)
	{
	}

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;
};

struct FGLTFJsonMaterialVariant : IGLTFJsonIndexedObject
{
	FString Name;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonMaterialVariant, void>;

	FGLTFJsonMaterialVariant(int32 Index)
		: IGLTFJsonIndexedObject(Index)
	{
	}
};

#undef UE_API
