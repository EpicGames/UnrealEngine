// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/GLTFMatrix.h"
#include "Json/GLTFJsonArray.h"
#include "Json/GLTFJsonWriter.h"

#define UE_API GLTFEXPORTER_API

template <typename BaseType>
struct TGLTFJsonMatrix : BaseType, IGLTFJsonArray
{
	TGLTFJsonMatrix(const BaseType& Other)
		: BaseType(Other)
	{
	}

	TGLTFJsonMatrix& operator=(const BaseType& Other)
	{
		*static_cast<BaseType*>(this) = Other;
		return *this;
	}

	bool operator==(const BaseType& Other) const
	{
		for (SIZE_T i = 0; i < GetNum(BaseType::Elements); ++i)
		{
			if (BaseType::Elements[i] != Other.Elements[i])
			{
				return false;
			}
		}

		return true;
	}

	bool operator!=(const BaseType& Other) const
	{
		return !(*this == Other);
	}

	virtual void WriteArray(IGLTFJsonWriter& Writer) const override
	{
		for (SIZE_T i = 0; i < GetNum(BaseType::Elements); ++i)
		{
			Writer.Write(BaseType::Elements[i]);
		}
	}

	bool IsNearlyEqual(const BaseType& Other, float Tolerance = KINDA_SMALL_NUMBER) const
	{
		for (SIZE_T i = 0; i < GetNum(BaseType::Elements); ++i)
		{
			if (!FMath::IsNearlyEqual(BaseType::Elements[i], Other.Elements[i], Tolerance))
			{
				return false;
			}
		}

		return true;
	}
};

struct FGLTFJsonMatrix2 : TGLTFJsonMatrix<FGLTFMatrix2>
{
	static UE_API const FGLTFJsonMatrix2 Identity;

	using TGLTFJsonMatrix::TGLTFJsonMatrix;
	using TGLTFJsonMatrix::operator=;
};

struct FGLTFJsonMatrix3 : TGLTFJsonMatrix<FGLTFMatrix3>
{
	static UE_API const FGLTFJsonMatrix3 Identity;

	using TGLTFJsonMatrix::TGLTFJsonMatrix;
	using TGLTFJsonMatrix::operator=;
};

struct FGLTFJsonMatrix4 : TGLTFJsonMatrix<FGLTFMatrix4>
{
	static UE_API const FGLTFJsonMatrix4 Identity;

	using TGLTFJsonMatrix::TGLTFJsonMatrix;
	using TGLTFJsonMatrix::operator=;
};

#undef UE_API
