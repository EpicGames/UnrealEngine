// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Json/GLTFJsonMaterialVariant.h"

#define UE_API GLTFEXPORTER_API

//For MorphTargets
struct FGLTFJsonTarget : IGLTFJsonObject
{
	FGLTFJsonAccessor* Position;
	FGLTFJsonAccessor* Normal;

	FGLTFJsonTarget()
		: Position(nullptr)
		, Normal(nullptr)
	{
	}

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

	UE_API bool HasValue() const;
};

struct FGLTFJsonAttributes : IGLTFJsonObject
{
	FGLTFJsonAccessor* Position;
	FGLTFJsonAccessor* Color0;
	FGLTFJsonAccessor* Normal;
	FGLTFJsonAccessor* Tangent;

	TArray<FGLTFJsonAccessor*> TexCoords;
	TArray<FGLTFJsonAccessor*> Joints;
	TArray<FGLTFJsonAccessor*> Weights;

	FGLTFJsonAttributes()
		: Position(nullptr)
		, Color0(nullptr)
		, Normal(nullptr)
		, Tangent(nullptr)
	{
	}

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

	UE_API bool HasValue() const;
};

struct FGLTFJsonPrimitive : IGLTFJsonObject
{
	FGLTFJsonAttributes Attributes;
	FGLTFJsonAccessor* Indices;
	FGLTFJsonMaterial* Material;
	EGLTFJsonPrimitiveMode Mode;

	TArray<FGLTFJsonMaterialVariantMapping> MaterialVariantMappings;

	TArray<FGLTFJsonTarget> Targets; //MorphTargets

	FGLTFJsonPrimitive()
		: Indices(nullptr)
		, Material(nullptr)
		, Mode(EGLTFJsonPrimitiveMode::Triangles)
	{
	}

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

	UE_API bool HasValue() const;
};

struct FGLTFJsonMesh : IGLTFJsonIndexedObject
{
	FString Name;

	TArray<FGLTFJsonPrimitive> Primitives;

	TArray<FString> TargetNames;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

	UE_API bool HasValue() const;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonMesh, void>;

	FGLTFJsonMesh(int32 Index)
		: IGLTFJsonIndexedObject(Index)
	{
	}
};

#undef UE_API
