// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Json/GLTFJsonTransform.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonNode : FGLTFJsonTransform, IGLTFJsonIndexedObject
{
	FString Name;

	FGLTFJsonCamera* Camera;
	FGLTFJsonSkin*   Skin;
	FGLTFJsonMesh*   Mesh;
	FGLTFJsonLight*  Light;
	FGLTFJsonLightMap* LightMap;
	FGLTFJsonLightIESInstance*  LightIESInstance;

	TArray<FGLTFJsonNode*> Children;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonNode, void>;

	FGLTFJsonNode(int32 Index)
		: IGLTFJsonIndexedObject(Index)
		, Camera(nullptr)
		, Skin(nullptr)
		, Mesh(nullptr)
		, Light(nullptr)
		, LightMap(nullptr)
		, LightIESInstance(nullptr)
	{
	}
};

#undef UE_API
