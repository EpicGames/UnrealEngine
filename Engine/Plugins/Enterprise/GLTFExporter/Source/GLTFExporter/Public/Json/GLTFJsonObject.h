// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonValue.h"

#define UE_API GLTFEXPORTER_API

struct IGLTFJsonObject : IGLTFJsonValue
{
	UE_API virtual void WriteValue(IGLTFJsonWriter& Writer) const override final;

	virtual void WriteObject(IGLTFJsonWriter& Writer) const = 0;
};

#undef UE_API
