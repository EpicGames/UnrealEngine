// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/GLTFQuaternion.h"
#include "Json/GLTFJsonArray.h"
#include "Json/GLTFJsonWriter.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonQuaternion : TGLTFQuaternion<float>, IGLTFJsonArray
{
	static UE_API const TGLTFQuaternion Identity;

	FGLTFJsonQuaternion(const TGLTFQuaternion& Other)
		: TGLTFQuaternion(Other)
	{
	}

	FGLTFJsonQuaternion& operator=(const TGLTFQuaternion& Other)
	{
		*static_cast<TGLTFQuaternion*>(this) = Other;
		return *this;
	}

	bool operator==(const TGLTFQuaternion& Other) const
	{
		for (SIZE_T i = 0; i < GetNum(Components); ++i)
		{
			if (Components[i] != Other.Components[i])
			{
				return false;
			}
		}

		return true;
	}

	bool operator!=(const TGLTFQuaternion& Other) const
	{
		return !(*this == Other);
	}

	virtual void WriteArray(IGLTFJsonWriter& Writer) const override
	{
		for (int32 i = 0; i < 4; ++i)
		{
			Writer.Write(Components[i]);
		}
	}

	bool IsNearlyEqual(const TGLTFQuaternion& Other, float Tolerance = KINDA_SMALL_NUMBER) const
	{
		return (FMath::Abs(X - Other.X) <= Tolerance && FMath::Abs(Y - Other.Y) <= Tolerance && FMath::Abs(Z - Other.Z) <= Tolerance && FMath::Abs(W - Other.W) <= Tolerance)
			|| (FMath::Abs(X + Other.X) <= Tolerance && FMath::Abs(Y + Other.Y) <= Tolerance && FMath::Abs(Z + Other.Z) <= Tolerance && FMath::Abs(W + Other.W) <= Tolerance);
	}
};

#undef UE_API
