// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"
#include "Json/GLTFJsonAsset.h"
#include "Json/GLTFJsonAccessor.h"
#include "Json/GLTFJsonAnimation.h"
#include "Json/GLTFJsonBuffer.h"
#include "Json/GLTFJsonBufferView.h"
#include "Json/GLTFJsonCamera.h"
#include "Json/GLTFJsonImage.h"
#include "Json/GLTFJsonMaterial.h"
#include "Json/GLTFJsonMesh.h"
#include "Json/GLTFJsonNode.h"
#include "Json/GLTFJsonSampler.h"
#include "Json/GLTFJsonScene.h"
#include "Json/GLTFJsonSkin.h"
#include "Json/GLTFJsonTexture.h"
#include "Json/GLTFJsonLight.h"
#include "Json/GLTFJsonLightMap.h"
#include "Json/GLTFJsonMaterialVariant.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonRoot : IGLTFJsonObject
{
	FGLTFJsonAsset Asset;

	FGLTFJsonExtensions Extensions;

	FGLTFJsonScene* DefaultScene;

	TGLTFJsonIndexedObjectArray<FGLTFJsonAccessor>   Accessors;
	TGLTFJsonIndexedObjectArray<FGLTFJsonAnimation>  Animations;
	TGLTFJsonIndexedObjectArray<FGLTFJsonBuffer>     Buffers;
	TGLTFJsonIndexedObjectArray<FGLTFJsonBufferView> BufferViews;
	TGLTFJsonIndexedObjectArray<FGLTFJsonCamera>     Cameras;
	TGLTFJsonIndexedObjectArray<FGLTFJsonMaterial>   Materials;
	TGLTFJsonIndexedObjectArray<FGLTFJsonMesh>       Meshes; // Important! : FGLTFJsonMeshes are validated in "FGLTFJsonBuilder::ValidateAndFixGLTFJson" and any that's found invalid (has no value) it will be removed from the list and deleted.
															 //					Any references to said deleted item need to be removed as well (for example Nodes.Mesh (1 line below))
	TGLTFJsonIndexedObjectArray<FGLTFJsonNode>       Nodes;
	TGLTFJsonIndexedObjectArray<FGLTFJsonImage>      Images;
	TGLTFJsonIndexedObjectArray<FGLTFJsonSampler>    Samplers;
	TGLTFJsonIndexedObjectArray<FGLTFJsonScene>      Scenes;
	TGLTFJsonIndexedObjectArray<FGLTFJsonSkin>       Skins;
	TGLTFJsonIndexedObjectArray<FGLTFJsonTexture>    Textures;
	TGLTFJsonIndexedObjectArray<FGLTFJsonLight>      Lights;
	TGLTFJsonIndexedObjectArray<FGLTFJsonLightMap>   LightMaps;
	TGLTFJsonIndexedObjectArray<FGLTFJsonLightIES>   LightIESs;
	TGLTFJsonIndexedObjectArray<FGLTFJsonLightIESInstance>  LightIESInstances;
	TGLTFJsonIndexedObjectArray<FGLTFJsonMaterialVariant>   MaterialVariants;

	FGLTFJsonRoot()
		: DefaultScene(nullptr)
	{
	}

	FGLTFJsonRoot(FGLTFJsonRoot&&) = default;
	FGLTFJsonRoot& operator=(FGLTFJsonRoot&&) = default;

	FGLTFJsonRoot(const FGLTFJsonRoot&) = delete;
	FGLTFJsonRoot& operator=(const FGLTFJsonRoot&) = delete;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

	UE_API void WriteJson(FArchive& Archive, bool bPrettyJson, float DefaultTolerance);
};

#undef UE_API
