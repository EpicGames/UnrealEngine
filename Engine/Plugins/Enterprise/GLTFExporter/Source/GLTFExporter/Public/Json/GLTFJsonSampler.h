// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonSampler : IGLTFJsonIndexedObject
{
	FString Name;

	EGLTFJsonTextureFilter MinFilter;
	EGLTFJsonTextureFilter MagFilter;

	EGLTFJsonTextureWrap WrapS;
	EGLTFJsonTextureWrap WrapT;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonSampler, void>;

	FGLTFJsonSampler(int32 Index)
		: IGLTFJsonIndexedObject(Index)
		, MinFilter(EGLTFJsonTextureFilter::None)
		, MagFilter(EGLTFJsonTextureFilter::None)
		, WrapS(EGLTFJsonTextureWrap::Repeat)
		, WrapT(EGLTFJsonTextureWrap::Repeat)
	{
	}
};

#undef UE_API
