// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonScene : IGLTFJsonIndexedObject
{
	FString Name;

	TArray<FGLTFJsonNode*> Nodes;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonScene, void>;

	FGLTFJsonScene(int32 Index)
		: IGLTFJsonIndexedObject(Index)
	{
	}
};

#undef UE_API
