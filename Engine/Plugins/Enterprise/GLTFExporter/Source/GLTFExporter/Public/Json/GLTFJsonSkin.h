// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonSkin : IGLTFJsonIndexedObject
{
	FString Name;

	FGLTFJsonAccessor* InverseBindMatrices;
	FGLTFJsonNode* Skeleton;

	TArray<FGLTFJsonNode*> Joints;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonSkin, void>;

	FGLTFJsonSkin(int32 Index)
		: IGLTFJsonIndexedObject(Index)
		, InverseBindMatrices(nullptr)
		, Skeleton(nullptr)
	{
	}
};

#undef UE_API
