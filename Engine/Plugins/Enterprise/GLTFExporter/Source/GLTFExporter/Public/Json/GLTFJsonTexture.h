// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonTexture : IGLTFJsonIndexedObject
{
	FString Name;

	FGLTFJsonSampler* Sampler;

	FGLTFJsonImage* Source;

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

protected:

	friend TGLTFJsonIndexedObjectArray<FGLTFJsonTexture, void>;

	FGLTFJsonTexture(int32 Index)
		: IGLTFJsonIndexedObject(Index)
		, Sampler(nullptr)
		, Source(nullptr)
	{
	}
};

#undef UE_API
