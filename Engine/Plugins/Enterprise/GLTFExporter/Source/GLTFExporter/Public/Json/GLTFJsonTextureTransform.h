// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonTextureTransform : IGLTFJsonObject
{
	FGLTFJsonVector2 Offset;
	FGLTFJsonVector2 Scale;
	float Rotation;

	FGLTFJsonTextureTransform()
		: Offset(FGLTFJsonVector2::Zero)
		, Scale(FGLTFJsonVector2::One)
		, Rotation(0)
	{
	}

	UE_API virtual void WriteObject(IGLTFJsonWriter& Writer) const override;

	UE_API bool IsNearlyEqual(const FGLTFJsonTextureTransform& Other, float Tolerance = KINDA_SMALL_NUMBER) const;
	UE_API bool IsExactlyEqual(const FGLTFJsonTextureTransform& Other) const;

	UE_API bool IsNearlyDefault(float Tolerance = KINDA_SMALL_NUMBER) const;
	UE_API bool IsExactlyDefault() const;
};

#undef UE_API
