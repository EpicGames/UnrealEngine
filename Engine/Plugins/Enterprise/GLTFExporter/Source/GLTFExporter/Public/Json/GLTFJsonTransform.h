// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonCore.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonTransform
{
	FGLTFJsonVector3    Translation;
	FGLTFJsonQuaternion Rotation;
	FGLTFJsonVector3    Scale;

	static UE_API const FGLTFJsonTransform Identity;

	FGLTFJsonTransform()
		: Translation(FGLTFJsonVector3::Zero)
		, Rotation(FGLTFJsonQuaternion::Identity)
		, Scale(FGLTFJsonVector3::One)
	{
	}

	bool operator==(const FGLTFJsonTransform& Other) const
	{
		return Translation == Other.Translation
			&& Rotation == Other.Rotation
			&& Scale == Other.Scale;
	}

	bool operator!=(const FGLTFJsonTransform& Other) const
	{
		return Translation != Other.Translation
			|| Rotation != Other.Rotation
			|| Scale != Other.Scale;
	}

	UE_API void WriteValue(IGLTFJsonWriter& Writer) const;

	bool IsNearlyEqual(const FGLTFJsonTransform& Other, float Tolerance = KINDA_SMALL_NUMBER) const
	{
		return Translation.IsNearlyEqual(Other.Translation, Tolerance)
			&& Rotation.IsNearlyEqual(Other.Rotation, Tolerance)
			&& Scale.IsNearlyEqual(Other.Scale, Tolerance);
	}
};

#undef UE_API
