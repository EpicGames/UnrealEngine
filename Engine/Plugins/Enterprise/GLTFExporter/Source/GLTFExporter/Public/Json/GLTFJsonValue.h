// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IGLTFJsonWriter;

struct IGLTFJsonValue
{
	virtual ~IGLTFJsonValue() = default;

	virtual void WriteValue(IGLTFJsonWriter& Writer) const = 0;
};
