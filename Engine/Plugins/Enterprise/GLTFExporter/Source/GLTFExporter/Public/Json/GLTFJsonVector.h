// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/GLTFVector.h"
#include "Json/GLTFJsonArray.h"
#include "Json/GLTFJsonWriter.h"

#define UE_API GLTFEXPORTER_API

template <typename BaseType>
struct TGLTFJsonVector : BaseType, IGLTFJsonArray
{
	TGLTFJsonVector(const BaseType& Other)
		: BaseType(Other)
	{
	}

	TGLTFJsonVector& operator=(const BaseType& Other)
	{
		*static_cast<BaseType*>(this) = Other;
		return *this;
	}

	bool operator==(const BaseType& Other) const
	{
		for (SIZE_T i = 0; i < GetNum(BaseType::Components); ++i)
		{
			if (BaseType::Components[i] != Other.Components[i])
			{
				return false;
			}
		}

		return true;
	}

	bool operator!=(const BaseType& Other) const
	{
		return !(*this == Other);
	}

	virtual void WriteArray(IGLTFJsonWriter& Writer) const override
	{
		for (SIZE_T i = 0; i < GetNum(BaseType::Components); ++i)
		{
			Writer.Write(BaseType::Components[i]);
		}
	}

	bool IsNearlyEqual(const BaseType& Other, float Tolerance = KINDA_SMALL_NUMBER) const
	{
		for (SIZE_T i = 0; i < GetNum(BaseType::Components); ++i)
		{
			if (!FMath::IsNearlyEqual(BaseType::Components[i], Other.Components[i], Tolerance))
			{
				return false;
			}
		}

		return true;
	}
};

struct FGLTFJsonVector2 : TGLTFJsonVector<FGLTFVector2>
{
	static UE_API const FGLTFJsonVector2 Zero;
	static UE_API const FGLTFJsonVector2 One;

	using TGLTFJsonVector::TGLTFJsonVector;
	using TGLTFJsonVector::operator=;
};

struct FGLTFJsonVector3 : TGLTFJsonVector<FGLTFVector3>
{
	static UE_API const FGLTFJsonVector3 Zero;
	static UE_API const FGLTFJsonVector3 One;

	using TGLTFJsonVector::TGLTFJsonVector;
	using TGLTFJsonVector::operator=;
};

struct FGLTFJsonVector4 : TGLTFJsonVector<FGLTFVector4>
{
	static UE_API const FGLTFJsonVector4 Zero;
	static UE_API const FGLTFJsonVector4 One;

	using TGLTFJsonVector::TGLTFJsonVector;
	using TGLTFJsonVector::operator=;
};

#undef UE_API
