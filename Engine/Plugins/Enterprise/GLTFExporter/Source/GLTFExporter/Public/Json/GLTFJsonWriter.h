// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Json/GLTFJsonValue.h"
#include "Json/GLTFJsonIndexedObject.h"
#include "Json/GLTFJsonExtensions.h"
#include "Utilities/GLTFJsonUtilities.h"

#define UE_API GLTFEXPORTER_API

class IGLTFJsonWriter
{
public:

	float DefaultTolerance;

	FGLTFJsonExtensions& Extensions;

	IGLTFJsonWriter(FGLTFJsonExtensions& Extensions)
		: DefaultTolerance(0)
		, Extensions(Extensions)
	{
	}

	static UE_API TSharedRef<IGLTFJsonWriter> Create(FArchive& Archive, bool bPrettyJson, FGLTFJsonExtensions& Extensions);

	virtual ~IGLTFJsonWriter() = default;

	virtual void Close() = 0;

	virtual void Write(bool Boolean) = 0;
	virtual void Write(int32 Number) = 0;
	virtual void Write(int64 Number) = 0;
	virtual void Write(float Number) = 0;
	virtual void Write(const FString& String) = 0;
	virtual void Write(TYPE_OF_NULLPTR) = 0;
	virtual void SetIdentifier(const FString& Identifier) = 0;

	virtual void StartObject() = 0;
	virtual void EndObject() = 0;
	virtual void StartArray() = 0;
	virtual void EndArray() = 0;

	void Write(const TCHAR* String)
	{
		Write(FString(String));
	}

	void Write(const IGLTFJsonValue& Value)
	{
		Value.WriteValue(*this);
	}

	void Write(const IGLTFJsonIndexedObject* Object)
	{
		if (Object != nullptr)
		{
			Write(Object->Index);
		}
		else
		{
			Write(INDEX_NONE);
		}
	}

	template <typename EnumType, typename = typename TEnableIf<TIsEnum<EnumType>::Value>::Type>
	void Write(EnumType Enum)
	{
		Write(FGLTFJsonUtilities::GetValue(Enum));
	}

	template <class ElementType, typename = typename TEnableIf<!std::is_same_v<ElementType, TCHAR>>::Type>
	void Write(const ElementType* Array, SIZE_T ArraySize)
	{
		StartArray();
		for (SIZE_T i = 0; i < ArraySize; ++i)
		{
			Write(Array[i]);
		}
		EndArray();
	}

	template <class ElementType, SIZE_T ArraySize, typename = typename TEnableIf<!std::is_same_v<ElementType, TCHAR>>::Type>
	void Write(const ElementType (&Array)[ArraySize])
	{
		Write(Array, ArraySize);
	}

	template <class ElementType>
	void Write(const TUniquePtr<ElementType>& Pointer)
	{
		if (Pointer.IsValid())
		{
			Write(*Pointer.Get());
		}
		else
		{
			Write(nullptr);
		}
	}

	template <class ElementType>
	void Write(const TArray<ElementType>& Array)
	{
		StartArray();
		for (const auto& Element : Array)
		{
			Write(Element);
		}
		EndArray();
	}

	template <class ElementType>
	void Write(const TSet<ElementType>& Set)
	{
		StartArray();
		for (const auto& Element : Set)
		{
			Write(Element);
		}
		EndArray();
	}

	void Write(const FString& Identifier, bool Boolean)
	{
		SetIdentifier(Identifier);
		Write(Boolean);
	}

	void Write(const FString& Identifier, int32 Number)
	{
		SetIdentifier(Identifier);
		Write(Number);
	}

	void Write(const FString& Identifier, int64 Number)
	{
		SetIdentifier(Identifier);
		Write(Number);
	}

	void Write(const FString& Identifier, float Number)
	{
		SetIdentifier(Identifier);
		Write(Number);
	}

	void Write(const FString& Identifier, const FString& String)
	{
		SetIdentifier(Identifier);
		Write(String);
	}

	void Write(const FString& Identifier, const TCHAR* String)
	{
		SetIdentifier(Identifier);
		Write(String);
	}

	void Write(const FString& Identifier, const IGLTFJsonValue& Value)
	{
		SetIdentifier(Identifier);
		Write(Value);
	}

	void Write(const FString& Identifier, const IGLTFJsonIndexedObject* Value)
	{
		SetIdentifier(Identifier);
		Write(Value);
	}

	template <typename EnumType, typename = typename TEnableIf<TIsEnum<EnumType>::Value>::Type>
	void Write(const FString& Identifier, EnumType Enum)
	{
		SetIdentifier(Identifier);
		Write(Enum);
	}

	template <class ElementType, typename = typename TEnableIf<!std::is_same_v<ElementType, TCHAR>>::Type>
	void Write(const FString& Identifier, const ElementType* Array, SIZE_T ArraySize)
	{
		SetIdentifier(Identifier);
		Write(Array, ArraySize);
	}

	template <class ElementType, SIZE_T ArraySize, typename = typename TEnableIf<!std::is_same_v<ElementType, TCHAR>>::Type>
	void Write(const FString& Identifier, const ElementType (&Array)[ArraySize])
	{
		SetIdentifier(Identifier);
		Write(Array);
	}

	template <class ElementType>
	void Write(const FString& Identifier, const TUniquePtr<ElementType>& Pointer)
	{
		SetIdentifier(Identifier);
		Write(Pointer);
	}

	template <class ElementType>
	void Write(const FString& Identifier, const TArray<ElementType>& Array)
	{
		SetIdentifier(Identifier);
		Write(Array);
	}

	template <class ElementType>
	void Write(const FString& Identifier, const TSet<ElementType>& Set)
	{
		SetIdentifier(Identifier);
		Write(Set);
	}

	void StartObject(const FString& Identifier)
	{
		SetIdentifier(Identifier);
		StartObject();
	}

	void StartArray(const FString& Identifier)
	{
		SetIdentifier(Identifier);
		StartArray();
	}

	void StartExtensions()
	{
		StartObject(TEXT("extensions"));
	}

	void EndExtensions()
	{
		EndObject();
	}

	void Write(const EGLTFJsonExtension& Extension, const IGLTFJsonValue& Value, bool bIsRequired = false)
	{
		Extensions.Used.Add(Extension);

		if (bIsRequired)
		{
			Extensions.Required.Add(Extension);
		}

		Write(FGLTFJsonUtilities::GetValue(Extension), Value);
	}

	void StartExtension(const EGLTFJsonExtension& Extension, bool bIsRequired = false)
	{
		Extensions.Used.Add(Extension);

		if (bIsRequired)
		{
			Extensions.Required.Add(Extension);
		}

		StartObject(FGLTFJsonUtilities::GetValue(Extension));
	}

	void EndExtension()
	{
		EndObject();
	}
};

#undef UE_API
