// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UserData/GLTFMaterialUserData.h"
#include "GLTFExportOptions.generated.h"

#define UE_API GLTFEXPORTER_API

UENUM(BlueprintType)
enum class EGLTFTextureImageFormat : uint8
{
	/** Don't export any textures. */
	None,
	/** Always use PNG (lossless compression). */
	PNG,
	/** If texture does not have an alpha channel, use JPEG (lossy compression); otherwise fallback to PNG. */
	JPEG UMETA(DisplayName = "JPEG (if no alpha)")
};

UENUM(BlueprintType)
enum class EGLTFMaterialVariantMode : uint8
{
	/** Never export material variants. */
	None,
	/** Export material variants but only use a simple quad if a material input needs to be baked out. */
	Simple,
	/** Export material variants and allow usage of the mesh data if a material input needs to be baked out with vertex data. */
	UseMeshData,
};

UCLASS(MinimalAPI, BlueprintType, Config=EditorPerProjectUserSettings, HideCategories=(DebugProperty))
class UGLTFExportOptions : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Scale factor used for exporting all assets (0.01 by default) for conversion from centimeters (Unreal default) to meters (glTF). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = General)
	float ExportUniformScale;

	/** If enabled, the preview mesh for a standalone animation or material asset will also be exported. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = General)
	bool bExportPreviewMesh;

	/** If enabled, floating-point-based JSON properties that are nearly equal to their default value will not be exported and thus regarded as exactly default, reducing JSON size. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = General, AdvancedDisplay)
	bool bSkipNearDefaultValues;

	/** If enabled, the copyright notice from project settings will be included as metadata in the glTF asset. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = General, AdvancedDisplay)
	bool bIncludeCopyrightNotice;

	/** If enabled, materials that have a proxy defined in their user data, will be exported using that proxy instead. This setting won't affect proxy materials exported or referenced directly. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material)
	bool bExportProxyMaterials;

	/** If enabled, materials imported with the Interchange-glTF importer will be directly mapped for the Exporter. bExport material options below will be ignored. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material)
	bool bUseImporterMaterialMapping;

	/** If enabled, materials with shading model unlit will be properly exported. Uses extension KHR_materials_unlit. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material)
	bool bExportUnlitMaterials;

	/** If enabled, materials with shading model clear coat will be properly exported. Uses extension KHR_materials_clearcoat. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material)
	bool bExportClearCoatMaterials;

	/** If enabled, materials with shading model cloth will be properly exported. Uses extension KHR_materials_sheen. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material)
	bool bExportClothMaterials;

	/** If enabled, materials with shading model thin translucency will be exported. Export is only partial. Uses extension KHR_materials_transmission. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material)
	bool bExportThinTranslucentMaterials;

	/** If enabled, materials using the Importer's SpecularGlossiness material function will be exported. Uses extension KHR_materials_pbrSpecularGlossiness. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material)
	bool bExportSpecularGlossinessMaterials;

	/** If enabled, allows materials to have an emissive factor that exceeds the standard range [0.0, 1.0]. Uses extension KHR_materials_emissive_strength. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material)
	bool bExportEmissiveStrength;

	/** Bake mode determining if and how a material input is baked out to a texture. Baking is only used for non-trivial material inputs (i.e. not simple texture or constant expressions). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material)
	EGLTFMaterialBakeMode BakeMaterialInputs;

	/** Default size of the baked out texture (containing the material input). Can be overridden by material- and input-specific bake settings, see GLTFMaterialExportOptions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material, Meta = (EditCondition = "BakeMaterialInputs != EGLTFMaterialBakeMode::Disabled"))
	FGLTFMaterialBakeSize DefaultMaterialBakeSize;

	/** Default filtering mode used when sampling the baked out texture. Can be overridden by material- and input-specific bake settings, see GLTFMaterialExportOptions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material, Meta = (EditCondition = "BakeMaterialInputs != EGLTFMaterialBakeMode::Disabled", ValidEnumValues = "TF_Nearest, TF_Bilinear, TF_Trilinear"))
	TEnumAsByte<TextureFilter> DefaultMaterialBakeFilter;

	/** Default addressing mode used when sampling the baked out texture. Can be overridden by material- and input-specific bake settings, see GLTFMaterialExportOptions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material, Meta = (EditCondition = "BakeMaterialInputs != EGLTFMaterialBakeMode::Disabled"))
	TEnumAsByte<TextureAddress> DefaultMaterialBakeTiling;

	/** Input-specific default bake settings that override the general defaults above. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material, Meta = (EditCondition = "BakeMaterialInputs != EGLTFMaterialBakeMode::Disabled"))
	TMap<EGLTFMaterialPropertyGroup, FGLTFOverrideMaterialBakeSettings> DefaultInputBakeSettings;

	/** Default LOD level used for exporting a mesh. Can be overridden by component or asset settings (e.g. minimum or forced LOD level). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Mesh, Meta = (ClampMin = "0"))
	int32 DefaultLevelOfDetail;

	/** If enabled, exports the SourceModel. If false, exports the Render Data. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Mesh)
	bool bExportSourceModel;

	/** If enabled, export vertex color. Not recommended due to vertex colors always being used as a base color multiplier in glTF, regardless of material. Often producing undesirable results. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Mesh)
	bool bExportVertexColors;

	/** If enabled, export vertex bone weights and indices in skeletal meshes. Necessary for animation sequences. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Mesh)
	bool bExportVertexSkinWeights;

	/** If enabled, make skeletal meshes into root nodes to strictly comply with the glTF specification. Final bone transforms remain the same and visual results are unaffected. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Mesh)
	bool bMakeSkinnedMeshesRoot;

	/** If enabled, use quantization for vertex tangents and normals, reducing size. Requires extension KHR_mesh_quantization, which may result in the mesh not loading in some glTF viewers. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Mesh)
	bool bUseMeshQuantization;

	/** If enabled, exports Morph Targets. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Mesh)
	bool bExportMorphTargets;

	/** If enabled, export level sequences. Only transform tracks are currently supported. The level sequence will be played at the assigned display rate. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Animation)
	bool bExportLevelSequences;

	/** If enabled, export single animation asset used by a skeletal mesh component. Export of vertex skin weights must be enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Animation, Meta = (EditCondition = "bExportVertexSkinWeights"))
	bool bExportAnimationSequences;

	/** Desired image format used for exported textures. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Texture)
	EGLTFTextureImageFormat TextureImageFormat;

	/** Level of compression used for textures exported with lossy image formats, 0 (default) or value between 1 (worst quality, best compression) and 100 (best quality, worst compression). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Texture, Meta = (ClampMin = "0", ClampMax = "100", EditCondition = "TextureImageFormat == EGLTFTextureImageFormat::JPEG"))
	int32 TextureImageQuality;

	/** If enabled, export UV offset and scale/tiling used in materials. Uses extension KHR_texture_transform. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Texture, Meta = (EditCondition = "TextureImageFormat != EGLTFTextureImageFormat::None"))
	bool bExportTextureTransforms;

	/** If enabled, export lightmaps (created by Lightmass) when exporting a level. Uses extension EPIC_lightmap_textures, which is supported by Unreal's glTF viewer. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Texture, Meta = (EditCondition = "TextureImageFormat != EGLTFTextureImageFormat::None"))
	bool bExportLightmaps;

	/** If enabled, exported normalmaps will be adjusted from Unreal to glTF convention (i.e. the green channel is flipped). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Texture, Meta = (EditCondition = "TextureImageFormat != EGLTFTextureImageFormat::None"))
	bool bAdjustNormalmaps;

	/** If enabled, export actors and components that are flagged as hidden in-game. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Scene)
	bool bExportHiddenInGame;

	/** If enabled, export directional, point, and spot light components. Uses extension KHR_lights_punctual.
	* Also exports IES light profiles using the EXT_lights_ies extension.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Scene)
	bool bExportLights;

	/** If enabled, export camera components. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Scene)
	bool bExportCameras;

	/** Mode determining if and how to export material variants that change the materials property on a static or skeletal mesh component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = VariantSets, Meta = (EditCondition = "VariantSetsMode != EGLTFVariantSetsMode::None"))
	EGLTFMaterialVariantMode ExportMaterialVariants;

	UFUNCTION(BlueprintCallable, Category = General)
	UE_API void ResetToDefault();
};

#undef UE_API
