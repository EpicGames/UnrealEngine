// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Options/GLTFExportOptions.h"
#include "GLTFProxyOptions.generated.h"

#define UE_API GLTFEXPORTER_API

UCLASS(MinimalAPI, BlueprintType, Config=EditorPerProjectUserSettings, HideCategories=(DebugProperty))
class UGLTFProxyOptions : public UObject
{
	GENERATED_UCLASS_BODY()

	/** If enabled, a material input may be baked out to a texture (using a simple quad). Baking is only used for non-trivial material inputs (i.e. not simple texture or constant expressions). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material)
	bool bBakeMaterialInputs;

	/** If enabled, materials with shading model thin translucency will be used. Conversion is only partial.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material)
	bool bUseThinTranslucentShadingModel;

	/** Default size of the baked out texture (containing the material input). Can be overridden by material- and input-specific bake settings, see GLTFMaterialExportOptions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material, Meta = (EditCondition = "bBakeMaterialInputs"))
	FGLTFMaterialBakeSize DefaultMaterialBakeSize;

	/** Default filtering mode used when sampling the baked out texture. Can be overridden by material- and input-specific bake settings, see GLTFMaterialExportOptions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material, Meta = (EditCondition = "bBakeMaterialInputs", ValidEnumValues = "TF_Nearest, TF_Bilinear, TF_Trilinear"))
	TEnumAsByte<TextureFilter> DefaultMaterialBakeFilter;

	/** Default addressing mode used when sampling the baked out texture. Can be overridden by material- and input-specific bake settings, see GLTFMaterialExportOptions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material, Meta = (EditCondition = "bBakeMaterialInputs"))
	TEnumAsByte<TextureAddress> DefaultMaterialBakeTiling;

	/** Input-specific default bake settings that override the general defaults above. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material, Meta = (EditCondition = "bBakeMaterialInputs"))
	TMap<EGLTFMaterialPropertyGroup, FGLTFOverrideMaterialBakeSettings> DefaultInputBakeSettings;

	UFUNCTION(BlueprintCallable, Category = General)
	UE_API void ResetToDefault();
};

#undef UE_API
