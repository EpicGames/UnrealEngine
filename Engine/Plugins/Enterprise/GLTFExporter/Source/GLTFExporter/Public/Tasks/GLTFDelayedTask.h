// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum class EGLTFTaskPriority : uint8
{
	Animation,
	Mesh,
	Material,
	Texture,
	MAX
};

class FGLTFDelayedTask
{
public:

	const EGLTFTaskPriority Priority;

	FGLTFDelayedTask(EGLTFTaskPriority Priority)
		: Priority(Priority)
	{
	}

	virtual ~FGLTFDelayedTask() = default;

	virtual FString GetName() = 0;

	virtual void Process() = 0;
};
