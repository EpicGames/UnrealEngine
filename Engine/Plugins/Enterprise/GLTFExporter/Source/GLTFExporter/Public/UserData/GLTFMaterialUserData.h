// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture.h"
#include "Engine/AssetUserData.h"
#include "GLTFMaterialUserData.generated.h"

#define UE_API GLTFEXPORTER_API

class UMaterialInterface;

UENUM(BlueprintType)
enum class EGLTFMaterialBakeMode : uint8
{
	/** Never bake material inputs. */
	Disabled,
	/** Only use a simple quad if a material input needs to be baked out. */
	Simple,
	/** Allow usage of the mesh data if a material input needs to be baked out with vertex data. */
	UseMeshData,
};

UENUM(BlueprintType)
enum class EGLTFMaterialPropertyGroup : uint8
{
	None UMETA(DisplayName = "None"),

	BaseColorOpacity UMETA(DisplayName = "Base Color + Opacity (Mask)"),
	MetallicRoughness UMETA(DisplayName = "Metallic + Roughness"),
	EmissiveColor UMETA(DisplayName = "Emissive Color"),
	Normal UMETA(DisplayName = "Normal"),
	AmbientOcclusion UMETA(DisplayName = "Ambient Occlusion"),
	ClearCoatRoughness UMETA(DisplayName = "Clear Coat + Clear Coat Roughness"),
	ClearCoatBottomNormal UMETA(DisplayName = "Clear Coat Bottom Normal"),
};

USTRUCT(Blueprintable)
struct FGLTFMaterialBakeSize
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "")
	int32 X = 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "")
	int32 Y = 1;

	/** If enabled, bake size is based on the largest texture used in the material input's expression graph. If none found, bake size will fall back to the explicit dimensions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "")
	bool bAutoDetect = false;

	static FGLTFMaterialBakeSize Default;
};

USTRUCT(Blueprintable)
struct FGLTFOverrideMaterialBakeSettings
{
	GENERATED_BODY()

	FGLTFOverrideMaterialBakeSettings();

	/** If enabled, default size will be overridden by the corresponding property. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "", meta = (InlineEditConditionToggle))
	bool bOverrideSize;

	/** Overrides default size of the baked out texture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "", meta = (EditCondition = "bOverrideSize"))
	FGLTFMaterialBakeSize Size;

	/** If enabled, default filtering mode will be overridden by the corresponding property. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "", meta = (InlineEditConditionToggle))
	bool bOverrideFilter;

	/** Overrides the default filtering mode used when sampling the baked out texture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "", meta = (EditCondition = "bOverrideFilter", ValidEnumValues = "TF_Nearest, TF_Bilinear, TF_Trilinear"))
	TEnumAsByte<TextureFilter> Filter;

	/** If enabled, default addressing mode will be overridden by the corresponding property. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "", meta = (InlineEditConditionToggle))
	bool bOverrideTiling;

	/** Overrides the default addressing mode used when sampling the baked out texture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "", meta = (EditCondition = "bOverrideTiling"))
	TEnumAsByte<TextureAddress> Tiling;
};

/** glTF-specific user data that can be added to material assets to override export options */
UCLASS(MinimalAPI, BlueprintType, meta = (DisplayName = "GLTF Material Export Options"))
class UGLTFMaterialExportOptions : public UAssetUserData
{
	GENERATED_BODY()

public:

	// TODO: add support for overriding more export options

	/** If assigned, export will use the proxy instead of the original material. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "General")
	TObjectPtr<UMaterialInterface> Proxy;

	/** Default bake settings for this material in general. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Override Bake Settings", meta = (ShowOnlyInnerProperties))
	FGLTFOverrideMaterialBakeSettings Default;

	/** Input-specific bake settings that override the defaults above. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Override Bake Settings")
	TMap<EGLTFMaterialPropertyGroup, FGLTFOverrideMaterialBakeSettings> Inputs;

	static UE_API const UMaterialInterface* ResolveProxy(const UMaterialInterface* Material);

	static UE_API FGLTFMaterialBakeSize GetBakeSizeForPropertyGroup(const UMaterialInterface* Material, EGLTFMaterialPropertyGroup PropertyGroup, FGLTFMaterialBakeSize DefaultValue);
	static UE_API TextureFilter GetBakeFilterForPropertyGroup(const UMaterialInterface* Material, EGLTFMaterialPropertyGroup PropertyGroup, TextureFilter DefaultValue);
	static UE_API TextureAddress GetBakeTilingForPropertyGroup(const UMaterialInterface* Material, EGLTFMaterialPropertyGroup PropertyGroup, TextureAddress DefaultValue);

private:

	template <typename Predicate>
	static const FGLTFOverrideMaterialBakeSettings* GetBakeSettingsByPredicate(const UMaterialInterface* Material, EGLTFMaterialPropertyGroup PropertyGroup, Predicate Pred);
};

#undef UE_API
