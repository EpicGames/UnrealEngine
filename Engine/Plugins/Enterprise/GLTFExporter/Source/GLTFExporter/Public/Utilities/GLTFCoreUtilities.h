// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimTypes.h"
#include "Camera/CameraTypes.h"
#include "Core/GLTFVector.h"
#include "Core/GLTFColor.h"
#include "Core/GLTFMatrix.h"
#include "Core/GLTFQuaternion.h"
#include "Json/GLTFJsonEnums.h"
#include "Engine/EngineTypes.h"
#include "Engine/Texture.h"
#include "PackedNormal.h"
#include "RHIDefinitions.h"
#include "SceneTypes.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFCoreUtilities
{
	static UE_API float ConvertLength(const float Length, const float ConversionScale = 0.01);

	static UE_API FGLTFVector3 ConvertVector(const FVector3f& Vector);

	static UE_API FGLTFVector3 ConvertPosition(const FVector3f& Position, const float ConversionScale = 0.01);

	static UE_API FGLTFVector3 ConvertScale(const FVector3f& Scale);

	static UE_API FGLTFVector3 ConvertNormal(const FVector3f& Normal);
	static UE_API FGLTFInt8Vector4 ConvertNormal(const FPackedNormal& Normal);
	static UE_API FGLTFInt16Vector4 ConvertNormal(const FPackedRGBA16N& Normal);

	static UE_API FGLTFVector3 ConvertNormalDelta(const FVector3f& NormalDelta);
	static UE_API FGLTFInt8Vector4 ConvertNormalDelta(const FPackedNormal& NormalDelta);
	static UE_API FGLTFInt16Vector4 ConvertNormalDelta(const FPackedRGBA16N& NormalDelta);

	static UE_API FGLTFVector4 ConvertTangent(const FVector3f& Tangent, const FVector4f& Normal = FVector4f(ForceInitToZero));
	static UE_API FGLTFVector4 ConvertTangent(const FVector4f& Tangent);
	static UE_API FGLTFInt8Vector4 ConvertTangent(const FPackedNormal& Tangent, const FPackedNormal& Normal = FPackedNormal());
	static UE_API FGLTFInt16Vector4 ConvertTangent(const FPackedRGBA16N& Tangent, const FPackedRGBA16N& Normal = FPackedRGBA16N());

	static UE_API FGLTFVector2 ConvertUV(const FVector2f& UV);
	static UE_API FGLTFVector2 ConvertUV(const FVector2DHalf& UV);

	static UE_API FGLTFColor4 ConvertColor(const FLinearColor& Color);
	static UE_API FGLTFColor3 ConvertColor3(const FLinearColor& Color);
	static UE_API FGLTFUInt8Color4 ConvertColor(const FColor& Color);

	static UE_API FGLTFQuaternion ConvertRotation(const FRotator3f& Rotation);
	static UE_API FGLTFQuaternion ConvertRotation(const FQuat4f& Rotation);

	static UE_API FGLTFMatrix4 ConvertMatrix(const FMatrix44f& Matrix);

	static UE_API FGLTFMatrix4 ConvertTransform(const FTransform3f& Transform, const float ConversionScale = 0.01);

	static UE_API float ConvertFieldOfView(float FOVInDegrees, float AspectRatio);

	static UE_API float ConvertLightAngle(const float Angle);

	static UE_API FGLTFQuaternion GetLocalCameraRotation();
	static UE_API FGLTFQuaternion GetLocalLightRotation();

	static UE_API EGLTFJsonCameraType ConvertCameraType(ECameraProjectionMode::Type ProjectionMode);
	static UE_API EGLTFJsonLightType ConvertLightType(ELightComponentType ComponentType);

	static UE_API EGLTFJsonInterpolation ConvertInterpolation(const EAnimInterpolationType Type);

	static UE_API EGLTFJsonShadingModel ConvertShadingModel(EMaterialShadingModel ShadingModel);
	static UE_API FString GetShadingModelString(EGLTFJsonShadingModel ShadingModel);

	static UE_API EGLTFJsonAlphaMode ConvertAlphaMode(EBlendMode Mode);

	static UE_API EGLTFJsonTextureWrap ConvertWrap(TextureAddress Address);

	static UE_API EGLTFJsonTextureFilter ConvertMinFilter(TextureFilter Filter);
	static UE_API EGLTFJsonTextureFilter ConvertMagFilter(TextureFilter Filter);

	template <typename ComponentType>
	static EGLTFJsonComponentType GetComponentType()
	{
		if constexpr (std::is_same_v<ComponentType, int8  >) return EGLTFJsonComponentType::Int8;
		else if constexpr (std::is_same_v<ComponentType, uint8 >) return EGLTFJsonComponentType::UInt8;
		else if constexpr (std::is_same_v<ComponentType, int16 >) return EGLTFJsonComponentType::Int16;
		else if constexpr (std::is_same_v<ComponentType, uint16>) return EGLTFJsonComponentType::UInt16;
		else if constexpr (std::is_same_v<ComponentType, int32 >) return EGLTFJsonComponentType::Int32;
		else if constexpr (std::is_same_v<ComponentType, uint32>) return EGLTFJsonComponentType::UInt32;
		else if constexpr (std::is_same_v<ComponentType, float >) return EGLTFJsonComponentType::Float;
		else return EGLTFJsonComponentType::None;
	}
};

#undef UE_API
