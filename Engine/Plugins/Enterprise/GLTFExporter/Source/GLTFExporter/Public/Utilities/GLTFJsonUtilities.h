// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Json/GLTFJsonEnums.h"

#define UE_API GLTFEXPORTER_API

struct FGLTFJsonUtilities
{
	template <typename EnumType, typename = typename TEnableIf<TIsEnum<EnumType>::Value>::Type>
	static int32 GetValue(EnumType Enum)
	{
		return static_cast<int32>(Enum);
	}

	static UE_API const TCHAR* GetValue(EGLTFJsonExtension Enum);
	static UE_API const TCHAR* GetValue(EGLTFJsonAlphaMode Enum);
	static UE_API const TCHAR* GetValue(EGLTFJsonMimeType Enum);
	static UE_API const TCHAR* GetValue(EGLTFJsonAccessorType Enum);
	static UE_API const TCHAR* GetValue(EGLTFJsonCameraType Enum);
	static UE_API const TCHAR* GetValue(EGLTFJsonLightType Enum);
	static UE_API const TCHAR* GetValue(EGLTFJsonInterpolation Enum);
	static UE_API const TCHAR* GetValue(EGLTFJsonTargetPath Enum);
	static UE_API const TCHAR* GetValue(EGLTFJsonShadingModel Enum);
};

#undef UE_API
