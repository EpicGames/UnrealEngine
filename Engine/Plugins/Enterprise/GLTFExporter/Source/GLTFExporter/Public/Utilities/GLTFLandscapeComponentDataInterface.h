// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class ULandscapeComponent;
class ULandscapeLayerInfoObject;

struct FGLTFLandscapeComponentDataInterface
{
protected:
	int32 ComponentSubsectionSizeQuads = 0;
	int32 ComponentSizeQuads = 0;
	int32 ComponentSizeVerts = 0;
	int32 ComponentSubsectionSizeVerts = 0;
	int32 ComponentNumSubsections = 0;

	TArray64<uint8> HeightMipDataSource;
	FColor* HeightMipData = nullptr;
	int32 HeightmapStride = 0;
	int32 HeightmapComponentOffsetX = 0;
	int32 HeightmapComponentOffsetY = 0;
	int32 HeightmapSubsectionOffset = 0;

	TArray64<uint8> XYOffsetMipDataSource;
	FColor* XYOffsetMipData = nullptr;

	const ULandscapeComponent& Component;
	int32 MipLevel;

public:
	FGLTFLandscapeComponentDataInterface(const ULandscapeComponent& Component, int32 ExportLOD);

	void ComponentXYToSubsectionXY(int32 CompX, int32 CompY, int32& SubNumX, int32& SubNumY, int32& SubX, int32& SubY) const;

	int32 TexelXYToIndex(int32 TexelX, int32 TexelY) const;

	void VertexXYToTexelXY(int32 VertX, int32 VertY, int32& OutX, int32& OutY) const;

	void VertexIndexToXY(int32 VertexIndex, int32& OutX, int32& OutY) const;

	int32 VertexIndexToTexel(int32 VertexIndex) const;

	bool GetWeightmapTextureData(ULandscapeLayerInfoObject* InLayerInfo,
		TArray<uint8>& OutData,
		bool bInUseEditingWeightmap = false, bool bInRemoveSubsectionDuplicates = false);

	FColor GetHeightData(int32 LocalX, int32 LocalY) const;

	UE_DEPRECATED(5.7, "RetopologizeTool/XYOffset deprecated with the removal of non-edit layer landscapes")
	FColor GetXYOffsetData(int32 LocalX, int32 LocalY) const;

	UE_DEPRECATED(5.7, "RetopologizeTool/XYOffset deprecated with the removal of non-edit layer landscapes")
	void GetXYOffset(int32 X, int32 Y, float& XOffset, float& YOffset) const;

	void GetPositionNormalUV(int32 LocalX, int32 LocalY,
		FVector3f& Position,
		FVector3f& Normal,
		FVector2f& UV) const;
};