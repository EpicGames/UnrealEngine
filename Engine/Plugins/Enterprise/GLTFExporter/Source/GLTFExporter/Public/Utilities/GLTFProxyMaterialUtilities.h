// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Json/GLTFJsonEnums.h"
#include "Materials/MaterialParameters.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"

#define UE_API GLTFEXPORTER_API

class UMaterial;
class UMaterialInterface;
class UMaterialInstanceConstant;
class UTexture;

class FGLTFProxyMaterialUtilities
{
public:

#if WITH_EDITOR
	static UE_API UMaterialInstanceConstant* CreateProxyMaterial(EGLTFJsonShadingModel ShadingModel, UObject* Outer = GetTransientPackage(), FName Name = NAME_None, EObjectFlags Flags = RF_NoFlags);
#endif

	static UE_API bool IsProxyMaterial(const UMaterial* Material);
	static UE_API bool IsProxyMaterial(const UMaterialInterface* Material);

	static UE_API UMaterial* GetBaseMaterial(EGLTFJsonShadingModel ShadingModel);

	static UE_API UMaterialInterface* GetProxyMaterial(UMaterialInterface* OriginalMaterial);
	static UE_API void SetProxyMaterial(UMaterialInterface* OriginalMaterial, UMaterialInterface* ProxyMaterial);

	static UE_API bool GetParameterValue(const UMaterialInterface* Material, const FHashedMaterialParameterInfo& ParameterInfo, float& OutValue, bool NonDefaultOnly = false);
	static UE_API bool GetParameterValue(const UMaterialInterface* Material, const FHashedMaterialParameterInfo& ParameterInfo, FLinearColor& OutValue, bool NonDefaultOnly = false);
	static UE_API bool GetParameterValue(const UMaterialInterface* Material, const FHashedMaterialParameterInfo& ParameterInfo, UTexture*& OutValue, bool NonDefaultOnly = false);

#if WITH_EDITOR
	static UE_API void SetParameterValue(UMaterialInstanceConstant* Material, const FHashedMaterialParameterInfo& ParameterInfo, float Value, bool NonDefaultOnly = false);
	static UE_API void SetParameterValue(UMaterialInstanceConstant* Material, const FHashedMaterialParameterInfo& ParameterInfo, const FLinearColor& Value, bool NonDefaultOnly = false);
	static UE_API void SetParameterValue(UMaterialInstanceConstant* Material, const FHashedMaterialParameterInfo& ParameterInfo, UTexture* Value, bool NonDefaultOnly = false);
#endif

	static UE_API bool GetTwoSided(const UMaterialInterface* Material, bool& OutValue, bool NonDefaultOnly = false);
	static UE_API bool GetIsThinSurface(const UMaterialInterface* Material, bool& OutValue, bool NonDefaultOnly = false);
	static UE_API bool GetBlendMode(const UMaterialInterface* Material, EBlendMode& OutValue, bool NonDefaultOnly = false);
	static UE_API bool GetOpacityMaskClipValue(const UMaterialInterface* Material, float& OutValue, bool NonDefaultOnly = false);

#if WITH_EDITOR
	static UE_API void SetTwoSided(UMaterialInstanceConstant* Material, bool Value, bool NonDefaultOnly = false);
	static UE_API void SetIsThinSurface(UMaterialInstanceConstant* Material, bool Value, bool NonDefaultOnly = false);
	static UE_API void SetBlendMode(UMaterialInstanceConstant* Material, EBlendMode Value, bool NonDefaultOnly = false);
	static UE_API void SetOpacityMaskClipValue(UMaterialInstanceConstant* Material, float Value, bool NonDefaultOnly = false);
#endif
};

#undef UE_API
