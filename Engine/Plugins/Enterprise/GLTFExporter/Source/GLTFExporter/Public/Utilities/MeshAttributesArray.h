// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

template <typename T>
class FMeshAttributesArray : public TArray<T>
{
public:

	using TArray<T>::TArray;
	using TArray<T>::operator=;

	friend uint32 GetTypeHash(const FMeshAttributesArray& AttributesArray)
	{
		uint32 Hash = GetTypeHash(AttributesArray.Num());
		for (const T& Attribute : AttributesArray)
		{
			Hash = HashCombine(Hash, GetTypeHash(Attribute));
		}
		return Hash;
	}
};