// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

#if WITH_EDITORONLY_DATA

#include "BoneIndices.h"
#include "MeshDescription.h" //for mesh attribute containers
#include "Utilities/MeshAttributesArray.h"

class USplineMeshComponent;

namespace UE::MeshParser
{
	struct FPrimitiveTargetDescription
	{
		bool bHasPositions = false;
		FMeshAttributesArray<FVector3f>         Positions;

		bool bHasNormals = false;
		FMeshAttributesArray<FVector3f>         Normals;
	};

	struct FMeshPrimitiveDescription
	{
		int32                                   MaterialIndex = INDEX_NONE;

		FMeshAttributesArray<int32>             Indices;
		FMeshAttributesArray<FVector3f>         Positions;
		FMeshAttributesArray<FVector3f>         Normals;
		FMeshAttributesArray<FVector4f>         Tangents;
		TArray<FMeshAttributesArray<FVector2f>> UVs;

		FMeshAttributesArray<FColor>            VertexColors;

		TArray<FMeshAttributesArray<UE::Math::TIntVector4<FBoneIndexType>>> JointInfluences; //TODO: Per Group (make group size configurable)
		TArray<FMeshAttributesArray<UE::Math::TIntVector4<uint16>>>         JointWeights; //TODO: Per Group (make group size configurable)

		TArray<FPrimitiveTargetDescription>		Targets; //MorphTargets

		bool IsEmpty();

		//Except for Joint related containers
		void PrepareContainers(const int32& IndexCount, const int32& AttributesCount, const int32& UVCount, bool bHasVertexColors, const int32& TargetCount);

		void PrepareJointContainers(const uint32& JointGroupCount, const int32& AttributesCount);
	};

	struct FMeshDetails
	{
		int32 NumberOfPrimitives = 0;
		int32 UVCount = 0;
		bool bHasVertexColors = false;
	};

	struct FExportConfigs
	{
		bool bExportVertexSkinWeights = false;
		bool bExportVertexColors = false;
		const USplineMeshComponent* SplineMeshComponent = nullptr;
		int32 SkeletonInfluenceCountPerGroup = INT32_MAX;
		bool bExportMorphTargets = false;

		FExportConfigs(bool bInExportVertexSkinWeights,
			bool bInExportVertexColors,
			const USplineMeshComponent* InSplineMeshComponent,
			int32 InSkeletonInfluenceCountPerGroup,
			bool bInExportMorphTargets);
	};

	/*
	* Note0: We are compacting the Vertex Attributes and Indices per Primitives, with no overlapping.
	* Note1: Primary approach tries compaction on a Vertex bases, if that fails (due to VertexInstance Attributes differing on a per Vertex base), it falls back onto VertexInstance based compaction.
	*			Parsing does not do compaction for identical Vertex attribute values.
	*			It's either driven by FVertexIDs or FVertexInstanceIDs within Indices.
	* Note2: TODO: perhaps add a solution path where we keep the unfication across primitives?
	* class T: Static/Skeletal Material Slots
	*/
	template <class T>
	struct FMeshDescriptionParser
	{
		FMeshDescriptionParser(const FMeshDescription* InMeshDescription, const TArray<T>& InMaterialSlots);

		/*
		* Parses FMeshDescription into the FMeshPrimitiveMeshDescriptions array based on the provided ExportConfigs.
		*/
		void Parse(TArray<FMeshPrimitiveDescription>& OutMeshPrimitiveDescriptions, TArray<FString>& OutTargetNames, const FExportConfigs& ExportConfigs);

	private:
		/*
		* Tries to parse the PolygonGroup to Primitive, using only the Vertices.
		* Checks if the VertexInstances use identical values per Vertices.
		* If any of the VertexInstances of a given Vertex do not match Values we fail out and return false.
		*	(in which case we are falling back on using ParseVertexInstanceBased function)
		*/
		bool ParseVertexBased(const TArrayView<const FTriangleID>& TriangleIDs, FMeshPrimitiveDescription& MeshPrimitiveDescription, const FExportConfigs& ExportConfigs);

		/*
		* Parses the PolygonGroup to Primitive, using the VertexInstances.
		* Each VertexInstance will be a new entry in the VertexAttribute containers (aka a new 'Vertex' in the parsed data set).
		*/
		void ParseVertexInstanceBased(const TArrayView<const FTriangleID>& TriangleIDs, FMeshPrimitiveDescription& MeshPrimitiveDescription, const FExportConfigs& ExportConfigs);

		/*
		* Gets the Material Index for the PolygonGroupID, based on the MeshDescription and the MaterialSlots provided in the struct's constructor.
		*/
		int32 GetMaterialIndex(const FPolygonGroupID& PolygonGroupID);

		//Inputs:
		const FMeshDescription* MeshDescription; //Primarily used for GetPolygonGroupTriangles, GetTriangleVertexInstances
		const TArray<T>& MaterialSlots;

		//Helper variable acquired from Inputs:
		TVertexAttributesConstRef<FVector3f> VertexPositions;
		TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceNormals;
		TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceTangents;
		TVertexInstanceAttributesConstRef<float> VertexInstanceBinormalSigns;
		TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs;
		TVertexInstanceAttributesConstRef<FVector4f> VertexInstanceColors;

		//MorphTarget Deltas:
		TArray<TVertexAttributesConstRef<FVector3f>> TargetVertexPositionDeltas;
		TArray<TVertexInstanceAttributesConstRef<FVector3f>> TargetVertexInstanceNormalDeltas;
		TArray<FString> TargetNames;
		int32 NumOfTargets;

		TConstArrayView<FVertexID> VertexInstanceIdToVertexId;
		TPolygonGroupAttributesConstRef<FName> PolygonGroupMaterialSlotNames;
	public:
		//MeshDetails like UVCount/NumberOfPrimitives/bHasVertexColor for ease of access:
		FMeshDetails MeshDetails;
	};

	namespace Private
	{
		/**
		* Queries number of valid polygons (polygons that have at least 1 triangle).
		* It queries it from Source Model (MeshDescription), and not the RenderData.
		*/
		template <class T>
		int32 GetNumberOfValidPolygons(const T* Mesh/*either UStaticMesh or USkeletalMesh*/, const int32 LODIndex);
	}
}

#endif